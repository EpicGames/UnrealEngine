// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HeadlessChaos : TestModuleRules
{
	static HeadlessChaos()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "HeadlessChaos";
			TestMetadata.TestShortName = "Headless Chaos";
			TestMetadata.UsesCatch2 = false;
		}
	}

	public HeadlessChaos(ReadOnlyTargetRules Target) : base(Target, false)
	{
		PublicIncludePathModuleNames.Add("Launch");

		// For testing access to private Chaos classes
		PrivateIncludePaths.Add("Runtime/Experimental/Chaos/Private");

		SetupModulePhysicsSupport(Target);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "ApplicationCore",
				"Core",
				"CoreUObject",
				"Projects",
                "GoogleTest",
				"GeometryCore",
				"ChaosVehiclesCore"
            }
        );


		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PublicDefinitions.Add("GTEST_OS_WINDOWS=1");
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			PublicDefinitions.Add("GTEST_OS_MAC=1");
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.IOS))
		{
			PublicDefinitions.Add("GTEST_OS_IOS=1");
		}
		else if (Target.Platform == UnrealTargetPlatform.Android)
		{
			PublicDefinitions.Add("GTEST_OS_LINUX_ANDROID=1");
		}
		else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
		{
			PublicDefinitions.Add("GTEST_OS_LINUX=1");
		}

		PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
	}
}
