// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms("Win64")]
public class HeadlessChaosTarget : TestTargetRules
{
	public HeadlessChaosTarget(TargetInfo Target) : base(Target)
	{
        bHasExports = false;

		GlobalDefinitions.Add("CHAOS_SERIALIZE_OUT=1");

		// our gtest does not have debug crt libs
		bDebugBuildsActuallyUseDebugCRT = false;

		// Enable additional debug functionality in the constraint management system to allow better unit testing
		GlobalDefinitions.Add("CHAOS_CONSTRAINTHANDLE_DEBUG_ENABLED=1");
		BuildEnvironment = TargetBuildEnvironment.Unique;
		bWarningsAsErrors = true;
	}
}
