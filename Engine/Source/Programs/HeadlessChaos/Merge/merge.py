#!/usr/bin/env python3

import argparse
import os
import subprocess
import re
from datetime import date

physicsdirectories = ["Engine/Source/Runtime/Engine/Classes/PhysicsEngine/...", "Engine/Source/Runtime/Engine/Classes/PhysicalMaterials/...", "Engine/Source/Runtime/Engine/Public/Physics/...", "Engine/Source/Runtime/Engine/Private/PhysicsEngine/...", "Engine/Source/Runtime/ClothingSystemRuntimeCommon/...", "Engine/Source/Runtime/ClothingSystemRuntimeInterface/...", "Engine/Source/Runtime/ClothingSystemRuntimeNv/...", "Engine/Source/Runtime/PhysicsCore/...", "Engine/Source/Runtime/PhysicsSQ/...", "Engine\Source\Runtime\AnimGraphRuntime\Public\BoneControllers\AnimNode_RigidBody.h", "Engine\Source\Runtime\AnimGraphRuntime\Private\BoneControllers\AnimNode_RigidBody.cpp", "Engine\Source\Editor\AnimGraph\Classes\AnimGraphNode_RigidBody.h", "Engine\Source\Editor\AnimGraph\Private\AnimGraphNode_RigidBody.cpp"]

versionfiles = ["Engine/Source/Runtime/Core/Public/UObject/ExternalPhysicsCustomObjectVersion.h", "Engine/Source/Runtime/Core/Public/UObject/ExternalPhysicsMaterialCustomObjectVersion.h"]
        
chaosdirectories = ["Engine/Plugins/Experimental/ChaosCloth/...", "Engine/Plugins/Experimental/ChaosClothEditor/...", "Engine/Plugins/Experimental/ChaosEditor/...", "Engine/Plugins/Experimental/ChaosSolverPlugin/...", "Engine/Source/Runtime/Experimental/Chaos/...", "Engine/Source/Runtime/Experimental/ChaosCore/...", "Engine/Source/Runtime/Experimental/ChaosSolverEngine/...", "Engine/Source/Runtime/Experimental/ChaosSolvers/...", "Engine/Source/Runtime/Experimental/FieldSystem/...", "Engine/Source/Runtime/Experimental/GeometryCollectionCore/...", "Engine/Source/Runtime/Experimental/GeometryCollectionEngine/...", "Engine/Source/Runtime/Experimental/GeometryCollectionSimulationCore/..."]

testdirectories = ["Engine/Source/Programs/HeadlessChaos/...", "Engine/Restricted/NotForLicensees/Source/Programs/GeometryCollectionTest/...", "Engine/Restricted/NotForLicensees/Source/Programs/HeadlessChaosDependencyChecker/...", "Engine/Restricted/NotForLicensees/Source/Programs/HeadlessPhysicsSQ/..."]

users = ["michael.lentine", "ori.cohen", "brice.criswell", "benn.gallagher", "ryan.kautzman", "michael.forot", "chris.caulfield", "devon.penney", "max.whitehead", "dteven.barnett", "satchit.subramanian", "jaco.vandyk", "kriss.gossart", "bill.henderson", "yilin.zhu"]

def resolve():
    cmd = "p4 resolve -am ..."
    ret = False
    while (ret != True):
        result = subprocess.run(cmd, stdout=subprocess.PIPE)
        output = result.stdout.decode("utf-8")
        print(output)
        ret = output.find("resolve skipped") == -1;
        if (ret != True):
            input("Failed to autoresolve. Please merge by hand before continuing.")

def merge_directories(branch, directory_list):
    for directory in directory_list:
        cmd = "p4 merge -S " + branch + " -P //UE4/Main -r -s //UE4/Main/" + directory
        os.system(cmd)
        resolve()

def merge_user_cls(branch, users, date_start, date_end):
    cl_numbers = []
    for user in users:
        cmd = "p4 changes -u " + user + " //UE4/Main...@" + date_start + ",@" + date_end
        result = subprocess.run(cmd, stdout=subprocess.PIPE)
        changes = result.stdout.decode("utf-8").split("\r\n")
        for change in changes:
            lineparts = change.split()
            if (len(lineparts) > 0):
                cl_numbers.append(int(lineparts[1]))
    cl_numbers.sort()
    for cl in cl_numbers:
        cmd = "p4 integrate //UE4/Main/...@" + str(cl) + "," + str(cl) + " " + branch + "/..."
        os.system(cmd)
        resolve()

def main():
    parser = argparse.ArgumentParser()
    parser.add_argument("scope", help="how much of physics/chaos to merge")
    parser.add_argument("branch", help="branch to merge to")
    parser.add_argument("--from_date", help="date to merge from for users", default=None)
    parser.add_argument("--end_date", help="date to merge from to users", default=None)
    args = parser.parse_args()
    today = date.today()
    datestr = today.strftime("%m/%d/%Y")
    if (args.from_date is None):
        date_start = datestr
    else:
        date_start = args.from_date
    if (args.end_date is None):
        date_end = datestr
    else:
        date_end = args.end_date
    if (args.scope == "all"):
        merge_directories(args.branch, physicsdirectories)
        merge_directories(args.branch, chaosdirectories)
        merge_directories(args.branch, versionfiles)
        merge_directories(args.branch, testdirectories)
        merge_user_cls(args.branch, users, date_start, date_end)
    if (args.scope == "release"):
        merge_directories(args.branch, physicsdirectories)
        merge_directories(args.branch, chaosdirectories)
    if (args.scope == "chaos"):
        merge_directories(args.branch, chaosdirectories)
        merge_directories(args.branch, testdirectories)
    if (args.scope == "chaosrelease"):
        merge_directories(args.branch, chaosdirectories)


if __name__ == "__main__":
    main()
