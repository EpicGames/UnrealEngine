// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/Character/CharacterGroundConstraint.h"

namespace ChaosTest
{
	using namespace Chaos;

	// Only function is a == operator so test that
	TEST(CharacterGroundConstraintTests, TestSettingsAndData)
	{
		FCharacterGroundConstraintSettings Settings;
		FCharacterGroundConstraintSettings SettingsCopy = Settings;
		EXPECT_EQ(Settings, SettingsCopy);

		Settings.RadialForceLimit = 2000.0f;
		EXPECT_FALSE(Settings == SettingsCopy);

		FCharacterGroundConstraintDynamicData Data;
		FCharacterGroundConstraintDynamicData DataCopy = Data;
		EXPECT_EQ(Data, DataCopy);

		Data.GroundDistance = 11.0f;
		EXPECT_FALSE(Data == DataCopy);
	}

	TEST(CharacterGroundConstraintTests, TestConstraintInitialization)
	{
		FCharacterGroundConstraint Constraint;

		// Not set proxy so constraint should be invalid
		EXPECT_FALSE(Constraint.IsValid());

		// Type should be set
		EXPECT_EQ(Constraint.GetType(), EConstraintType::CharacterGroundConstraintType);
		EXPECT_TRUE(Constraint.IsType(EConstraintType::CharacterGroundConstraintType));

		// Force and torque should be initialized to zero
		EXPECT_VECTOR_FLOAT_EQ(Constraint.GetSolverAppliedForce(), FVector(0));
		EXPECT_VECTOR_FLOAT_EQ(Constraint.GetSolverAppliedTorque(), FVector(0));
	}

	TEST(CharacterGroundConstraintTests, TestConstraintDirtyFlags)
	{
		FCharacterGroundConstraint Constraint;

		EXPECT_FALSE(Constraint.IsDirty());

		Constraint.SetGroundDistance(10.0);
		EXPECT_EQ(Constraint.GetGroundDistance(), 10.0);

		EXPECT_TRUE(Constraint.IsDirty());
		EXPECT_TRUE(Constraint.IsDirty(EChaosPropertyFlags::CharacterGroundConstraintDynamicData));
		EXPECT_FALSE(Constraint.IsDirty(EChaosPropertyFlags::CharacterGroundConstraintSettings));
		EXPECT_FALSE(Constraint.IsDirty(EChaosPropertyFlags::CharacterParticleProxy));
		EXPECT_FALSE(Constraint.IsDirty(EChaosPropertyFlags::GroundParticleProxy));

		Constraint.ClearDirtyFlags();
		EXPECT_FALSE(Constraint.IsDirty());

		Constraint.SetTargetHeight(20.0);
		EXPECT_EQ(Constraint.GetTargetHeight(), 20.0);

		EXPECT_TRUE(Constraint.IsDirty());
		EXPECT_FALSE(Constraint.IsDirty(EChaosPropertyFlags::CharacterGroundConstraintDynamicData));
		EXPECT_TRUE(Constraint.IsDirty(EChaosPropertyFlags::CharacterGroundConstraintSettings));
		EXPECT_FALSE(Constraint.IsDirty(EChaosPropertyFlags::CharacterParticleProxy));
		EXPECT_FALSE(Constraint.IsDirty(EChaosPropertyFlags::GroundParticleProxy));

		Constraint.ClearDirtyFlags();

		Constraint.SetMaxCharacterGroundMassRatio(1.5);
		EXPECT_EQ(Constraint.GetMaxCharacterGroundMassRatio(), 1.5);
		EXPECT_TRUE(Constraint.IsDirty());
		EXPECT_FALSE(Constraint.IsDirty(EChaosPropertyFlags::CharacterGroundConstraintDynamicData));
		EXPECT_TRUE(Constraint.IsDirty(EChaosPropertyFlags::CharacterGroundConstraintSettings));
		EXPECT_FALSE(Constraint.IsDirty(EChaosPropertyFlags::CharacterParticleProxy));
		EXPECT_FALSE(Constraint.IsDirty(EChaosPropertyFlags::GroundParticleProxy));

		Constraint.ClearDirtyFlags();

		FSingleParticlePhysicsProxy* CharacterProxy = FSingleParticlePhysicsProxy::Create(FGeometryParticle::CreateParticle());
		FSingleParticlePhysicsProxy* GroundProxy = FSingleParticlePhysicsProxy::Create(FGeometryParticle::CreateParticle());

		Constraint.Init(CharacterProxy);
		EXPECT_EQ(Constraint.GetCharacterParticleProxy(), CharacterProxy);

		EXPECT_TRUE(Constraint.IsDirty(EChaosPropertyFlags::CharacterGroundConstraintDynamicData));
		EXPECT_TRUE(Constraint.IsDirty(EChaosPropertyFlags::CharacterGroundConstraintSettings));
		EXPECT_TRUE(Constraint.IsDirty(EChaosPropertyFlags::CharacterParticleProxy));
		EXPECT_FALSE(Constraint.IsDirty(EChaosPropertyFlags::GroundParticleProxy));

		Constraint.SetGroundParticleProxy(GroundProxy);
		EXPECT_EQ(Constraint.GetGroundParticleProxy(), GroundProxy);
		EXPECT_TRUE(Constraint.IsDirty(EChaosPropertyFlags::GroundParticleProxy));

		Constraint.ClearDirtyFlags();

		FDirtyProxiesBucketInfo BucketInfo;
		BucketInfo.Num[(uint32)EPhysicsProxyType::CharacterGroundConstraintType] = 1;
		FDirtyPropertiesManager Manager;
		Manager.PrepareBuckets(BucketInfo);

		FDirtyChaosProperties RemoteData;

		EXPECT_FALSE(RemoteData.HasCharacterGroundConstraintSettings());

		Constraint.SetTargetHeight(15.0);
		Constraint.SetMaxCharacterGroundMassRatio(1.2);
		Constraint.SyncRemoteData(Manager, 0, RemoteData);

		ASSERT_TRUE(RemoteData.HasCharacterGroundConstraintSettings());
		EXPECT_FALSE(RemoteData.HasCharacterGroundConstraintDynamicData());
		const FCharacterGroundConstraintSettings& Settings = RemoteData.GetCharacterGroundConstraintSettings(Manager, 0);

		EXPECT_EQ(Settings.TargetHeight, 15.0);
		EXPECT_EQ(Settings.MaxCharacterGroundMassRatio, 1.2);

		Constraint.ClearDirtyFlags();
		RemoteData.Clear(Manager, 0);

		EXPECT_FALSE(RemoteData.HasCharacterGroundConstraintDynamicData());
		EXPECT_FALSE(RemoteData.HasCharacterGroundConstraintSettings());

		Constraint.SetGroundDistance(34.5);
		Constraint.SyncRemoteData(Manager, 0, RemoteData);

		ASSERT_TRUE(RemoteData.HasCharacterGroundConstraintDynamicData());
		EXPECT_FALSE(RemoteData.HasCharacterGroundConstraintSettings());
		const FCharacterGroundConstraintDynamicData& Data = RemoteData.GetCharacterGroundConstraintDynamicData(Manager, 0);

		EXPECT_EQ(Data.GroundDistance, 34.5);
	}
} // namespace ChaosTest