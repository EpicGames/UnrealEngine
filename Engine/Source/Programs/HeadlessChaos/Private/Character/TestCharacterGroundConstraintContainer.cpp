// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/PBDRigidsSOAs.h"

namespace ChaosTest
{
	using namespace Chaos;

	class CharacterGroundConstraintContainerTest : public ::testing::Test
	{
	protected:
		CharacterGroundConstraintContainerTest()
			: SOAs(UniqueIndices)
		{
		}

		void SetSolverForce(FCharacterGroundConstraintHandle* Constraint, const FVec3& Value)
		{
			Constraint->SolverAppliedForce = Value;
		}

		FCharacterGroundConstraintContainer Container;
		TArray<FCharacterGroundConstraintSettings> Settings;
		TArray<FCharacterGroundConstraintDynamicData> Data;
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs SOAs;
	};

	TEST_F(CharacterGroundConstraintContainerTest, TestInitialization)
	{
		EXPECT_EQ(Container.NumConstraints(), 0);
		EXPECT_EQ(Container.GetConstraints().Num(), 0);

		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(2);
		Settings.SetNum(1);
		Data.SetNum(1);

		Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[1]);

		FCharacterGroundConstraintHandle* Constraint = Container.GetConstraint(0);
		ASSERT_NE(Constraint, nullptr);
		EXPECT_VECTOR_FLOAT_EQ(Constraint->GetSolverAppliedForce(), FVec3::ZeroVector);
		EXPECT_VECTOR_FLOAT_EQ(Constraint->GetSolverAppliedTorque(), FVec3::ZeroVector);
		EXPECT_EQ(Constraint->GetSettings(), Settings[0]);
		EXPECT_EQ(Constraint->GetData(), Data[0]);
		EXPECT_TRUE(Constraint->IsEnabled());
		EXPECT_FALSE(Constraint->IsSleeping());
		EXPECT_TRUE(Constraint->IsValid());

		Container.RemoveConstraint(Constraint);

		Particles[0]->SetDisabled(true);
		Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[1]);
		Constraint = Container.GetConstraint(0);
		EXPECT_FALSE(Constraint->IsEnabled());
	}

	TEST_F(CharacterGroundConstraintContainerTest, TestAddRemoveConstraints)
	{
		const int NumParticles = 6;
		const int NumConstraints = 4;
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(NumParticles);
		Settings.SetNum(NumConstraints);
		Data.SetNum(NumConstraints);

		FCharacterGroundConstraintHandle* Constraints[4];

		Constraints[0] = Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[1]);
		Constraints[1] = Container.AddConstraint(Settings[1], Data[1], Particles[2]);
		Constraints[2] = Container.AddConstraint(Settings[2], Data[2], Particles[3], Particles[1]);
		Constraints[3] = Container.AddConstraint(Settings[3], Data[3], Particles[4], Particles[5]);

		ASSERT_EQ(Container.NumConstraints(), NumConstraints);
		EXPECT_EQ(Container.GetNumConstraints(), NumConstraints);

		for (int Idx = 0; Idx < NumConstraints; ++Idx)
		{
			FCharacterGroundConstraintHandle* Constraint = Container.GetConstraint(Idx);
			ASSERT_NE(Constraint, nullptr);
		
			EXPECT_EQ(Constraint->GetSettings(), Settings[Idx]);
			EXPECT_EQ(Constraint->GetData(), Data[Idx]);
		}

		auto ContainerConstraints = Container.GetConstConstraints();
		ASSERT_EQ(ContainerConstraints.Num(), NumConstraints);
		for (int Idx = 0; Idx < NumConstraints; ++Idx)
		{
			EXPECT_EQ(ContainerConstraints[Idx], Constraints[Idx]);
		}

		Container.RemoveConstraint(Constraints[0]);
		Container.RemoveConstraint(Constraints[2]);
		ContainerConstraints = Container.GetConstConstraints();
		ASSERT_EQ(ContainerConstraints.Num(), 2);
		// Doesn't matter what order they're in but the two left should be 1 and 3
		if (ContainerConstraints[0] == Constraints[1])
		{
			EXPECT_EQ(ContainerConstraints[1], Constraints[3]);
		}
		else if (ContainerConstraints[0] == Constraints[3])
		{
			EXPECT_EQ(ContainerConstraints[1], Constraints[1]);
		}
		else
		{
			EXPECT_TRUE(false);
		}
	}

	TEST_F(CharacterGroundConstraintContainerTest, TestDisconnectConstraints)
	{
		const int NumParticles = 6;
		const int NumConstraints = 5;
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(NumParticles);
		Settings.SetNum(NumConstraints);
		Data.SetNum(NumConstraints);

		TArray<FCharacterGroundConstraintHandle*> Constraints;
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[1]));
		Constraints.Add(Container.AddConstraint(Settings[1], Data[1], Particles[2]));
		Constraints.Add(Container.AddConstraint(Settings[2], Data[2], Particles[3], Particles[1]));
		Constraints.Add(Container.AddConstraint(Settings[3], Data[3], Particles[4], Particles[5]));
		Constraints.Add(Container.AddConstraint(Settings[4], Data[4], Particles[5]));

		for (auto Constraint : Constraints)
		{
			EXPECT_TRUE(Constraint->IsEnabled());
		}

		// Remove particle 5
		// Constraint 4 should be disabled as particle 5 is its character particle
		// Constraint 3 should have the ground particle set to null
		Container.DisconnectConstraints({ Particles[5] });
		EXPECT_FALSE(Constraints[4]->IsEnabled());
		EXPECT_EQ(Constraints[3]->GetGroundParticle(), nullptr);
	}

	TEST_F(CharacterGroundConstraintContainerTest, TestAddToGraph)
	{
		const int NumDynamicParticles = 15;
		const int NumStaticParticles = 2;

		TArray<FPBDRigidParticleHandle*> DynamicParticles = SOAs.CreateDynamicParticles(NumDynamicParticles);
		TArray<FGeometryParticleHandle*> StaticParticles = SOAs.CreateStaticParticles(NumStaticParticles);

		TArray<TVec2<FGeometryParticleHandle*>> ParticlePairs =
		{
			// Two character particles with the same dynamic ground particle
			{ DynamicParticles[0], DynamicParticles[9] },
			{ DynamicParticles[1], DynamicParticles[9] },
			// One character particle with no ground particle
			{ DynamicParticles[2], nullptr },
			// One character particle with single dynamic ground particle
			{ DynamicParticles[3], DynamicParticles[10] },
			// One character particle with two constraints to different dynamic bodies
			{ DynamicParticles[4], DynamicParticles[11] },
			{ DynamicParticles[4], DynamicParticles[12] },
			// One character particle with two constraints to the same dynamic bodies
			{ DynamicParticles[5], DynamicParticles[13] },
			{ DynamicParticles[5], DynamicParticles[13] },
			// One character particle with two constraints to a dynamic and static body
			{ DynamicParticles[6], DynamicParticles[14] },
			{ DynamicParticles[6], StaticParticles[0] },
			// Two character particle with constraints to the same static body
			{ DynamicParticles[7], StaticParticles[1] },
			{ DynamicParticles[8], StaticParticles[1] },
		};

		const int NumConstraints = ParticlePairs.Num();
		Settings.SetNum(NumConstraints);
		Data.SetNum(NumConstraints);
		for (int Idx = 0; Idx < NumConstraints; ++Idx)
		{
			Container.AddConstraint(Settings[Idx], Data[Idx], ParticlePairs[Idx][0], ParticlePairs[Idx][1]);
		}

		// Initialize graph
		Private::FPBDIslandManager Graph(SOAs);
		Container.SetContainerId(0); // Usually set by the evolution the container is registered with
		Graph.AddConstraintContainer(Container);

		for (auto Particle : DynamicParticles)
		{
			Graph.AddParticle(Particle);
		}
		for (auto Particle : StaticParticles)
		{
			Graph.AddParticle(Particle);
		}
		Graph.UpdateParticles();

		// Add constraints and update graph
		Container.AddConstraintsToGraph(Graph);
		Graph.UpdateIslands();

		auto Constraints = Container.GetConstraints();
		for (FCharacterGroundConstraintHandle* Constraint : Constraints)
		{
			EXPECT_TRUE(Constraint->IsInConstraintGraph());
		}

		TArray<TArray<FGeometryParticleHandle*>> ExpectedParticlesInIsland =
		{
			{ DynamicParticles[0], DynamicParticles[1], DynamicParticles[9] },
			{ DynamicParticles[2] },
			{ DynamicParticles[3], DynamicParticles[10] },
			{ DynamicParticles[4], DynamicParticles[11], DynamicParticles[12] },
			{ DynamicParticles[5], DynamicParticles[13] },
			{ DynamicParticles[6], DynamicParticles[14], StaticParticles[0] },
			{ DynamicParticles[7], StaticParticles[1] },
			{ DynamicParticles[8], StaticParticles[1] },
		};
		TArray<TArray<FCharacterGroundConstraintHandle*>> ExpectedConstraintsInIsland =
		{
			{ Constraints[0], Constraints[1] },
			{ Constraints[2] },
			{ Constraints[3] },
			{ Constraints[4], Constraints[5] },
			{ Constraints[6], Constraints[7] },
			{ Constraints[8], Constraints[9] },
			{ Constraints[10] },
			{ Constraints[11] },
		};
		const int ExpectedNumIslands = ExpectedParticlesInIsland.Num();

		for (int32 IslandIdx = 0; IslandIdx < ExpectedNumIslands; ++IslandIdx)
		{
			// Find all the islands that contain one of these particles (should only be one because they are all dynamic)
			const TArray<const Private::FPBDIsland*> Islands = Graph.FindParticleIslands(ExpectedParticlesInIsland[IslandIdx][0]);
			EXPECT_EQ(Islands.Num(), 1);

			// Find all the particles in the island and make sure they match the expected set
			const TArray<const FGeometryParticleHandle*> IslandParticles = Graph.FindParticlesInIslands(Islands);
			EXPECT_EQ(IslandParticles.Num(), ExpectedParticlesInIsland[IslandIdx].Num());
			for (const FGeometryParticleHandle* ExpectedIslandParticle : ExpectedParticlesInIsland[IslandIdx])
			{
				EXPECT_TRUE(IslandParticles.Contains(ExpectedIslandParticle));
			}

			// Find all the constraints in the island and make sure they match the expected set
			const TArray<const FConstraintHandle*> IslandConstraints = Graph.FindConstraintsInIslands(Islands, Container.GetContainerId());
			EXPECT_EQ(IslandConstraints.Num(), ExpectedConstraintsInIsland[IslandIdx].Num());
			for (const FConstraintHandle* ExpectedIslandConstraint : ExpectedConstraintsInIsland[IslandIdx])
			{
				EXPECT_TRUE(IslandConstraints.Contains(ExpectedIslandConstraint));
			}
		}

		Graph.Reset();
	}

	TEST_F(CharacterGroundConstraintContainerTest, TestContainerSolver)
	{
		// Create solver
		int Priority = 2;
		TUniquePtr<FConstraintContainerSolver> Solver = Container.CreateSceneSolver(Priority);
		EXPECT_EQ(Solver->GetPriority(), Priority);

		// Create constraints in two islands
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(4);
		Settings.SetNum(1);
		Data.SetNum(1);

		// Initialize the data so that each constraint has work to do
		Particles[0]->SetX(FVec3(0.0, 0.0, 10.0));
		Particles[1]->SetX(FVec3(0.0, 0.0, 10.0));
		Particles[2]->SetX(FVec3(0.0, 0.0, 10.0));
		Particles[0]->SetP(FVec3(0.0, 0.0, 10.0));
		Particles[1]->SetP(FVec3(0.0, 0.0, 10.0));
		Particles[2]->SetP(FVec3(0.0, 0.0, 10.0));
		Settings[0].TargetHeight = 20.0f;
		Data[0].GroundDistance = 10.0f;

		TArray<FCharacterGroundConstraintHandle*> Constraints;
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[3]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[1], Particles[3]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[2]));

		Private::FPBDIslandManager Graph(SOAs);
		Container.SetContainerId(0);
		Graph.AddConstraintContainer(Container);
		for (auto Particle : Particles)
		{
			Graph.AddParticle(Particle);
		}
		Graph.UpdateParticles();
		Container.AddConstraintsToGraph(Graph);
		Graph.UpdateIslands();

		// First use non-island version
		Solver->AddConstraints();
		EXPECT_EQ(Solver->GetNumConstraints(), Constraints.Num());

		FSolverBodyContainer SolverBodies;
		SolverBodies.Reset(Particles.Num());

		Solver->AddBodies(SolverBodies);
		EXPECT_EQ(SolverBodies.Num(), Particles.Num());

		const FReal Dt = 1.0 / 30.0;
		SolverBodies.GatherInput(Dt, 0, SolverBodies.Num());
		Solver->GatherInput(Dt);

		for (FCharacterGroundConstraintHandle* Constraint : Constraints)
		{
			EXPECT_VECTOR_FLOAT_EQ(Constraint->GetSolverAppliedForce(), FVec3::ZeroVector);
		}

		Solver->ApplyPositionConstraints(Dt, 0, 1);
		Solver->ScatterOutput(Dt);

		for (FCharacterGroundConstraintHandle* Constraint : Constraints)
		{
			EXPECT_NE(Constraint->GetSolverAppliedForce(), FVec3::ZeroVector);
		}

		// Reset the constraint forces to something else
		for (FCharacterGroundConstraintHandle* Constraint : Constraints)
		{
			SetSolverForce(Constraint, FVec3(1, 1, 1));
		}

		// Now use island API
		Solver->Reset(Constraints.Num());

		for (auto Particle : Particles)
		{
			Particle->SetSolverBodyIndex(INDEX_NONE);
		}

		int IslandIdx = 0;
		Private::FPBDIsland* Island = Graph.GetIsland(IslandIdx);
		Solver->AddConstraints(Island->GetConstraints(0));
		EXPECT_EQ(Solver->GetNumConstraints(), 2);

		SolverBodies.Reset(Particles.Num());

		Solver->AddBodies(SolverBodies);
		EXPECT_EQ(SolverBodies.Num(), 3);


		SolverBodies.GatherInput(Dt, 0, SolverBodies.Num());
		Solver->GatherInput(Dt);
		Solver->ApplyPositionConstraints(Dt, 0, 1);
		Solver->ScatterOutput(Dt);

		// Expect the constraints in the island to be updated, but not
		// the constraint in the other island
		EXPECT_NE(Constraints[0]->GetSolverAppliedForce(), FVec3::ZeroVector);
		EXPECT_NE(Constraints[0]->GetSolverAppliedForce(), FVec3(1, 1, 1));
		EXPECT_NE(Constraints[1]->GetSolverAppliedForce(), FVec3::ZeroVector);
		EXPECT_NE(Constraints[1]->GetSolverAppliedForce(), FVec3(1, 1, 1));

		EXPECT_VECTOR_FLOAT_EQ(Constraints[2]->GetSolverAppliedForce(), FVec3(1, 1, 1));

		Graph.Reset();
	}
}