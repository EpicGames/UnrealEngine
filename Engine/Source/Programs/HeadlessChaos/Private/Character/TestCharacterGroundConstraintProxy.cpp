// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/Character/CharacterGroundConstraint.h"
#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/CharacterGroundConstraintProxy.h"

namespace ChaosTest
{
	using namespace Chaos;

	class CharacterGroundConstraintsProxyTest : public ::testing::Test
	{
	protected:
		void SetUp() override
		{
			FChaosSolversModule* Module = FChaosSolversModule::GetModule();
			Solver = Module->CreateSolver(/*Owner*/nullptr, /*AsyncDt=*/-1);
			Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
		}

		void TearDown() override
		{
			FChaosSolversModule* Module = FChaosSolversModule::GetModule();
			Module->DestroySolver(Solver);
		}

		FSingleParticlePhysicsProxy* CreateParticle()
		{
			auto Geom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
			FSingleParticlePhysicsProxy* Proxy = FSingleParticlePhysicsProxy::Create(FPBDRigidParticle::CreateParticle());
			Proxy->SetHandle(Solver->GetEvolution()->CreateDynamicParticles(1)[0]);
			Proxy->GetGameThreadAPI().SetGeometry(Geom);
			Proxy->GetGameThreadAPI().SetM(1.0);
			Proxy->GetGameThreadAPI().SetInvI(FVec3(1.0, 1.0, 1.0));
			return Proxy;
		}

		FPBDRigidsSolver* Solver;
	};

	// This tests initialization of a character ground constraint and pushing
	// of data from the game thread representation to the physics thread representation
	TEST_F(CharacterGroundConstraintsProxyTest, TestProxyDataSyncing)
	{
		// Create a game thread constraint and register it with the solver
		FCharacterGroundConstraint* Constraint_GT = new FCharacterGroundConstraint();
		FSingleParticlePhysicsProxy* CharacterProxy = CreateParticle();
		FSingleParticlePhysicsProxy* GroundProxy = CreateParticle();
		Constraint_GT->Init(CharacterProxy);
		Constraint_GT->SetGroundParticleProxy(GroundProxy);

		Solver->RegisterObject(CharacterProxy);
		Solver->RegisterObject(GroundProxy);
		Solver->RegisterObject(Constraint_GT);

		// Get the proxy from the game thread constraint
		// Should have been set when registering the constraint with the solver
		FCharacterGroundConstraintProxy* Proxy = Constraint_GT->GetProxy<FCharacterGroundConstraintProxy>();
		ASSERT_TRUE(Proxy != nullptr);

		EXPECT_FALSE(Proxy->IsInitialized());
		EXPECT_EQ(Proxy->GetGameThreadAPI(), Constraint_GT);
		EXPECT_EQ(Proxy->GetPhysicsThreadAPI(), nullptr);

		// Create a local dirty properties manager and properties to use to transfer data from
		// the game thread constraint to the physics thread constraint and initialize with the
		// game thread constraint data

		const int32 DataIdx = 0;
		FDirtyProxiesBucketInfo BucketInfo;
		BucketInfo.Num[(uint32)EPhysicsProxyType::CharacterGroundConstraintType] = 1;
		FDirtyPropertiesManager Manager;
		Manager.PrepareBuckets(BucketInfo);

		FDirtyChaosProperties RemoteData;

		Proxy->PushStateOnGameThread(Manager, 0, RemoteData);

		// Initialize the physics thread data on the proxy
		Proxy->InitializeOnPhysicsThread(Solver, Manager, DataIdx, RemoteData);

		FCharacterGroundConstraintHandle* Constraint_PT = Proxy->GetPhysicsThreadAPI();
		EXPECT_TRUE(Constraint_PT != nullptr);
		EXPECT_EQ(Constraint_PT->GetCharacterParticle(), CharacterProxy->GetHandle_LowLevel());
		EXPECT_EQ(Constraint_PT->GetGroundParticle(), GroundProxy->GetHandle_LowLevel());

		EXPECT_EQ(Solver->GetCharacterGroundConstraints().GetNumConstraints(), 1);

		// Check that the constraint is set on the constrained particles
		ASSERT_EQ(Solver->GetParticles().GetDynamicParticles().ParticleConstraints(0).Num(), 1);
		EXPECT_EQ(Solver->GetParticles().GetDynamicParticles().ParticleConstraints(0)[0], Constraint_PT);

		ASSERT_EQ(Solver->GetParticles().GetDynamicParticles().ParticleConstraints(1).Num(), 1);
		EXPECT_EQ(Solver->GetParticles().GetDynamicParticles().ParticleConstraints(1)[0], Constraint_PT);

		// Change something on the game side and check that it is synced on the physics side
		RemoteData.Clear(Manager, DataIdx);

		Constraint_GT->SetGroundDistance(1234.5f);
		Constraint_GT->SetRadialForceLimit(1010.0f);

		Proxy->PushStateOnGameThread(Manager, DataIdx, RemoteData);
		Proxy->PushStateOnPhysicsThread(Solver, Manager, DataIdx, RemoteData);

		EXPECT_FLOAT_EQ(Constraint_PT->GetData().GroundDistance, 1234.5f);
		EXPECT_FLOAT_EQ(Constraint_PT->GetSettings().RadialForceLimit, 1010.0f);

		// Nothing should be flagged as dirty yet
		EXPECT_FALSE(Constraint_PT->HasDataChanged());
		EXPECT_FALSE(Constraint_PT->HaveSettingsChanged());

		// Change something on the physics thread and check that it's synced back to the game thread
		FCharacterGroundConstraintDynamicData DynData;
		DynData.GroundDistance = 1.2f;
		DynData.GroundNormal = FVector(1.0f, 0.0f, 0.0f);
		Constraint_PT->SetData(DynData);

		// Data should be flagged as dirty
		EXPECT_TRUE(Constraint_PT->HasDataChanged());
		EXPECT_FALSE(Constraint_PT->HaveSettingsChanged());

		FCharacterGroundConstraintSettings& Settings = Constraint_PT->GetSettings_Mutable();
		Settings.RadialForceLimit = 2020.0f;

		// Settings should be flagged as dirty
		EXPECT_TRUE(Constraint_PT->HasDataChanged());
		EXPECT_TRUE(Constraint_PT->HaveSettingsChanged());

		Chaos::FDirtyCharacterGroundConstraintData DirtyConstraintData;
		Proxy->BufferPhysicsResults(DirtyConstraintData);
		Proxy->PullFromPhysicsState(DirtyConstraintData, 0);

		EXPECT_FLOAT_EQ(Constraint_GT->GetGroundDistance(), DynData.GroundDistance);
		EXPECT_VECTOR_FLOAT_EQ(Constraint_GT->GetGroundNormal(), DynData.GroundNormal);

		EXPECT_FLOAT_EQ(Constraint_GT->GetRadialForceLimit(), 2020.0f);

		// Remove the constraint and particles from the solver
		Solver->UnregisterObject(Constraint_GT);
		Solver->UnregisterObject(CharacterProxy);
		Solver->UnregisterObject(GroundProxy);

		EXPECT_EQ(Proxy->GetGameThreadAPI(), nullptr);

		// Physics thread constraint is deleted in a callback set on the marshalling manager
	}

	TEST_F(CharacterGroundConstraintsProxyTest, TestEnableDisable)
	{
		// Create a game thread constraint and register it with the solver
		FCharacterGroundConstraint* Constraint_GT = new FCharacterGroundConstraint();
		FSingleParticlePhysicsProxy* CharacterProxy = CreateParticle();
		FSingleParticlePhysicsProxy* GroundProxy = CreateParticle();
		Constraint_GT->Init(CharacterProxy);
		Constraint_GT->SetGroundParticleProxy(GroundProxy);

		Solver->RegisterObject(CharacterProxy);
		Solver->RegisterObject(GroundProxy);
		Solver->RegisterObject(Constraint_GT);

		FCharacterGroundConstraintProxy* Proxy = Constraint_GT->GetProxy<FCharacterGroundConstraintProxy>();

		const int32 DataIdx = 0;
		FDirtyProxiesBucketInfo BucketInfo;
		BucketInfo.Num[(uint32)EPhysicsProxyType::CharacterGroundConstraintType] = 1;
		FDirtyPropertiesManager Manager;
		Manager.PrepareBuckets(BucketInfo);

		FDirtyChaosProperties RemoteData;

		Proxy->PushStateOnGameThread(Manager, 0, RemoteData);
		Proxy->InitializeOnPhysicsThread(Solver, Manager, DataIdx, RemoteData);

		FCharacterGroundConstraintHandle* Constraint_PT = Proxy->GetPhysicsThreadAPI();

		// Check enabling/disabling
		EXPECT_TRUE(Constraint_PT->IsEnabled());
		Constraint_PT->SetEnabled(false);
		EXPECT_FALSE(Constraint_PT->IsEnabled());
		Constraint_PT->SetEnabled(true);
		EXPECT_TRUE(Constraint_PT->IsEnabled());

		// Check that the constraint is set on the constrained particles
		ASSERT_EQ(Solver->GetParticles().GetDynamicParticles().ParticleConstraints(0).Num(), 1);
		EXPECT_EQ(Solver->GetParticles().GetDynamicParticles().ParticleConstraints(0)[0], Constraint_PT);

		// Destroy the constraint on the physics thread. Ensure that the particle constraints are unregistered
		Proxy->DestroyOnPhysicsThread(Solver);

		ASSERT_EQ(Solver->GetParticles().GetDynamicParticles().ParticleConstraints(0).Num(), 0);

		// Reinitialize on the physics thread
		Proxy->InitializeOnPhysicsThread(Solver, Manager, DataIdx, RemoteData);
		ASSERT_EQ(Solver->GetParticles().GetDynamicParticles().ParticleConstraints(0).Num(), 1);

		// Disable the character particle - constraint should be disabled
		Solver->GetEvolution()->DisableParticle(CharacterProxy->GetHandle_LowLevel());
		ASSERT_FALSE(Constraint_PT->IsEnabled());

		Solver->GetEvolution()->EnableParticle(CharacterProxy->GetHandle_LowLevel());
		ASSERT_TRUE(Constraint_PT->IsEnabled());
	}

	// Test disabling constraint particles on the physics thread
	TEST_F(CharacterGroundConstraintsProxyTest, TestParticleDisable)
	{
		// Create a game thread constraint and register it with the solver
		FCharacterGroundConstraint* Constraint_GT = new FCharacterGroundConstraint();
		FSingleParticlePhysicsProxy* CharacterProxy = CreateParticle();
		FSingleParticlePhysicsProxy* GroundProxy = CreateParticle();
		Constraint_GT->Init(CharacterProxy);
		Constraint_GT->SetGroundParticleProxy(GroundProxy);

		Solver->RegisterObject(CharacterProxy);
		Solver->RegisterObject(GroundProxy);
		Solver->RegisterObject(Constraint_GT);

		// Get the proxy from the game thread constraint
		// Should have been set when registering the constraint with the solver
		FCharacterGroundConstraintProxy* Proxy = Constraint_GT->GetProxy<FCharacterGroundConstraintProxy>();
		ASSERT_TRUE(Proxy != nullptr);

		const int32 DataIdx = 0;
		FDirtyProxiesBucketInfo BucketInfo;
		BucketInfo.Num[(uint32)EPhysicsProxyType::CharacterGroundConstraintType] = 1;
		FDirtyPropertiesManager Manager;
		Manager.PrepareBuckets(BucketInfo);

		FDirtyChaosProperties RemoteData;

		Proxy->PushStateOnGameThread(Manager, 0, RemoteData);

		// Initialize the physics thread data on the proxy
		Proxy->InitializeOnPhysicsThread(Solver, Manager, DataIdx, RemoteData);

		FCharacterGroundConstraintHandle* Constraint_PT = Proxy->GetPhysicsThreadAPI();
		ASSERT_TRUE(Constraint_PT != nullptr);

		// First disable the character particle. This should disable the constraint
		Solver->GetEvolution()->DisableParticle(Constraint_PT->GetCharacterParticle());
		EXPECT_FALSE(Constraint_PT->IsEnabled());

		// Now re-enable. The constraint should be re-enabled
		Solver->GetEvolution()->EnableParticle(Constraint_PT->GetCharacterParticle());
		EXPECT_TRUE(Constraint_PT->IsEnabled());

		// Disable the ground particle. The constraint should remain enabled with the ground body set to null
		FGeometryParticleHandle* GroundParticle = Constraint_PT->GetGroundParticle();
		EXPECT_TRUE(GroundParticle->ParticleConstraints().Contains(Constraint_PT));
		Solver->GetEvolution()->DisableParticle(GroundParticle);
		EXPECT_TRUE(Constraint_PT->IsEnabled());
		EXPECT_EQ(Constraint_PT->GetGroundParticle(), nullptr);
		EXPECT_FALSE(GroundParticle->ParticleConstraints().Contains(Constraint_PT));

		// Should not be able to set a disabled particle as the ground particle
		Constraint_PT->SetGroundParticle(GroundParticle);
		EXPECT_EQ(Constraint_PT->GetGroundParticle(), nullptr);

		Chaos::FDirtyCharacterGroundConstraintData DirtyConstraintData;
		Proxy->BufferPhysicsResults(DirtyConstraintData);
		Proxy->PullFromPhysicsState(DirtyConstraintData, 0);

		// Temporarily disable this until syncing of the ground particle can be fixed
		//EXPECT_EQ(Constraint_GT->GetGroundParticleProxy(), nullptr);

		// Remove the constraint and particles from the solver
		Solver->UnregisterObject(Constraint_GT);
		Solver->UnregisterObject(CharacterProxy);
		Solver->UnregisterObject(GroundProxy);

		EXPECT_EQ(Proxy->GetGameThreadAPI(), nullptr);

		// Physics thread constraint is deleted in a callback set on the marshalling manager
	}
}