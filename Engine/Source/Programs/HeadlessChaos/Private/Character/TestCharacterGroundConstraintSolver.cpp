// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/Character/CharacterGroundConstraintSolver.h"

namespace ChaosTest
{
	using namespace Chaos;

	class CharacterGroundConstraintSolverTest : public ::testing::Test
	{
	protected:
		void SetUp() override
		{
			CharacterBody = FSolverBody::MakeInitialized();
			CharacterBody.SetInvM(0.01);
			CharacterBody.SetInvILocal(FVec3(0.005, 0.005, 0.005));

			GroundBody = FSolverBody::MakeInitialized();

			Dt = FReal(1.0 / 30.0);
		}

		void SolvePosition(int NumIterations)
		{
			for (int It = 0; It < NumIterations; ++It)
			{
				Solver.SolvePosition();
			}
		}

		void UpdateSingleBody(int NumPositionIterations, int NumVelocityIterations)
		{
			Solver.SetBodies(&CharacterBody);
			Solver.GatherInput(Dt, Settings, Data);
			SolvePosition(NumPositionIterations);
			CharacterBody.ApplyCorrections();
			Solver.Reset();
		}

		void UpdateTwoBody(int NumPositionIterations, int NumVelocityIterations)
		{
			Solver.SetBodies(&CharacterBody, &GroundBody);
			Solver.GatherInput(Dt, Settings, Data);
			SolvePosition(NumPositionIterations);
			CharacterBody.ApplyCorrections();
			GroundBody.ApplyCorrections();
			Solver.Reset();
		}

		Private::FCharacterGroundConstraintSolver Solver;
		FCharacterGroundConstraintSettings Settings;
		FCharacterGroundConstraintDynamicData Data;
		FSolverBody CharacterBody;
		FSolverBody GroundBody;
		FReal Dt;
	};

	TEST_F(CharacterGroundConstraintSolverTest, TestInitialization)
	{
		// Should be able to access the impulses before initialization and get zero
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(1.0f), FVec3::ZeroVector);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(1.0f), FVec3::ZeroVector);
	}

	// Initial overlap at zero velocity
	// Should be corrected in a single iteration without introducing velocity
	TEST_F(CharacterGroundConstraintSolverTest, TestSingleBody_NormalImpulse_InitialOverlap)
	{
		CharacterBody.SetX(FVec3(10.0, 10.0, 20.0));
		CharacterBody.SetP(FVec3(10.0, 10.0, 20.0));
		Data.GroundDistance = 10.0f;
		Settings.TargetHeight = 15.0f;
		Settings.RadialForceLimit = 0.0f;
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;

		UpdateSingleBody(1, 0);

		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), FVec3(10.0, 10.0, 25.0));
		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.V(), FVec3(0.0, 0.0, 0.0));

		FVec3 ExpectedImpulse = (1.0 / (CharacterBody.InvM() * Dt)) * FVec3(0.0, 0.0, 5.0);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), FVec3::ZeroVector);
	}

	// Initial overlap but enough velocity to move out of overlap
	// Solver should do nothing
	TEST_F(CharacterGroundConstraintSolverTest, TestSingleBody_NormalImpulse_InitialOverlap_NoFinalOverlap)
	{
		CharacterBody.SetX(FVec3(10.0, 10.0, 20.0));
		CharacterBody.SetP(FVec3(10.0, 10.0, 25.01));
		CharacterBody.SetV(FVec3(0.0, -100.0, 5.01 / Dt));
		Data.GroundDistance = 10.0f;
		Settings.TargetHeight = 15.0f;
		Settings.RadialForceLimit = 0.0f;
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;

		UpdateSingleBody(1, 0);

		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), FVec3(10.0, 10.0, 25.01));
		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.V(), FVec3(0.0, -100.0, 5.01 / Dt));

		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), FVec3::ZeroVector);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), FVec3::ZeroVector);
	}

	// No overlap but projected overlap
	// Solver should fix overlap using displacement not correction
	TEST_F(CharacterGroundConstraintSolverTest, TestSingleBody_NormalImpulse_NoInitialOverlap_FinalOverlap)
	{
		CharacterBody.SetX(FVec3(10.0, 10.0, 26.0));
		CharacterBody.SetP(FVec3(10.0, 10.0, 20.0));
		CharacterBody.SetV(FVec3(0.0, 500.0, -6.0 / Dt));
		Data.GroundDistance = 16.0f;
		Settings.TargetHeight = 15.0f;
		Settings.RadialForceLimit = 0.0f;
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;

		Solver.SetBodies(&CharacterBody);
		Solver.GatherInput(Dt, Settings, Data);
		SolvePosition(1);

		EXPECT_TRUE(CharacterBody.CP() == FSolverVec3::ZeroVector);
		EXPECT_FALSE(CharacterBody.DP() == FSolverVec3::ZeroVector);

		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.CorrectedP(), FVec3(10.0, 10.0, 25.0));

		FVec3 ExpectedImpulse = (1.0 / (CharacterBody.InvM() * Dt)) * FVec3(0.0, 0.0, 5.0);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), FVec3::ZeroVector);
	}

	TEST_F(CharacterGroundConstraintSolverTest, TestTwoBody_MotionTarget)
	{
		CharacterBody.SetX(FVec3(0.0, 0.0, 10.0));
		CharacterBody.SetP(FVec3(0.0, 0.0, 10.0));
		GroundBody.SetX(FVec3(0.0, 0.0, 0.0));
		GroundBody.SetP(FVec3(0.0, 0.0, 0.0));
		Data.GroundDistance = 10.0f;
		Settings.TargetHeight = 10.0f;
		Settings.RadialForceLimit = 1.0e10f;
		Data.TargetDeltaPosition = FVec3(10.0, 0.0, 0.0);
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;
		Settings.AssumedOnGroundHeight = 1.0f;

		CharacterBody.SetInvM(1.0f);
		CharacterBody.SetInvILocal(FVec3(1.0f, 1.0f, 1.0f));
		GroundBody.SetInvM(1.0f);
		GroundBody.SetInvILocal(FVec3(1.0f, 1.0f, 1.0f));

		UpdateTwoBody(3, 0);

		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), FVec3(5.0, 0.0, 10.0));
		EXPECT_VECTOR_FLOAT_EQ(GroundBody.P(), FVec3(-5.0, 0.0, 0.0));
	}

	// No overlap or projected overlap but overlap due to other constraints
	// Solver should fix overlap using displacement not correction
	TEST_F(CharacterGroundConstraintSolverTest, TestSingleBody_NormalImpulse_OtherConstraintOverlap)
	{
		CharacterBody.SetX(FVec3(10.0, 10.0, 26.0));
		CharacterBody.SetP(FVec3(10.0, 10.0, 26.0));
		Data.GroundDistance = 16.0f;
		Settings.TargetHeight = 15.0f;
		Settings.RadialForceLimit = 0.0f;
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;

		Solver.SetBodies(&CharacterBody);
		Solver.GatherInput(Dt, Settings, Data);
		SolvePosition(1);

		// Solver should do nothing first iteration as there is no projected overlap
		EXPECT_TRUE(CharacterBody.CP() == FSolverVec3::ZeroVector);
		EXPECT_TRUE(CharacterBody.DP() == FSolverVec3::ZeroVector);

		// Add projected overlap due to another constraint
		CharacterBody.ApplyPositionDelta(FSolverVec3(1.0f, -2.0f, -4.0f));

		Solver.SolvePosition();

		FSolverVec3 Impulse = Solver.GetLinearImpulse(1);
		FSolverVec3 ExpectedSolverImpulse = (1.0 / (CharacterBody.InvM())) * 3.0f * FSolverVec3(0.0, 0.0f, 1.0f);
		EXPECT_VECTOR_FLOAT_EQ(Impulse, ExpectedSolverImpulse);

		// Run a second time to check that the impulse doesn't change - the constraint should be
		// solved exactly in a single iteration for a single body
		Solver.SolvePosition();

		Impulse = Solver.GetLinearImpulse(1);
		EXPECT_VECTOR_FLOAT_EQ(Impulse, ExpectedSolverImpulse);

		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.CorrectedP(), FVec3(11.0, 8.0, 25.0));

		FVec3 ExpectedImpulse = ExpectedSolverImpulse / Dt;
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), FVec3::ZeroVector);
	}

	// Initial overlap at zero velocity with a ground normal pointing at 45 degrees
	// Should be corrected in a single iteration without introducing velocity
	// Correction should be along the ground normal
	TEST_F(CharacterGroundConstraintSolverTest, TestSingleBody_NormalImpulse_InitialOverlapOnSlope)
	{
		CharacterBody.SetX(FVec3(10.0, 10.0, 20.0));
		CharacterBody.SetP(FVec3(10.0, 10.0, 20.0));
		Data.GroundDistance = 10.0f;
		Settings.TargetHeight = 15.0f;
		Settings.RadialForceLimit = 0.0f;
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;
		Data.GroundNormal = FVec3(1.0, 0.0, 1.0);
		Data.GroundNormal.SafeNormalize();

		UpdateSingleBody(1, 0);

		FVec3 ExpectedDeltaPos = 5.0 * FMath::Sin(FMath::DegreesToRadians(45.0)) * Data.GroundNormal;
		FVec3 ExpectedPos = CharacterBody.X() + ExpectedDeltaPos;
		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), ExpectedPos);
		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.V(), FVec3(0.0, 0.0, 0.0));

		FVec3 ExpectedImpulse = (1.0 / (CharacterBody.InvM() * Dt)) * ExpectedDeltaPos;
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), FVec3::ZeroVector);
	}

	// Initial overlap with motion target
	// Target should be reachable
	TEST_F(CharacterGroundConstraintSolverTest, TestSingleBody_MotionTarget_LinearReachable)
	{
		CharacterBody.SetX(FVec3(5.0, 10.0, 20.0));
		CharacterBody.SetP(FVec3(10.0, 10.0, 20.0));
		CharacterBody.SetR(FRotation3::FromAxisAngle(FVec3(0.0, 0.0, 1.0), 0.0));
		CharacterBody.SetQ(CharacterBody.R());
		Data.GroundDistance = 10.0f;
		Settings.TargetHeight = 15.0f;
		Settings.RadialForceLimit = (1.0 / (CharacterBody.InvM() * Dt * Dt)) * 10.1;
		Data.TargetDeltaPosition = FVec3(-5.0, 0.0, 0.0);
		Data.TargetDeltaFacing = -0.4;
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = (1.0 / (CharacterBody.InvILocal().Z * Dt * Dt)) * 0.5;

		UpdateSingleBody(1, 0);

		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), FVec3(0.0, 10.0, 25.0));

		FRotation3 Rot = CharacterBody.R() * FRotation3::FromAxisAngle(FVec3(0.0, 0.0, 1.0), Data.TargetDeltaFacing);
		// Solver uses approximate formula for angular displacement to quaternion, so only expect
		// results to be equal up to order of delta angle cubed
		FReal ErrorTolerance = 0.16 * FMath::Abs(Data.TargetDeltaFacing * Data.TargetDeltaFacing * Data.TargetDeltaFacing);
		EXPECT_NEAR(CharacterBody.Q().W, Rot.W, ErrorTolerance);
		EXPECT_NEAR(CharacterBody.Q().X, Rot.X, ErrorTolerance);
		EXPECT_NEAR(CharacterBody.Q().Y, Rot.Y, ErrorTolerance);
		EXPECT_NEAR(CharacterBody.Q().Z, Rot.Z, ErrorTolerance);

		FVec3 ExpectedImpulse = (1.0 / (CharacterBody.InvM() * Dt)) * FVec3(-10.0, 0.0, 5.0);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);

		FVec3 ExpectedAngularImpulse = (1.0 / (CharacterBody.InvILocal().Z * Dt)) * Data.TargetDeltaFacing * FVec3(0.0, 0.0, 1.0);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), ExpectedAngularImpulse);
	}

	// Initial overlap with motion target
	// Target should be unreachable. Solver should only move up to force limit
	TEST_F(CharacterGroundConstraintSolverTest, TestSingleBody_MotionTarget_LinearUnReachable)
	{
		CharacterBody.SetX(FVec3(10.0, 10.0, 20.0));
		CharacterBody.SetP(FVec3(10.0, 10.0, 20.0));
		Data.GroundDistance = 10.0f;
		Settings.TargetHeight = 15.0f;
		Settings.RadialForceLimit = (1.0 / (CharacterBody.InvM() * Dt * Dt)) * 10.0;
		Data.TargetDeltaPosition = FVec3(-11.0, 0.0, 0.0);
		Data.TargetDeltaFacing = 0.6;
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = (1.0 / (CharacterBody.InvILocal().Z * Dt * Dt)) * 0.5;
		Settings.RadialForceMotionTargetScaling = 0.0f;

		UpdateSingleBody(5, 0);

		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), FVec3(0.0, 10.0, 25.0));

		FRotation3 Rot = CharacterBody.R() * FRotation3::FromAxisAngle(FVec3(0.0, 0.0, 1.0), 0.5);
		// Solver uses approximate formula for angular displacement to quaternion, so only expect
		// results to be equal up to order of delta angle cubed
		FReal ErrorTolerance = 0.16 * 0.5 * 0.5 * 0.5;
		EXPECT_NEAR(CharacterBody.Q().W, Rot.W, ErrorTolerance);
		EXPECT_NEAR(CharacterBody.Q().X, Rot.X, ErrorTolerance);
		EXPECT_NEAR(CharacterBody.Q().Y, Rot.Y, ErrorTolerance);
		EXPECT_NEAR(CharacterBody.Q().Z, Rot.Z, ErrorTolerance);

		FVec3 ExpectedImpulse = (1.0 / (CharacterBody.InvM() * Dt)) * FVec3(-10.0, 0.0, 5.0);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);
		EXPECT_FLOAT_EQ(FMath::Abs(Solver.GetLinearImpulse(Dt).X), Settings.RadialForceLimit * Dt);
		EXPECT_FLOAT_EQ(FMath::Abs(Solver.GetAngularImpulse(Dt).Z), Settings.TwistTorqueLimit * Dt);
	}

	// No overlap with motion target
	// Target should be reachable but off ground so should be no movement
	TEST_F(CharacterGroundConstraintSolverTest, TestSingleBody_MotionTarget_LinearReachableOffGround)
	{
		CharacterBody.SetX(FVec3(5.0, 10.0, 20.0));
		CharacterBody.SetP(FVec3(10.0, 10.0, 20.0));
		Data.GroundDistance = 20.0f;
		Settings.TargetHeight = 18.0f;
		Settings.RadialForceLimit = (1.0 / (CharacterBody.InvM() * Dt * Dt)) * 10.1;
		Data.TargetDeltaPosition = FVec3(-5.0, 0.0, 0.0);
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;
		Settings.AssumedOnGroundHeight = 1.0f;

		UpdateSingleBody(1, 0);

		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), FVec3(10.0, 10.0, 20.0));

		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), FVec3::ZeroVector);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), FVec3::ZeroVector);
	}

	// No overlap with motion target
	// Target should be reachable. Off ground but within assumed on ground height so should reach target
	TEST_F(CharacterGroundConstraintSolverTest, TestSingleBody_MotionTarget_LinearReachableSlightlyOffGround)
	{
		CharacterBody.SetX(FVec3(5.0, 10.0, 20.0));
		CharacterBody.SetP(FVec3(10.0, 10.0, 20.0));
		Data.GroundDistance = 20.0f;
		Settings.TargetHeight = 18.0f;
		Settings.RadialForceLimit = (1.0 / (CharacterBody.InvM() * Dt * Dt)) * 10.1;
		Data.TargetDeltaPosition = FVec3(-5.0, 0.0, 0.0);
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;
		Settings.AssumedOnGroundHeight = 2.1f;

		UpdateSingleBody(1, 0);

		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), FVec3(0.0, 10.0, 20.0));

		FVec3 ExpectedImpulse = (1.0 / (CharacterBody.InvM() * Dt)) * FVec3(-10.0, 0.0, 0.0);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), FVec3::ZeroVector);
	}

	// No overlap with motion target
	// Motion target into slope is projected onto the ground plane
	TEST_F(CharacterGroundConstraintSolverTest, TestSingleBody_MotionTarget_OnSlope)
	{
		CharacterBody.SetX(FVec3(0.0, 0.0, 10.0));
		CharacterBody.SetP(FVec3(0.0, 0.0, 10.0));
		Data.GroundDistance = 10.0f;
		Settings.TargetHeight = 10.0f;
		Settings.RadialForceLimit = (1.0 / (CharacterBody.InvM() * Dt * Dt)) * 100.0;
		Data.TargetDeltaPosition = FVec3(10.0, 0.0, 0.0);
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;
		Settings.AssumedOnGroundHeight = 1.0f;
		Data.GroundNormal = FVec3(-1.0, 0.0, 1.0);
		Data.GroundNormal.SafeNormalize();

		UpdateSingleBody(10, 0);

		FVec3 ExpectedDeltaPos = Data.TargetDeltaPosition - FVec3::DotProduct(Data.TargetDeltaPosition, Data.GroundNormal) * Data.GroundNormal;
		FVec3 ExpectedPos = CharacterBody.X() + ExpectedDeltaPos;
		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), ExpectedPos);

		FVec3 ExpectedImpulse = (1.0 / (CharacterBody.InvM() * Dt)) * ExpectedDeltaPos;
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), FVec3::ZeroVector);
	}

	// No overlap with zero motion target and moving ground body
	// Character body should move with the ground body
	// Ground body should be unaffected
	TEST_F(CharacterGroundConstraintSolverTest, TestTwoBody_MotionTarget_MovingGround)
	{
		CharacterBody.SetX(FVec3(0.0, 0.0, 10.0));
		CharacterBody.SetP(FVec3(0.0, 0.0, 10.0));
		GroundBody.SetX(FVec3(0.0, 0.0, 0.0));
		GroundBody.SetP(FVec3(0.0, 10.0, 0.0));
		Data.GroundDistance = 10.0f;
		Settings.TargetHeight = 10.0f;
		Settings.RadialForceLimit = (1.0 / (CharacterBody.InvM() * Dt * Dt)) * 100.0;
		Data.TargetDeltaPosition = FVec3(10.0, -5.0, 0.0);
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;
		Settings.AssumedOnGroundHeight = 1.0f;
		Settings.MotionTargetMassBias = 0.0f;

		UpdateTwoBody(1, 0);

		FVec3 ExpectedDeltaPos =  GroundBody.P() - GroundBody.X() + Data.TargetDeltaPosition;
		FVec3 ExpectedPos = CharacterBody.X() + ExpectedDeltaPos;
		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), ExpectedPos);
		EXPECT_VECTOR_FLOAT_EQ(GroundBody.P(), FVec3(0.0, 10.0, 0.0));

		FVec3 ExpectedImpulse = (1.0 / (CharacterBody.InvM() * Dt)) * ExpectedDeltaPos;
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), FVec3::ZeroVector);
	}

	// No overlap with zero motion target and rotating ground body
	// Character body should rotate with the ground body
	// Ground body should be unaffected
	TEST_F(CharacterGroundConstraintSolverTest, TestTwoBody_MotionTarget_RotatingGround)
	{
		CharacterBody.SetX(FVec3(10.0, 0.0, 10.0));
		CharacterBody.SetP(FVec3(10.0, 0.0, 10.0));
		GroundBody.SetInvM(0.001);
		GroundBody.SetInvILocal(FVec3(0.001, 0.001, 0.001));
		GroundBody.SetX(FVec3(0.0, 0.0, 0.0));
		GroundBody.SetP(FVec3(0.0, 0.0, 0.0));
		GroundBody.SetQ(FRotation3::FromAxisAngle(FVec3(0.0, 0.0, 1.0), 0.1));
		Data.GroundDistance = 10.0f;
		Settings.TargetHeight = 10.0f;
		Settings.RadialForceLimit = (1.0 / (CharacterBody.InvM() * Dt * Dt)) * 100.0;
		Data.TargetDeltaPosition = FVec3(0.0, 0.0, 0.0);
		Data.TargetDeltaFacing = 0.0f;
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = (1.0 / (CharacterBody.InvILocal().Z * Dt * Dt)) * 100.0;
		Settings.AssumedOnGroundHeight = 1.0f;

		GroundBody.SetInvM(0.0);
		GroundBody.SetInvILocal(FVec3(0.0, 0.0, 0.0));

		UpdateTwoBody(1, 0);

		FVec3 ExpectedDeltaPos = GroundBody.Q() * FVec3(10.0, 0.0, 0.0) - FVec3(10.0, 0.0, 0.0);
		FVec3 ExpectedPos = CharacterBody.X() + ExpectedDeltaPos;
		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), ExpectedPos);
		EXPECT_VECTOR_FLOAT_EQ(GroundBody.P(), FVec3(0.0, 0.0, 0.0));

		// Solver uses approximate formula for angular displacement to quaternion, so only expect
		// results to be equal up to order of delta angle cubed
		FReal ErrorTolerance = 0.16 * 0.1 * 0.1 * 0.1;
		EXPECT_NEAR(CharacterBody.Q().W, GroundBody.Q().W, ErrorTolerance);
		EXPECT_NEAR(CharacterBody.Q().X, GroundBody.Q().X, ErrorTolerance);
		EXPECT_NEAR(CharacterBody.Q().Y, GroundBody.Q().Y, ErrorTolerance);
		EXPECT_NEAR(CharacterBody.Q().Z, GroundBody.Q().Z, ErrorTolerance);

		FVec3 ExpectedImpulse = (1.0 / (CharacterBody.InvM() * Dt)) * ExpectedDeltaPos;
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);

		FVec3 ExpectedAngularImpulse = (1.0 / (CharacterBody.InvILocal().Z * Dt)) * FVec3(0.0, 0.0, 0.1);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), ExpectedAngularImpulse);
	}

	// Initial overlap at zero velocity. Two body
	// Character body should be corrected in a single iteration without introducing velocity
	// Ground body should not move (doesn't currently get corrected)
	TEST_F(CharacterGroundConstraintSolverTest, TestTwoBody_NormalImpulse_InitialOverlap)
	{
		CharacterBody.SetX(FVec3(10.0, 10.0, 20.0));
		CharacterBody.SetP(FVec3(10.0, 10.0, 20.0));
		GroundBody.SetInvM(0.001);
		GroundBody.SetInvILocal(FVec3(0.0005, 0.0005, 0.0005));
		GroundBody.SetX(FVec3(10.0, 5.0, 0.0));
		GroundBody.SetP(FVec3(10.0, 5.0, 0.0));
		Data.GroundDistance = 10.0f;
		Settings.TargetHeight = 15.0f;
		Settings.RadialForceLimit = 0.0f;
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;

		UpdateTwoBody(1, 0);

		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), FVec3(10.0, 10.0, 25.0));
		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.V(), FVec3(0.0, 0.0, 0.0));
		EXPECT_VECTOR_FLOAT_EQ(GroundBody.P(), FVec3(10.0, 5.0, 0.0));

		FVec3 ExpectedImpulse = (1.0 / (CharacterBody.InvM() * Dt)) * FVec3(0.0, 0.0, 5.0);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), FVec3::ZeroVector);
	}

	// No overlap but projected overlap. Two body
	// Character body and ground body have the same mass
	// Linear displacement should be equal and opposite
	TEST_F(CharacterGroundConstraintSolverTest, TestTwoBody_NormalImpulse_NoInitialOverlap_FinalOverlap)
	{
		CharacterBody.SetX(FVec3(10.0, 10.0, 30.0));
		CharacterBody.SetP(FVec3(10.0, 10.0, 24.0));
		GroundBody.SetX(FVec3(10.0, 10.0, 0.0));
		GroundBody.SetP(FVec3(10.0, 10.0, 6.0));

		GroundBody.SetInvM(CharacterBody.InvM());
		GroundBody.SetInvILocal(CharacterBody.InvILocal());

		Data.GroundDistance = 20.0f;
		Settings.TargetHeight = 10.0f;
		Settings.RadialForceLimit = 0.0f;
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;

		UpdateTwoBody(1, 0);

		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.P(), FVec3(10.0, 10.0, 25.0));
		EXPECT_VECTOR_FLOAT_EQ(GroundBody.P(), FVec3(10.0, 10.0, 5.0));

		FVec3 ExpectedImpulse = (1.0 / ((CharacterBody.InvM() + GroundBody.InvM()) * Dt)) * FVec3(0.0, 0.0, 2.0);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), FVec3::ZeroVector);
	}

	// No overlap but projected overlap
	// Solver should fix overlap
	// Mass of ground shoul be adjusted so that max character/ground mass ratio is respected
	TEST_F(CharacterGroundConstraintSolverTest, TestSingleBody_NormalImpulse_MassConditioning)
	{
		CharacterBody.SetX(FVec3(0.0, 0.0, 20.0));
		CharacterBody.SetP(FVec3(0.0, 0.0, 10.0));
		CharacterBody.SetV(FVec3(0.0, 0.0, -10.0 / Dt));

		GroundBody.SetX(FVec3(0.0, 0.0, 0.0));
		GroundBody.SetP(FVec3(0.0, 0.0, 0.0));
		GroundBody.SetV(FVec3(0.0, 0.0, 0.0));

		Data.GroundDistance = 20.0f;
		Settings.TargetHeight = 15.0f;
		Settings.RadialForceLimit = 0.0f;
		Settings.SwingTorqueLimit = 0.0f;
		Settings.TwistTorqueLimit = 0.0f;

		// Set the mass of the ground to be half the mass of the character (i.e. ratio = 2)
		// and set the max ratio to 1.5. With a ground mass of 100 the character mass
		// should be adjusted to look like 150
		Settings.MaxCharacterGroundMassRatio = 1.5f;
		CharacterBody.SetInvM(1.0 / 200.0);
		GroundBody.SetInvM(1.0 / 100.0);
		GroundBody.SetInvILocal(FVec3(0.01, 0.01, 0.01));
		const FReal ExpectedCharacterInvMass = 1.0f / 150.0f;

		Solver.SetBodies(&CharacterBody, &GroundBody);
		Solver.GatherInput(Dt, Settings, Data);
		SolvePosition(1);

		EXPECT_TRUE(CharacterBody.CP() == FSolverVec3::ZeroVector);
		EXPECT_FALSE(CharacterBody.DP() == FSolverVec3::ZeroVector);

		FVec3 ExpectedImpulse = (1.0 / ((GroundBody.InvM() + ExpectedCharacterInvMass) * Dt)) * FVec3(0.0, 0.0, 5.0);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetLinearImpulse(Dt), ExpectedImpulse);
		EXPECT_VECTOR_FLOAT_EQ(Solver.GetAngularImpulse(Dt), FVec3::ZeroVector);

		const FVec3 ExpectedCharacterCorrectedP = FVec3(0.0, 0.0, 10.0) + ExpectedCharacterInvMass * ExpectedImpulse * Dt;
		const FVec3 ExpectedGroundCorrectedP = FVec3(0.0, 0.0, 0.0) - FReal(GroundBody.InvM()) * ExpectedImpulse * Dt;
		EXPECT_VECTOR_FLOAT_EQ(CharacterBody.CorrectedP(), ExpectedCharacterCorrectedP);
		EXPECT_VECTOR_FLOAT_EQ(GroundBody.CorrectedP(), ExpectedGroundCorrectedP);
		EXPECT_VECTOR_FLOAT_EQ(FVec3(CharacterBody.CorrectedP() - GroundBody.CorrectedP()), FVec3(0.0, 0.0, 15.0));
	}

} // namespace ChaosTest