// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestBoneHierarchy.h"

#include "PhysicsProxy/AnalyticImplicitGroup.h"
#include "BoneHierarchy.h"

#include "Chaos/Sphere.h"

namespace GeometryCollectionTest
{
	template <class TImplicitShape>
	void AllOnSurface(const TImplicitShape *Shape, const TArray<Chaos::FVec3> &Points, const Chaos::FReal Tolerance = KINDA_SMALL_NUMBER)
	{
		for (auto &Pt : Points)
		{
			const Chaos::FReal Phi = Shape->SignedDistance(Pt);
			EXPECT_NEAR(Phi, 0, Tolerance);
		}
	}

	void TestSphere(const Chaos::FVec3& Center, const Chaos::FReal Radius, const FTransform &BoneRelXf)
	{
		Chaos::FSphere* Sphere = new Chaos::FSphere(Center, Radius);
		TUniquePtr<FAnalyticImplicitGroup> Group1(new FAnalyticImplicitGroup("Root", 0));
		Group1->Init(1);
		Group1->SetParentBoneIndex(INDEX_NONE);
		Group1->Add(BoneRelXf, Sphere);
		const TArray<Chaos::FVec3> *Points = Group1->BuildSamplePoints(1.0, 1, 1000);
		check(Points);
		check(Points->Num());
		Chaos::FImplicitObject *Implicit = Group1->BuildSimImplicitObject();
		AllOnSurface(Implicit, *Points);
	}
	void TestSphere2(
		const Chaos::FVec3& Center1, 
		const Chaos::FVec3& Center2, 
		const Chaos::FReal Radius1,
		const Chaos::FReal Radius2,
		const FTransform &BoneRelXf1,
		const FTransform &BoneRelXf2)
	{
		Chaos::FSphere* Sphere1 = new Chaos::FSphere(Center1, Radius1);
		Chaos::FSphere* Sphere2 = new Chaos::FSphere(Center2, Radius2);
		TUniquePtr<FAnalyticImplicitGroup> Group1(new FAnalyticImplicitGroup("Root", 0));
		Group1->Init(2);
		Group1->SetParentBoneIndex(INDEX_NONE);
		Group1->Add(BoneRelXf1, Sphere1);
		Group1->Add(BoneRelXf2, Sphere2);
		const TArray<Chaos::FVec3> *Points = Group1->BuildSamplePoints(1.0, 1, 1000);
		check(Points);
		check(Points->Num());
		Chaos::FImplicitObject *Implicit = Group1->BuildSimImplicitObject();
		AllOnSurface(Implicit, *Points);
	}

	void RunAnalyticImplicitGroupTest()
	{
		TestSphere(Chaos::FVec3(0, 0, 0), 1.0, FTransform::Identity);
		TestSphere(Chaos::FVec3(0, 0, 0), 1.0, FTransform(FVector(1,0,0)));
		TestSphere(Chaos::FVec3(0, 0, 0), 1.0, FTransform(FVector(1,1,0)));
		TestSphere(Chaos::FVec3(0, 0, 0), 1.0, FTransform(FVector(1,1,1)));

		//This test fails in Dev-Physics for spheres of larger radii
		//TestSphere(R, Chaos::FVec3(0, 0, 0), 10.0, FTransform::Identity);
		//TestSphere(R, Chaos::FVec3(0, 0, 0), 10.0, FTransform(FVector(1,0,0)));
		//TestSphere(R, Chaos::FVec3(0, 0, 0), 10.0, FTransform(FVector(1,1,0)));
		//TestSphere(R, Chaos::FVec3(0, 0, 0), 10.0, FTransform(FVector(1,1,1)));

		TestSphere2( 
			Chaos::FVec3(0, 0, 0), 
			Chaos::FVec3(0, 0, 0), 
			1.0, 
			1.0, 
			FTransform(FVector(-2,0,0)),
			FTransform(FVector(2,0,0)));
	}

	void RunBoneHierarchyTest()
	{
		FBoneHierarchy Hierarchy;
		Hierarchy.InitPreAdd(2);
		{
			TUniquePtr<FAnalyticImplicitGroup> Group1(
				new FAnalyticImplicitGroup("Root", 0));
			Group1->SetParentBoneIndex(INDEX_NONE);
			Group1->Add(FTransform::Identity, new Chaos::FSphere(Chaos::FVec3(0), 1.));
			Hierarchy.Add(MoveTemp(Group1));

			TUniquePtr<FAnalyticImplicitGroup> Group2(
				new FAnalyticImplicitGroup("Bone1", 1));
			Group2->SetParentBoneIndex(0);
			Group2->Add(FTransform::Identity, new Chaos::FSphere(Chaos::FVec3(0), 1.));
			Hierarchy.Add(MoveTemp(Group2));

			Hierarchy.InitPostAdd();
		}

		Hierarchy.PrepareForUpdate();
		Hierarchy.SetAnimLocalSpaceTransform(0,
			FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0)), 
			FVector(1,0,0)));
		Hierarchy.SetAnimLocalSpaceTransform(1,
			FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0)), 
			FVector(0,1,0)));
		Hierarchy.SetActorWorldSpaceTransform(
			FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0)), 
			FVector(0,0,1)));
		Hierarchy.PrepareAnimWorldSpaceTransforms();
		{
			const FTransform* Xf = Hierarchy.GetAnimWorldSpaceTransformsForBone(1);
			check(Xf);
			const FTransform XfExpected(
				FQuat::MakeFromEuler(FVector(0, 0, 0)), 
				FVector(1, 1, 1));
			check(Xf->Equals(XfExpected));
		}

		Hierarchy.PrepareForUpdate();
		Hierarchy.SetAnimLocalSpaceTransform(0,
			FTransform(FQuat::MakeFromEuler(FVector(10, 0, 0)), 
			FVector(0,0,0)));
		Hierarchy.SetAnimLocalSpaceTransform(1,
			FTransform(FQuat::MakeFromEuler(FVector(10, 0, 0)), 
			FVector(0,0,0)));
		Hierarchy.SetActorWorldSpaceTransform(
			FTransform(FQuat::MakeFromEuler(FVector(10, 0, 0)), 
			FVector(0,0,0)));
		Hierarchy.PrepareAnimWorldSpaceTransforms();
		{
			const FTransform* Xf = Hierarchy.GetAnimWorldSpaceTransformsForBone(1);
			check(Xf);
			const FTransform XfExpected(
				FQuat::MakeFromEuler(FVector(30, 0, 0)), 
				FVector(0, 0, 0));
			check(Xf->Equals(XfExpected));
		}
	}

	void TestImplicitBoneHierarchy()
	{
		RunAnalyticImplicitGroupTest();
		RunBoneHierarchyTest();
	}
} // namespace GeometryCollectionTest
