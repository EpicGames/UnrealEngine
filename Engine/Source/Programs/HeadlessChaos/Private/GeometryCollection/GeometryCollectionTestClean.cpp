// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestClean.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/TransformCollection.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "../Resource/FracturedGeometry.h"

namespace GeometryCollectionTest
{
	void TestDeleteCoincidentVertices()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.f, 0.f, 0.f)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.f, 0.f, 0.f)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.5f, 0.f, 1.f)), FVector(1.0)));

		//  0
		//  ...1
		//  ......2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);
		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);
		(Collection->Parent)[2] = 1;
//		GeometryCollectionAlgo::ParentTransform(Collection, 0, 1);

		FGeometryCollection* Coll = Collection.Get();

		GeometryCollectionAlgo::DeleteCoincidentVertices(Coll, 1e-2f);

		
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::VerticesGroup), 24);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::FacesGroup), 36);

		GeometryCollectionAlgo::DeleteZeroAreaFaces(Coll, 1e-4f);

		EXPECT_EQ(Coll->NumElements(FGeometryCollection::VerticesGroup), 24);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::FacesGroup), 36);
	}

	void TestDeleteCoincidentVertices2()
	{
		FGeometryCollection* Coll = FGeometryCollection::NewGeometryCollection(FracturedGeometry::RawVertexArray,
																   			   FracturedGeometry::RawIndicesArray,
																			   FracturedGeometry::RawBoneMapArray,
																			   FracturedGeometry::RawTransformArray,
																			   FracturedGeometry::RawLevelArray,
																			   FracturedGeometry::RawParentArray,
																			   FracturedGeometry::RawChildrenArray,
																			   FracturedGeometry::RawSimulationTypeArray,
																			   FracturedGeometry::RawStatusFlagsArray);

		EXPECT_EQ(Coll->NumElements(FGeometryCollection::VerticesGroup), 667);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::FacesGroup), 493);

		GeometryCollectionAlgo::DeleteCoincidentVertices(Coll, 1e-2f);

		EXPECT_EQ(Coll->NumElements(FGeometryCollection::VerticesGroup), 270);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::FacesGroup), 493);
	}

	void TestDeleteZeroAreaFaces()
	{
		FGeometryCollection* Coll = FGeometryCollection::NewGeometryCollection(FracturedGeometry::RawVertexArray,
																			   FracturedGeometry::RawIndicesArray,
																			   FracturedGeometry::RawBoneMapArray,
																			   FracturedGeometry::RawTransformArray,
																			   FracturedGeometry::RawLevelArray,
																			   FracturedGeometry::RawParentArray,
																			   FracturedGeometry::RawChildrenArray,
																			   FracturedGeometry::RawSimulationTypeArray,
																			   FracturedGeometry::RawStatusFlagsArray
			);

		EXPECT_EQ(Coll->NumElements(FGeometryCollection::VerticesGroup), 667);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::FacesGroup), 493);

		GeometryCollectionAlgo::DeleteZeroAreaFaces(Coll, 1e-4f);

		EXPECT_EQ(Coll->NumElements(FGeometryCollection::VerticesGroup), 667);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::FacesGroup), 493);
	}

	void TestFillHoles()
	{
		FGeometryCollection* Coll = FGeometryCollection::NewGeometryCollection(FracturedGeometry::RawVertexArray,
			FracturedGeometry::RawIndicesArray,
			FracturedGeometry::RawBoneMapArray,
			FracturedGeometry::RawTransformArray,
			FracturedGeometry::RawLevelArray,
			FracturedGeometry::RawParentArray,
			FracturedGeometry::RawChildrenArray,
			FracturedGeometry::RawSimulationTypeArray,
			FracturedGeometry::RawStatusFlagsArray);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::VerticesGroup), 667);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::FacesGroup), 493);

		TArray<TArray<TArray<int32>>> BoundaryVertexIndices;
		Coll->RemoveElements(FGeometryCollection::FacesGroup, { 0,1,2 });

		GeometryCollectionAlgo::FindOpenBoundaries(Coll, 1e-2f, BoundaryVertexIndices);

		auto CountHoles = [](const TArray<TArray<TArray<int32>>> &InBoundaryVertexIndices)
		{
			int NumHoles = 0;
			for (const TArray<TArray<int32>> &GeomBoundaries : InBoundaryVertexIndices)
			{
				NumHoles += GeomBoundaries.Num();
			}
			return NumHoles;
		};

		auto CountTinyFaces = [](const FGeometryCollection *InColl, Chaos::FReal InTinyNumber = 1e-4)
		{
			int TinyFaces = 0;
			for (const FIntVector & Face : InColl->Indices)
			{
				FVector p10 = FVector(InColl->Vertex[Face.Y] - InColl->Vertex[Face.X]);
				FVector p20 = FVector(InColl->Vertex[Face.Z] - InColl->Vertex[Face.X]);
				FVector Cross = FVector::CrossProduct(p20, p10);
				if (Cross.SizeSquared() < InTinyNumber)
				{
					TinyFaces++;
				}
			}
			return TinyFaces;
		};
		
		int32 TinyFacesBefore = CountTinyFaces(Coll);
		EXPECT_EQ(CountHoles(BoundaryVertexIndices), 3);
		GeometryCollectionAlgo::TriangulateBoundaries(Coll, BoundaryVertexIndices);
		int32 TinyFacesAfter = CountTinyFaces(Coll);
		EXPECT_EQ(CountTinyFaces(Coll), TinyFacesBefore);

		BoundaryVertexIndices.Empty();
		GeometryCollectionAlgo::FindOpenBoundaries(Coll, 1e-2f, BoundaryVertexIndices);
		EXPECT_EQ(CountHoles(BoundaryVertexIndices), 2);

		GeometryCollectionAlgo::TriangulateBoundaries(Coll, BoundaryVertexIndices, true, 0);
		BoundaryVertexIndices.Empty();

		GeometryCollectionAlgo::FindOpenBoundaries(Coll, 1e-2f, BoundaryVertexIndices);
		EXPECT_EQ(CountHoles(BoundaryVertexIndices), 0);
		EXPECT_GT(CountTinyFaces(Coll), TinyFacesBefore);


		EXPECT_EQ(Coll->NumElements(FGeometryCollection::VerticesGroup), 667);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::FacesGroup), 496);

		EXPECT_EQ(Coll->NumElements(FGeometryCollection::VerticesGroup), 667);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::FacesGroup), 496);

		EXPECT_TRUE(Coll->HasContiguousFaces());
		EXPECT_TRUE(Coll->HasContiguousVertices());
		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Coll));
	}

	void TestDeleteHiddenFaces()
	{
		FGeometryCollection* Coll = FGeometryCollection::NewGeometryCollection(FracturedGeometry::RawVertexArray,
																			   FracturedGeometry::RawIndicesArray,
																			   FracturedGeometry::RawBoneMapArray,
																			   FracturedGeometry::RawTransformArray,
																			   FracturedGeometry::RawLevelArray,
																			   FracturedGeometry::RawParentArray,
																			   FracturedGeometry::RawChildrenArray,
																			   FracturedGeometry::RawSimulationTypeArray,
																			   FracturedGeometry::RawStatusFlagsArray);

		EXPECT_EQ(Coll->NumElements(FGeometryCollection::VerticesGroup), 667);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::FacesGroup), 493);

		TManagedArray<bool>& VisibleArray = Coll->Visible;

		int32 NumFaces = Coll->NumElements(FGeometryCollection::FacesGroup);
		for (int32 Idx = 0; Idx < NumFaces; ++Idx)
		{
			if (!(Idx % 5))
			{
				VisibleArray[Idx] = false;
			}
		}

		EXPECT_EQ(Coll->NumElements(FGeometryCollection::VerticesGroup), 667);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::FacesGroup), 493);

		GeometryCollectionAlgo::DeleteHiddenFaces(Coll);

		EXPECT_EQ(Coll->NumElements(FGeometryCollection::VerticesGroup), 667);
		EXPECT_EQ(Coll->NumElements(FGeometryCollection::FacesGroup), 394);
	}
}

