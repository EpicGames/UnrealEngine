// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestClustering.h"
#include "GeometryCollection/GeometryCollectionTestFramework.h"
#include "GeometryCollection/GeometryCollectionTestUtility.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"

#include "GeometryCollection/TransformCollection.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "Field/FieldSystem.h"
#include "Field/FieldSystemNodes.h"


#include "GeometryCollectionProxyData.h"
#include "PhysicsProxy/PhysicsProxies.h"
#include "Chaos/ErrorReporter.h"
#include "ChaosSolversModule.h"
#include "PhysicsSolver.h"
#include "Chaos/PBDRigidClustering.h"

#include "HAL/IConsoleManager.h"
#include "HeadlessChaosTestUtility.h"

DEFINE_LOG_CATEGORY_STATIC(GCTCL_Log, Verbose, All);

// #TODO Lots of duplication in here, anyone making solver or object changes
// has to go and fix up so many callsites here and they're all pretty much
// Identical. The similar code should be pulled out

namespace GeometryCollectionTest
{
	using namespace ChaosTest;

	bool ClusterMapContains(const Chaos::FRigidClustering::FClusterMap& ClusterMap, const FPBDRigidParticleHandle* InKey, TArray<FPBDRigidParticleHandle*> Elements)
	{
		if (ClusterMap.Num())
		{
			if (const Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>* Key = InKey->CastToClustered())
			{
				if (ClusterMap.Contains(Key))
				{
					if (ClusterMap[Key].Num() == Elements.Num())
					{
						for (FPBDRigidParticleHandle* Element : Elements)
						{
							if (!ClusterMap[Key].Contains(Element))
							{
								return false;
							}
						}

						return true;
					}
				}
			}
		}
		return false;
	}


	GTEST_TEST(AllTraits,GeometryCollection_RigidBodies_ClusterTest_SingleLevelNonBreaking)
	{
		FFramework UnitTest;

		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();
		UnitTest.AddSimulationObject(Floor);

		TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, -10, 10)),FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, 10, 10)), FVector(1.0)));
		EXPECT_EQ(RestCollection->Transform.Num(), 2);

		FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(RestCollection.Get());
		EXPECT_EQ(RestCollection->Transform.Num(), 3);
		RestCollection->Transform[2] = FTransform3f(FQuat4f::MakeFromEuler(FVector3f(90.0, 0, 0.)), FVector3f(0, 0, 40));

		//GeometryCollectionAlgo::PrintParentHierarchy(RestCollection.Get());

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 1000.f };
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();

		FReal StartingRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size(), CurrentRigidDistance = 0.f;

		TManagedArray<bool>& Active = Collection->DynamicCollection->Active;

		EXPECT_FALSE(Active[0]);
		EXPECT_FALSE(Active[1]);
		EXPECT_TRUE(Active[2]); // only the root cluster should be active when using clustering 
		UnitTest.Advance();
		EXPECT_FALSE(Active[0]);
		EXPECT_FALSE(Active[1]);
		EXPECT_TRUE(Active[2]);

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto & ClusterMap = Clustering.GetChildrenMap();
		EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection->PhysObject->GetSolverClusterHandle_Internal(0),
			{ Collection->PhysObject->GetParticle_Internal(0),Collection->PhysObject->GetParticle_Internal(1) }));

		FReal InitialZ = Collection->RestCollection->Transform[2].GetTranslation().Z;
		for (int Frame = 1; Frame < 10; Frame++)
		{			
			UnitTest.Advance();

			EXPECT_FALSE(Active[0]);
			EXPECT_FALSE(Active[1]);
			EXPECT_TRUE(Active[2]);

			CurrentRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size();
			EXPECT_LT(FMath::Abs(CurrentRigidDistance - StartingRigidDistance), SMALL_NUMBER); // two bodies under cluster maintain distance
			EXPECT_LT(Collection->DynamicCollection->GetTransform(2).GetTranslation().Z, InitialZ); // body should be falling and decreasing in Z			
		}

		EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection->PhysObject->GetSolverClusterHandle_Internal(0),
			{ Collection->PhysObject->GetParticle_Internal(0),Collection->PhysObject->GetParticle_Internal(1) }));

	}

	GTEST_TEST(AllTraits, GeometryCollection_DynamicCollection_ChildrenAccess)
	{
		FFramework UnitTest;

		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();
		UnitTest.AddSimulationObject(Floor);

		TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, -10, 10)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, 10, 10)), FVector(1.0)));
		EXPECT_EQ(RestCollection->Transform.Num(), 2);

		FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(RestCollection.Get());
		EXPECT_EQ(RestCollection->Transform.Num(), 3);
		RestCollection->Transform[2] = FTransform3f(FQuat4f::MakeFromEuler(FVector3f(90.0, 0, 0.)), FVector3f(0, 0, 40));

		//GeometryCollectionAlgo::PrintParentHierarchy(RestCollection.Get());

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 1000.f };
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();

		FReal StartingRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size(), CurrentRigidDistance = 0.f;

		TManagedArray<bool>& Active = Collection->DynamicCollection->Active;

		EXPECT_FALSE(Active[0]);
		EXPECT_FALSE(Active[1]);
		EXPECT_TRUE(Active[2]); // only the root cluster should be active when using clustering 
		UnitTest.Advance();
		EXPECT_FALSE(Active[0]);
		EXPECT_FALSE(Active[1]);
		EXPECT_TRUE(Active[2]);


		int32 ChildCount[3] = {0, 0, 0};
		for (int32 Index = 0; Index < 3; ++Index)
		{
			Collection->DynamicCollection->IterateThroughChildren(Index, [&](int32 ChildIndex)
			{
				ChildCount[Index]++;
				return true;
			});
		}
		EXPECT_EQ(ChildCount[0], 0);
		EXPECT_EQ(ChildCount[1], 0);
		EXPECT_EQ(ChildCount[2], 2);
	}


	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_DeactivateClusterParticle)
	{
		FFramework UnitTest;

		// 5 cube leaf nodes
		TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(20.f)),FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(30.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(40.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(50.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(60.f)), FVector(1.0)));

		// 4 mid-level cluster parents
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 5, { 4,3 }, true, false); // just validate at end of construction
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 6, { 5,2 }, true, false);
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 7, { 6,1 }, true, false);
		FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(RestCollection.Get());
		FGeometryCollectionClusteringUtility::ValidateResults(RestCollection.Get());		

		//GeometryCollectionAlgo::PrintParentHierarchy(RestCollection.Get());

		
		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 50.0, 50.0, 50.0, FLT_MAX };
		Params.MaxClusterLevel = 1;	

		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();		

		FReal StartingRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size(), CurrentRigidDistance = 0.f;

		UnitTest.Advance();

			TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			Collection->PhysObject->GetParticle_Internal(4),
			Collection->PhysObject->GetParticle_Internal(5),
			Collection->PhysObject->GetParticle_Internal(6),
			Collection->PhysObject->GetParticle_Internal(7),
			Collection->PhysObject->GetParticle_Internal(8),
		};

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto & ClusterMap = Clustering.GetChildrenMap();

		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[4], ParticleHandles[3] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[5], ParticleHandles[2] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[7], { ParticleHandles[6], ParticleHandles[1] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[8], { ParticleHandles[7], ParticleHandles[0] }));

		TArray<bool> Conditions = { false,false };
		TArray<bool> DisabledFlags;

		for (int Frame = 1; Frame < 4; Frame++)
		{
			UnitTest.Advance();
			
			if (Frame == 2)
			{
				Clustering.DeactivateClusterParticle(ParticleHandles[8]);
			}

			DisabledFlags.Reset();
			for (const auto* Handle : ParticleHandles)
			{
				DisabledFlags.Add(Handle->Disabled());
			}

			if (Frame == 1)
			{
				if (DisabledFlags[0] == true &&
					DisabledFlags[1] == true &&
					DisabledFlags[2] == true &&
					DisabledFlags[3] == true &&
					DisabledFlags[4] == true &&
					DisabledFlags[5] == true &&
					DisabledFlags[6] == true &&
					DisabledFlags[7] == true &&
					DisabledFlags[8] == false)
				{
					Conditions[0] = true;
				}
			}
			else if (Frame == 2 || Frame == 3)
			{
				if (Conditions[0] == true)
				{
					if (DisabledFlags[0] == false &&
						DisabledFlags[1] == true &&
						DisabledFlags[2] == true &&
						DisabledFlags[3] == true &&
						DisabledFlags[4] == true &&
						DisabledFlags[5] == true &&
						DisabledFlags[6] == true &&
						DisabledFlags[7] == false &&
						DisabledFlags[8] == true)
					{
						Conditions[1] = true;

						EXPECT_TRUE(!ClusterMap.Contains(ParticleHandles[8]));
						EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[7], { ParticleHandles[6], ParticleHandles[1] }));
						EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[5], ParticleHandles[2] }));
						EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[4], ParticleHandles[3] }));
					}
				}
			}
	
		}
	
		for (int i = 0; i < Conditions.Num(); i++)
		{
			EXPECT_TRUE(Conditions[i]);
		}	
	}
	
	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_BreakClusterParticle)
	{
		FFramework UnitTest;

		// 5 cube leaf nodes
		TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(20.f)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(30.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(40.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(50.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(60.f)), FVector(1.0)));

		// 4 mid-level cluster parents
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 5, { 4,3 }, true, false); // just validate at end of construction
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 6, { 5,2 }, true, false);
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 7, { 6,1 }, true, false);
		FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(RestCollection.Get());
		FGeometryCollectionClusteringUtility::ValidateResults(RestCollection.Get());


		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 50.0, 50.0, 50.0, FLT_MAX };
		Params.MaxClusterLevel = 1;

		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();

		FReal StartingRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size(), CurrentRigidDistance = 0.f;

		UnitTest.Advance();

		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			Collection->PhysObject->GetParticle_Internal(4),
			Collection->PhysObject->GetParticle_Internal(5),
			Collection->PhysObject->GetParticle_Internal(6),
			Collection->PhysObject->GetParticle_Internal(7),
			Collection->PhysObject->GetParticle_Internal(8),
		};

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();

		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[4], ParticleHandles[3] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[5], ParticleHandles[2] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[7], { ParticleHandles[6], ParticleHandles[1] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[8], { ParticleHandles[7], ParticleHandles[0] }));


		TArray<bool> Conditions = { false,false };
		TArray<bool> DisabledFlags;

		for (int Frame = 1; Frame < 4; Frame++)
		{
			UnitTest.Advance();

			if (Frame == 2)
			{
				Clustering.SetExternalStrain(ParticleHandles[0], 50.0f);
				Clustering.BreakingModel();
			}

			DisabledFlags.Reset();
			for (const auto* Handle : ParticleHandles)
			{
				DisabledFlags.Add(Handle->Disabled());
			}

			//UE_LOGF(GCTCL_Log, Verbose, "FRAME : %d", Frame);
			//for (int32 rdx = 0; rdx < (int32)Particles.Size(); rdx++)
			//{
			//  UE_LOGF(GCTCL_Log, Verbose, "... ... ...Disabled[%d] : %d", rdx, Particles.Disabled(rdx));
			//  UE_LOGF(GCTCL_Log, Verbose, "... ... ...    InvM[%d] : %f", rdx, Particles.InvM(rdx));
			//}

			if (Frame == 1)
			{
				if (DisabledFlags[0] == true &&
					DisabledFlags[1] == true &&
					DisabledFlags[2] == true &&
					DisabledFlags[3] == true &&
					DisabledFlags[4] == true &&
					DisabledFlags[5] == true &&
					DisabledFlags[6] == true &&
					DisabledFlags[7] == true &&
					DisabledFlags[8] == false)
				{
					Conditions[0] = true;
				}
			}
			else if (Frame == 2 || Frame == 3)
			{
				if (Conditions[0] == true)
				{
					if (DisabledFlags[0] == false &&
						DisabledFlags[1] == true &&
						DisabledFlags[2] == true &&
						DisabledFlags[3] == true &&
						DisabledFlags[4] == true &&
						DisabledFlags[5] == true &&
						DisabledFlags[6] == true &&
						DisabledFlags[7] == false &&
						DisabledFlags[8] == true)
					{
						Conditions[1] = true;

						EXPECT_TRUE(!ClusterMap.Contains(ParticleHandles[8]));
						EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[7], { ParticleHandles[6], ParticleHandles[1] }));
						EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[5], ParticleHandles[2] }));
						EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[4], ParticleHandles[3] }));
					}
				}
			}

		}

		for (int i = 0; i < Conditions.Num(); i++)
		{
			EXPECT_TRUE(Conditions[i]);
		}
	}

	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_SingleLevelBreaking)
	{
		//
		// Test overview:
		// Create two 1cm cubes in a cluster arranged vertically and 20cm apart.
		// Position the cluster above the ground.
		// Wait until the cluster hits the ground.
		// Ensure that the cluster breaks and that the children have the correct states from then on.
		//

		FFramework UnitTest;

		UnitTest.AddSimulationObject(TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>());


		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody(FVector::ZeroVector);
		RestCollection->Transform[2] = FTransform3f(FQuat4f::MakeFromEuler(FVector3f(0., 90.f, 0.)), FVector3f(0, 0, 17));
		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 0.1f };
		Params.ClusterGroupIndex = 0;
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();


		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();

		Collection->PhysObject->SetCollisionParticlesPerObjectFraction(1.0);

		FReal StartingRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size(), CurrentRigidDistance = 0.f;

		UnitTest.Advance();

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();

		EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection->PhysObject->GetSolverClusterHandle_Internal(0),
			{ Collection->PhysObject->GetParticle_Internal(0),Collection->PhysObject->GetParticle_Internal(1) }));

		// Particles array contains the following:		
		// 0: Box1 (top)
		// 1: Box2 (bottom)
		int32 BrokenFrame = INDEX_NONE;

		// 2: Box1+Box2 Cluster
		for (int Frame = 1; Frame < 20; Frame++)
		{
			UnitTest.Advance();

			CurrentRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size();

			if ((BrokenFrame == INDEX_NONE) && !Collection->PhysObject->GetSolverClusterHandle_Internal(2)->Disabled())
			{
				// The two boxes are dropping to the ground as a cluster
				EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(0)->Disabled());
				EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(1)->Disabled());

				// The boxes are still separated by StartingRigidDistance
				EXPECT_LT(FMath::Abs(CurrentRigidDistance - StartingRigidDistance), 1e-4);
			}

			if ((BrokenFrame == INDEX_NONE) && Collection->PhysObject->GetParticle_Internal(2)->Disabled())
			{
				// The cluster has just hit the ground and should have broken.
				EXPECT_FALSE(Collection->PhysObject->GetParticle_Internal(0)->Disabled());
				EXPECT_FALSE(Collection->PhysObject->GetParticle_Internal(1)->Disabled());
				EXPECT_EQ(ClusterMap.Num(), 0);
				BrokenFrame = Frame;
			}

			if ((BrokenFrame != INDEX_NONE) && (Frame > BrokenFrame + 1)) // +1 so that the boxes have a bit of time to move away from each other
			{
				// The boxes are now moving independently - the bottom one is on the ground and should be stopped.
				// The top one is still falling, so they should be closer together	
				EXPECT_GT(FMath::Abs(CurrentRigidDistance - StartingRigidDistance), 1e-4);
			}
		}

		// Make sure it actually broke
		EXPECT_FALSE(Collection->PhysObject->GetParticle_Internal(0)->Disabled());
		EXPECT_FALSE(Collection->PhysObject->GetParticle_Internal(1)->Disabled());
		EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(2)->Disabled());
		EXPECT_TRUE(BrokenFrame != INDEX_NONE);

		EXPECT_GT(FMath::Abs(CurrentRigidDistance - StartingRigidDistance), 1e-4);
	}


	// Wrap two boxes in a cluster (as a sphere), and then wrap that cluster in another cluster (as a sphere).
	// Drop the cluster onto the ground. The outer cluster will break, activating the inner cluster.
	// Then the inner cluster will break, activating the boxes.
	// Note: the inner cluster has a damage threshold of 0, so it breaks even though it is resting on
	// the ground with no velocity after the outer cluster breaks.
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_NestedCluster)
	{
		FFramework UnitTest;

		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();
		UnitTest.AddSimulationObject(Floor);

		TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, -10, 10)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, 10, 10)), FVector(1.0)));
		EXPECT_EQ(RestCollection->Transform.Num(), 2);

		FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(RestCollection.Get());
		EXPECT_EQ(RestCollection->Transform.Num(), 3);
		RestCollection->Transform[2] = FTransform3f(FQuat4f::MakeFromEuler(FVector3f(90.f, 0, 0.)), FVector3f(0, 0, 40));

		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 3, { 2 }, true);
		EXPECT_EQ(RestCollection->Transform.Num(), 4);
		RestCollection->Transform[3] = FTransform3f(FQuat4f::MakeFromEuler(FVector3f(0.f, 0, 0.)), FVector4f(0, 0, 10));

		//GeometryCollectionAlgo::PrintParentHierarchy(RestCollection.Get());

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 0.1f, 0.0f };
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();
		
		FReal StartingRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size(), CurrentRigidDistance = 0.f;

		UnitTest.Advance();

		// Particle Handles array contains
		// [0]: GeometryCollection Sphere0 at 0,10,60
		// [1]: GeometryCollection Sphere1 at 0,10,40
		// [2]: GeometryCollection Cluster0 of Sphere0 and Sphere1 at 0,10,50 (root rotated 90deg about X)
		// [3]: GeometryCollection Cluster1 of Cluster0 at 0,10,50

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();

		EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection->PhysObject->GetParticle_Internal(2), { Collection->PhysObject->GetParticle_Internal(0),Collection->PhysObject->GetParticle_Internal(1) }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection->PhysObject->GetParticle_Internal(3), { Collection->PhysObject->GetParticle_Internal(2) }));
		
		TArray<bool> Conditions = {false,false,false};

		for (int Frame = 1; Frame < 100; Frame++)
		{
			UnitTest.Advance();

			CurrentRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size();

			if (Conditions[0]==false)
			{
				if (
					Collection->PhysObject->GetParticle_Internal(0)->Disabled() == true &&
					Collection->PhysObject->GetParticle_Internal(1)->Disabled() == true &&
					Collection->PhysObject->GetParticle_Internal(2)->Disabled() == true &&
					Collection->PhysObject->GetParticle_Internal(3)->Disabled() == false) 
				{
					// Only the outer Cluster1 is active. 
					// This is the initial condition
					Conditions[0] = true;
				}
			}
			else if (Conditions[0]==true && Conditions[1] == false)
			{
				if (
					Collection->PhysObject->GetParticle_Internal(0)->Disabled() == true &&
					Collection->PhysObject->GetParticle_Internal(1)->Disabled() == true &&
					Collection->PhysObject->GetParticle_Internal(2)->Disabled() == false &&
					Collection->PhysObject->GetParticle_Internal(3)->Disabled() == true)
				{
					// Cluster1 is now disabled, and Cluster0 was activated.
					// This happens when Cluster1 collides with the floor
					Conditions[1] = true;
					EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection->PhysObject->GetParticle_Internal(2), { Collection->PhysObject->GetParticle_Internal(0),Collection->PhysObject->GetParticle_Internal(1) }));
					EXPECT_EQ(ClusterMap.Num(), 1);
					EXPECT_TRUE(!ClusterMap.Contains(Collection->PhysObject->GetParticle_Internal(3)));
				}
			}
			else if (Conditions[1] == true && Conditions[2] == false)
			{
				if (
					Collection->PhysObject->GetParticle_Internal(0)->Disabled() == false &&
					Collection->PhysObject->GetParticle_Internal(1)->Disabled() == false &&
					Collection->PhysObject->GetParticle_Internal(2)->Disabled() == true &&
					Collection->PhysObject->GetParticle_Internal(3)->Disabled() == true)
				{
					// Cluster0 is now disabled because it had a damage threshold of 0
					// and the boxes should now be active.
					Conditions[2] = true;
					EXPECT_EQ(ClusterMap.Num(), 0);
				}
			}
		}
		

		for (int i = 0; i < Conditions.Num(); i++)
		{
			EXPECT_TRUE(Conditions[i]);
		}
		
	}

	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_NestedCluster_NonIdentityMassToLocal)
	{
		// Advance and release each cluster, everything is kinematic, so the output transforms should never change.
		// This tests the transforms in BufferPhysicsResults, validating that MassToLocal, ChildToParent, and X,P
		// will properly map back into the GeometryCollections animation transform hierarchy. 
		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody_TwoParents_TwoBodiesB(FVector(0, 0, 20));
		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { FLT_MAX };
		Params.MaxClusterLevel = 1;
		Params.ClusterGroupIndex = 0;
		FGeometryCollectionWrapper* Collection1 = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();
		TSharedPtr<FGeometryDynamicCollection> DynamicCollection1 = Collection1->DynamicCollection;
		DynamicCollection1->ModifyAttribute<uint8>("DynamicState", FGeometryCollection::TransformGroup)[1] = (uint8)EObjectStateTypeEnum::Chaos_Object_Kinematic;
		DynamicCollection1->ModifyAttribute<uint8>("DynamicState", FGeometryCollection::TransformGroup)[0] = (uint8)EObjectStateTypeEnum::Chaos_Object_Kinematic;

		UnitTest.AddSimulationObject(Collection1);

		UnitTest.Initialize();

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();
		TArray<FTransform> Collection1_InitialTM; GeometryCollectionAlgo::GlobalMatrices(Collection1->RestCollection->Transform, Collection1->RestCollection->Parent, Collection1_InitialTM);
		const auto& SovlerParticleHandles = UnitTest.Solver->GetParticles().GetParticleHandles();

		UnitTest.Solver->RegisterSimOneShotCallback([&]()
		{
			EXPECT_EQ(SovlerParticleHandles.Size(),4);
			EXPECT_EQ(ClusterMap.Num(),2);
			EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection1->PhysObject->GetParticle_Internal(2), { Collection1->PhysObject->GetParticle_Internal(1),Collection1->PhysObject->GetParticle_Internal(0) }));
			EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection1->PhysObject->GetParticle_Internal(3), { Collection1->PhysObject->GetParticle_Internal(2) }));
		});
		

		UnitTest.Advance();

		EXPECT_EQ(SovlerParticleHandles.Size(), 4);
		EXPECT_EQ(ClusterMap.Num(), 2);
		EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection1->PhysObject->GetParticle_Internal(2), { Collection1->PhysObject->GetParticle_Internal(1),Collection1->PhysObject->GetParticle_Internal(0) }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection1->PhysObject->GetParticle_Internal(3), { Collection1->PhysObject->GetParticle_Internal(2) }));
		TArray<FTransform> Collection1_PreReleaseTM; 
		GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection1.Get(), Collection1_PreReleaseTM);
		for (int Idx = 0; Idx < Collection1_PreReleaseTM.Num(); Idx++) {
			EXPECT_TRUE( (Collection1_PreReleaseTM[Idx].GetTranslation()-Collection1_InitialTM[Idx].GetTranslation()).Size()<KINDA_SMALL_NUMBER);
		}

		UnitTest.Solver->GetEvolution()->GetRigidClustering().DeactivateClusterParticle(Collection1->PhysObject->GetParticle_Internal(3));
		UnitTest.Advance();

		EXPECT_EQ(SovlerParticleHandles.Size(), 4);
		EXPECT_EQ(ClusterMap.Num(), 1);
		EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection1->PhysObject->GetParticle_Internal(2), { Collection1->PhysObject->GetParticle_Internal(1),Collection1->PhysObject->GetParticle_Internal(0) }));
		TArray<FTransform> Collection1_PostReleaseTM; 
		GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection1.Get(), Collection1_PostReleaseTM);
		for (int Idx = 0; Idx < Collection1_PostReleaseTM.Num(); Idx++) {
			EXPECT_TRUE((Collection1_PostReleaseTM[Idx].GetTranslation() - Collection1_InitialTM[Idx].GetTranslation()).Size() < KINDA_SMALL_NUMBER);
		}

		UnitTest.Solver->GetEvolution()->GetRigidClustering().DeactivateClusterParticle(Collection1->PhysObject->GetParticle_Internal(2));
		UnitTest.Advance();

		EXPECT_EQ(SovlerParticleHandles.Size(), 4);
		EXPECT_EQ(ClusterMap.Num(), 0);
		TArray<FTransform> Collection1_PostRelease2TM; GeometryCollectionAlgo::Private::GlobalMatrices( *DynamicCollection1, Collection1_PostRelease2TM);
		for (int Idx = 0; Idx < Collection1_PostRelease2TM.Num(); Idx++) {
			EXPECT_TRUE((Collection1_PostRelease2TM[Idx].GetTranslation() - Collection1_InitialTM[Idx].GetTranslation()).Size() < KINDA_SMALL_NUMBER);
		}

	}


	
	GTEST_TEST(AllTraits, DISASBLED_GeometryCollection_RigidBodies_ClusterTest_NestedCluster_MultiStrain)
	{
		FFramework UnitTest;
		
		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();
		UnitTest.AddSimulationObject(Floor);

		TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(20.f)),FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(30.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(40.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(50.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(60.f)), FVector(1.0)));

		FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(RestCollection.Get()); 
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 4, { 2, 3 }, true, true);
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 2, { 1, 0 }, true, true);
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 1, { 3, 0 }, true, true);
		

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 30.0, 30.0, 30, FLT_MAX };

		// basically a stand-in for a 'component'
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();

		FReal StartingRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size(), CurrentRigidDistance = 0.f;

		TArray<bool> Conditions = { false,false,false,false };

		UnitTest.Advance();

		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ParticleHandles = { 
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			Collection->PhysObject->GetParticle_Internal(4),
			Collection->PhysObject->GetParticle_Internal(5),
			Collection->PhysObject->GetParticle_Internal(6),
			Collection->PhysObject->GetParticle_Internal(7),
			Collection->PhysObject->GetParticle_Internal(8),
		};

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();

		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[6],ParticleHandles[4] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[7],ParticleHandles[2] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[7], { ParticleHandles[8],ParticleHandles[1] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[8], { ParticleHandles[3],ParticleHandles[0] }));


		for (int Frame = 1; Frame < 40; Frame++)
		{
			UnitTest.Advance();			

			CurrentRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size();
			
			if (Conditions[0] == false)
			{
				if (ParticleHandles[0]->Disabled() == true &&
					ParticleHandles[1]->Disabled() == true &&
					ParticleHandles[2]->Disabled() == true &&
					ParticleHandles[3]->Disabled() == true &&
					ParticleHandles[4]->Disabled() == true &&
					ParticleHandles[5]->Disabled() == false && // root
					ParticleHandles[6]->Disabled() == true &&
					ParticleHandles[7]->Disabled() == true &&
					ParticleHandles[8]->Disabled() == true)
				{
					Conditions[0] = true;
				}
			}
			// Root cluster broken, check for activated children
			else if (Conditions[0] == true && Conditions[1] == false)
			{
				if (
					ParticleHandles[0]->Disabled() == true &&
					ParticleHandles[1]->Disabled() == true &&
					ParticleHandles[2]->Disabled() == true &&
					ParticleHandles[3]->Disabled() == true &&
					ParticleHandles[4]->Disabled() == false &&
					ParticleHandles[5]->Disabled() == true && // root, now disabled
					ParticleHandles[6]->Disabled() == false &&
					ParticleHandles[7]->Disabled() == true &&
					ParticleHandles[8]->Disabled() == true)
				{
					Conditions[1] = true;

					EXPECT_EQ(ClusterMap.Num(), 3);
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[8], { ParticleHandles[3],ParticleHandles[0] }));
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[7], { ParticleHandles[1],ParticleHandles[8] }));
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[7],ParticleHandles[2] }));
				}
			}
			else if (Conditions[1] == true && Conditions[2] == false)
			{
				if (
					ParticleHandles[0]->Disabled() == true &&
					ParticleHandles[1]->Disabled() == true &&
					ParticleHandles[2]->Disabled() == false &&
					ParticleHandles[3]->Disabled() == true &&
					ParticleHandles[4]->Disabled() == false &&
					ParticleHandles[5]->Disabled() == true &&
					ParticleHandles[6]->Disabled() == true &&
					ParticleHandles[7]->Disabled() == false &&
					ParticleHandles[8]->Disabled() == true)
				{
					Conditions[2] = true;

					EXPECT_EQ(ClusterMap.Num(), 2);
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[8], { ParticleHandles[3],ParticleHandles[0] }));
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[7], { ParticleHandles[1],ParticleHandles[8] }));
				}
			}
			else if (Conditions[2] == true && Conditions[3] == false)
			{
				if (
					ParticleHandles[0]->Disabled() == true &&
					ParticleHandles[1]->Disabled() == false &&
					ParticleHandles[2]->Disabled() == false &&
					ParticleHandles[3]->Disabled() == true &&
					ParticleHandles[4]->Disabled() == false &&
					ParticleHandles[5]->Disabled() == true &&
					ParticleHandles[6]->Disabled() == true &&
					ParticleHandles[7]->Disabled() == true &&
					ParticleHandles[8]->Disabled() == false)
				{
					Conditions[3] = true;

					EXPECT_EQ(ClusterMap.Num(), 1);
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[8], { ParticleHandles[3],ParticleHandles[0] }));
				}
			}
			else if (Conditions[3] == true)
			{
				// fLT_MAX strain so last cluster should never break. 
				EXPECT_TRUE(ParticleHandles[0]->Disabled() == true);
				EXPECT_TRUE(ParticleHandles[1]->Disabled() == false);
				EXPECT_TRUE(ParticleHandles[2]->Disabled() == false);
				EXPECT_TRUE(ParticleHandles[3]->Disabled() == true);
				EXPECT_TRUE(ParticleHandles[4]->Disabled() == false);
				EXPECT_TRUE(ParticleHandles[5]->Disabled() == true);
				EXPECT_TRUE(ParticleHandles[6]->Disabled() == true);
				EXPECT_TRUE(ParticleHandles[7]->Disabled() == true);
				EXPECT_TRUE(ParticleHandles[8]->Disabled() == false);
				EXPECT_EQ(ClusterMap.Num(), 1);
				EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[8], { ParticleHandles[3],ParticleHandles[0] }));
			}
		}
		for (int i = 0; i < Conditions.Num(); i++)
		{
			//EXPECT_TRUE(Conditions[i]);
		}

	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_KinematicAnchor)
	{
		// Test : Set one element kinematic. When the cluster breaks the elements that do not contain the kinematic
		//        rigid body should be dynamic, while the clusters that contain the kinematic body should remain 
		//        kinematic.

		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(20.f)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(30.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(40.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(50.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(60.f)), FVector(1.0)));

		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 5, { 4,3 }, true, false); // just validate at end of construction
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 6, { 5,2 }, true, false);
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 7, { 6,1 }, true, false);
		FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(RestCollection.Get());
		FGeometryCollectionClusteringUtility::ValidateResults(RestCollection.Get());

		//GeometryCollectionAlgo::PrintParentHierarchy(RestCollection.Get());

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 50.0, 50.0, 50.0, FLT_MAX };
		Params.MaxClusterLevel = 1;
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		Collection->DynamicCollection->template ModifyAttribute<uint8>("DynamicState", FGeometryCollection::TransformGroup)[1] = (uint8)EObjectStateTypeEnum::Chaos_Object_Kinematic;

		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();

		FReal StartingRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size();
		FReal CurrentRigidDistance = 0;

		// Staged conditions
		// Initial state should set up the hierarchy correctly, leaving correct disabled flags on frame 1
		bool bValidInitialState = false;
		// After releasing particle 8, the states should be updated on frame 2
		bool bParticle8SucessfulRelease = false;
		// After releasing particle 8, the states should be updated on frame 4
		bool bParticle7SucessfulRelease = false;
		// After simulating post-release the states should match frame 4
		bool bValidFinalActiveState = false;

		// Tick once to fush commands
		UnitTest.Advance();

		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			Collection->PhysObject->GetParticle_Internal(4),
			Collection->PhysObject->GetParticle_Internal(5),
			Collection->PhysObject->GetParticle_Internal(6),
			Collection->PhysObject->GetParticle_Internal(7),
			Collection->PhysObject->GetParticle_Internal(8),
		};
		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ClusterHandles = {
			Collection->PhysObject->GetSolverClusterHandle_Internal(0),
			Collection->PhysObject->GetSolverClusterHandle_Internal(1),
			Collection->PhysObject->GetSolverClusterHandle_Internal(2),
			Collection->PhysObject->GetSolverClusterHandle_Internal(3),
			Collection->PhysObject->GetSolverClusterHandle_Internal(4),
			Collection->PhysObject->GetSolverClusterHandle_Internal(5),
			Collection->PhysObject->GetSolverClusterHandle_Internal(6),
			Collection->PhysObject->GetSolverClusterHandle_Internal(7),
			Collection->PhysObject->GetSolverClusterHandle_Internal(8),
		};

		FRigidClustering& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const FRigidClustering::FClusterMap& ClusterMap = Clustering.GetChildrenMap();

		// Verify that the parent-child relationship is reflected in the clustering hierarchy
		// Tree should be:
		//
		//          8
		//         / \
		//        7   0
		//       / \
		//      6   1
		//     / \
		//    5   2
		//   / \
		//  4   3
		//
		// Entire cluster is kinematic due to particle 1
		//
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[4],ParticleHandles[3] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[5],ParticleHandles[2] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[7], { ParticleHandles[6],ParticleHandles[1] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[8], { ParticleHandles[7],ParticleHandles[0] }));

		// Storage for positions for particles 0, 1, 6 for testing assumptions
		FVector Ref0(0.0);
		FVector Ref1(0.0);
		FVector Ref6(0.0);

		for (int Frame = 1; Frame < 10; Frame++)
		{
			UnitTest.Advance();
			
			// On frames 2 and 4, deactivate particles 8 and 7, releasing their children (7,0 then 6,1)
			if (Frame == 2)
			{
				UnitTest.Solver->GetEvolution()->GetRigidClustering().DeactivateClusterParticle(ParticleHandles[8]);
			}
			if (Frame == 4)
			{
				UnitTest.Solver->GetEvolution()->GetRigidClustering().DeactivateClusterParticle(ParticleHandles[7]);
			}

			// Verify that the kinematic particle remains kinematic (InvMass == 0.0)
			// and the the dynamic particles have a non-zero inv mass
			EXPECT_NE(ParticleHandles[0]->InvM(), 0); // dynamic rigid
			EXPECT_EQ(ParticleHandles[1]->InvM(), 0); // kinematic rigid
			EXPECT_NE(ParticleHandles[2]->InvM(), 0); // dynamic rigid
			EXPECT_NE(ParticleHandles[3]->InvM(), 0); // dynamic rigid
			EXPECT_NE(ParticleHandles[4]->InvM(), 0); // dynamic rigid
			EXPECT_NE(ParticleHandles[5]->InvM(), 0); // dynamic rigid
			EXPECT_NE(ParticleHandles[6]->InvM(), 0); // dynamic cluster



			if (!bValidInitialState && Frame == 1)
			{
				if (ParticleHandles[0]->Disabled() == true &&
					ParticleHandles[1]->Disabled() == true &&
					ParticleHandles[2]->Disabled() == true &&
					ParticleHandles[3]->Disabled() == true &&
					ParticleHandles[4]->Disabled() == true &&
					ParticleHandles[5]->Disabled() == true &&
					ParticleHandles[6]->Disabled() == true &&
					ParticleHandles[7]->Disabled() == true &&
					ParticleHandles[8]->Disabled() == false)
				{
					bValidInitialState = true;
					Ref0 = ParticleHandles[0]->GetX();
					Ref1 = ParticleHandles[1]->GetX();
					Ref6 = ParticleHandles[6]->GetX();

					// Test kinematic particles have valid (0.0) inverse mass and have the kinematic object state set
					EXPECT_EQ(ParticleHandles[7]->InvM(), 0.f); // kinematic cluster
					EXPECT_EQ(ParticleHandles[7]->ObjectState(), Chaos::EObjectStateType::Kinematic); // kinematic cluster
					EXPECT_EQ(ParticleHandles[8]->InvM(), 0.f); // kinematic cluster
					EXPECT_EQ(ParticleHandles[8]->ObjectState(), Chaos::EObjectStateType::Kinematic); // kinematic cluster
				}
			}
			else if (bValidInitialState && !bParticle8SucessfulRelease && Frame == 2)
			{
				if (ParticleHandles[0]->Disabled() == false &&
					ParticleHandles[1]->Disabled() == true &&
					ParticleHandles[2]->Disabled() == true &&
					ParticleHandles[3]->Disabled() == true &&
					ParticleHandles[4]->Disabled() == true &&
					ParticleHandles[5]->Disabled() == true &&
					ParticleHandles[6]->Disabled() == true &&
					ParticleHandles[7]->Disabled() == false &&
					ParticleHandles[8]->Disabled() == true)
				{
					bParticle8SucessfulRelease = true;
					FVector X0 = ParticleHandles[0]->GetX();
					FVector X1 = ParticleHandles[1]->GetX();
					FVector X6 = ParticleHandles[6]->GetX();

					FVector X00 = Ref0;
					FVector X11 = Ref1;
					FVector X66 = Ref6;


					check(!X0.ContainsNaN());
					check(!Ref0.ContainsNaN());
					check(FMath::IsFinite(X0.Size()));
					check(FMath::IsFinite(Ref0.Size()));
					check(FMath::IsFinite(X0.Size() - Ref0.Size()));
					check(FMath::IsFinite(FMath::Abs(X0.Size() - X00.Size())));
					EXPECT_NEAR(FMath::Abs(X0.Size() - Ref0.Size()), 0, KINDA_SMALL_NUMBER);// << *FString("Kinematic body1 moved");
					check(FMath::IsFinite(FMath::Abs(X1.Size() - X11.Size())));
					EXPECT_NEAR(FMath::Abs(X1.Size() - Ref1.Size()), 0, KINDA_SMALL_NUMBER);// << *FString("Kinematic body2 moved");
					check(FMath::IsFinite(FMath::Abs(X6.Size() - X66.Size())));
					EXPECT_NEAR(FMath::Abs(X6.Size() - Ref6.Size()), 0, KINDA_SMALL_NUMBER);// << *FString("Kinematic body7 moved");

					// Test kinematic particles have valid (0.0) inverse mass and have the kinematic object state set
					EXPECT_EQ(ParticleHandles[7]->InvM(), 0.f); // kinematic cluster
					EXPECT_EQ(ParticleHandles[7]->ObjectState(), Chaos::EObjectStateType::Kinematic); // kinematic cluster
					EXPECT_EQ(ParticleHandles[8]->InvM(), 0.f); // kinematic cluster
					EXPECT_EQ(ParticleHandles[8]->ObjectState(), Chaos::EObjectStateType::Kinematic); // kinematic cluster

					// Test that after declustering the new cluster hierarchy is what we expect
					// Tree should be:
					//
					//        7      Removed:   8 (Disabled)
					//       / \                 \
					//      6   1                 0 (Now unclustered)
					//     / \
					//    5   2
					//   / \
					//  4   3
					//
					// 8 has been removed, zero is dynamic and the remaining tree is kinematic due to particle 1
					//
					EXPECT_EQ(ClusterMap.Num(), 3);
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[7], { ParticleHandles[6],ParticleHandles[1] }));
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[5],ParticleHandles[2] }));
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[4],ParticleHandles[3] }));					
				}
			}
			else if (bParticle8SucessfulRelease && !bParticle7SucessfulRelease && Frame == 4)
			{
				if (ParticleHandles[0]->Disabled() == false &&
					ParticleHandles[1]->Disabled() == false &&
					ParticleHandles[2]->Disabled() == true &&
					ParticleHandles[3]->Disabled() == true &&
					ParticleHandles[4]->Disabled() == true &&
					ParticleHandles[5]->Disabled() == true &&
					ParticleHandles[6]->Disabled() == false &&
					ParticleHandles[7]->Disabled() == true &&
					ParticleHandles[8]->Disabled() == true)
				{
					bParticle7SucessfulRelease = true;
					FVector X0 = ParticleHandles[0]->GetX();
					FVector X1 = ParticleHandles[1]->GetX();
					FVector X6 = ParticleHandles[6]->GetX();

					// 0 is a dynamic unclustered body (was owned by cluster 8), check that it's moved since declustering
					EXPECT_GT(FMath::Abs(X0.Size() - Ref0.Size()), KINDA_SMALL_NUMBER);
					// 1 is a kinematic unclustered body (was owned by cluster 7), check that it's stayed in place
					EXPECT_NEAR(FMath::Abs(X1.Size() - Ref1.Size()), 0, KINDA_SMALL_NUMBER);
					// 6 is a dynamic cluster (was owned by cluster 7). Now that 1 is not a part of the cluster
					// however it's just been declustered so make sure it's still near the starting location
					EXPECT_NEAR(FMath::Abs(X6.Size() - Ref6.Size()), 0, KINDA_SMALL_NUMBER);

					// Check the newly disabled 7 is still kinematic
					EXPECT_EQ(ParticleHandles[7]->InvM(), 0.f); // kinematic cluster
					EXPECT_EQ(ParticleHandles[7]->ObjectState(), Chaos::EObjectStateType::Kinematic); // kinematic cluster

					// Test that after declustering the new cluster hierarchy is what we expect
					// Tree should be:
					//
					//      6    Removed:  7 (disabled)
					//     / \              \
					//    5   2              1 (declustered, but kinematic)
					//   / \
					//  4   3
					//
					// 7 has been removed, 1 is kinematic and the rest of the tree is dynamic as the kinematic element is
					// no longer in the cluster
					//
					EXPECT_EQ(ClusterMap.Num(), 2);
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[5],ParticleHandles[2] }));
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[4],ParticleHandles[3] }));
				}
			}
			else if (bParticle7SucessfulRelease && !bValidFinalActiveState && Frame == 6)
			{
				if (ParticleHandles[0]->Disabled() == false &&
					ParticleHandles[1]->Disabled() == false &&
					ParticleHandles[2]->Disabled() == true &&
					ParticleHandles[3]->Disabled() == true &&
					ParticleHandles[4]->Disabled() == true &&
					ParticleHandles[5]->Disabled() == true &&
					ParticleHandles[6]->Disabled() == false &&
					ParticleHandles[7]->Disabled() == true &&
					ParticleHandles[8]->Disabled() == true)
				{
					bValidFinalActiveState = true;
					FVector X0 = ParticleHandles[0]->GetX();
					FVector X1 = ParticleHandles[1]->GetX();
					FVector X6 = ParticleHandles[6]->GetX();

					// 0 is a dynamic unclustered body (was owned by cluster 8), check that it's moved since declustering
					EXPECT_GT(FMath::Abs(X0.Size() - Ref0.Size()), KINDA_SMALL_NUMBER);
					// 1 is a kinematic unclustered body (was owned by cluster 7), check that it's stayed in place
					EXPECT_NEAR(FMath::Abs(X1.Size() - Ref1.Size()), 0, KINDA_SMALL_NUMBER);
					// 6 is a dynamic cluster (was owned by cluster 7). Now that 1 is not a part of the cluster
					// it is dynamic, check that it has moved since declustering
					EXPECT_GT(FMath::Abs(X6.Size() - Ref6.Size()), KINDA_SMALL_NUMBER);

					// Check the previously declustered 7 is still kinematic
					EXPECT_EQ(ParticleHandles[7]->InvM(), 0.f); // kinematic cluster
					EXPECT_EQ(ParticleHandles[7]->ObjectState(), Chaos::EObjectStateType::Kinematic); // kinematic cluster

					// Test that the tree is still the same after the final decluster operation.
					// Tree should be:
					//
					//      6
					//     / \
					//    5   2
					//   / \
					//  4   3
					//
					EXPECT_EQ(ClusterMap.Num(), 2);
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[5],ParticleHandles[2] }));
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[4],ParticleHandles[3] }));
				}
			}
		}

		// Test our staged conditions

		// Initial state should set up the heirachy correctly, leaving correct disabled flags on frame 1
		EXPECT_TRUE(bValidInitialState);
		// After releasing particle 8, the states should be updated on frame 2
		EXPECT_TRUE(bParticle8SucessfulRelease);
		// After releasing particle 8, the states should be updated on frame 4
		EXPECT_TRUE(bParticle7SucessfulRelease);
		// After simulating post-release the states should match frame 4
		EXPECT_TRUE(bValidFinalActiveState);
	}

	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_StaticAnchor)
	{
		// Test : Set one element static. When the cluster breaks the elements that do not contain the static
		//        rigid body should be dynamic, while the clusters that contain the static body should remain 
		//        static. 

		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(20.f)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(30.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(40.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(50.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(60.f)), FVector(1.0)));

		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 5, { 4,3 }, true, false); // just validate at end of construction
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 6, { 5,2 }, true, false);
		FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(RestCollection.Get(), 7, { 6,1 }, true, false);
		FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(RestCollection.Get());
		FGeometryCollectionClusteringUtility::ValidateResults(RestCollection.Get());

		//GeometryCollectionAlgo::PrintParentHierarchy(RestCollection.Get());

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 50.0, 50.0, 50.0, FLT_MAX };
		Params.MaxClusterLevel = 1;
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		Collection->DynamicCollection->template ModifyAttribute<uint8>("DynamicState", FGeometryCollection::TransformGroup)[1] = (uint8)EObjectStateTypeEnum::Chaos_Object_Static;

		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();

		FReal StartingRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size();
		FReal CurrentRigidDistance = 0.f;

		// Staged conditions
		// Initial state should set up the hierachy correctly, leaving correct disabled flags on frame 1
		bool bValidInitialState = false;
		// After releasing particle 8, the states should be updated on frame 2
		bool bParticle8SucessfulRelease = false;
		// After releasing particle 8, the states should be updated on frame 4
		bool bParticle7SucessfulRelease = false;
		// After simulating post-release the states should match frame 4
		bool bValidFinalActiveState = false;

		// Tick once to fush commands
		UnitTest.Advance();

		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			Collection->PhysObject->GetParticle_Internal(4),
			Collection->PhysObject->GetParticle_Internal(5),
			Collection->PhysObject->GetParticle_Internal(6),
			Collection->PhysObject->GetParticle_Internal(7),
			Collection->PhysObject->GetParticle_Internal(8),
		};
		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ClusterHandles = {
			Collection->PhysObject->GetSolverClusterHandle_Internal(0),
			Collection->PhysObject->GetSolverClusterHandle_Internal(1),
			Collection->PhysObject->GetSolverClusterHandle_Internal(2),
			Collection->PhysObject->GetSolverClusterHandle_Internal(3),
			Collection->PhysObject->GetSolverClusterHandle_Internal(4),
			Collection->PhysObject->GetSolverClusterHandle_Internal(5),
			Collection->PhysObject->GetSolverClusterHandle_Internal(6),
			Collection->PhysObject->GetSolverClusterHandle_Internal(7),
			Collection->PhysObject->GetSolverClusterHandle_Internal(8),
		};

		FRigidClustering& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const FRigidClustering::FClusterMap& ClusterMap = Clustering.GetChildrenMap();

		// Verify that the parent-child relationship is reflected in the clustering hierarchy
		// Tree should be:
		//
		//          8
		//         / \
		//        7   0
		//       / \
		//      6   1
		//     / \
		//    5   2
		//   / \
		//  4   3
		//
		// Entire cluster is kinematic due to particle 1
		//
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[4],ParticleHandles[3] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[5],ParticleHandles[2] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[7], { ParticleHandles[6],ParticleHandles[1] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[8], { ParticleHandles[7],ParticleHandles[0] }));

		// Storage for positions for particles 0, 1, 6 for testing assumptions
		FVector Ref0;
		FVector Ref1;
		FVector Ref6;

		for(int Frame = 1; Frame < 10; Frame++)
		{
			UnitTest.Advance();

			// On frames 2 and 4, deactivate particles 8 and 7, releasing their children (7,0 then 6,1)
			if(Frame == 2)
			{
				UnitTest.Solver->GetEvolution()->GetRigidClustering().DeactivateClusterParticle(ParticleHandles[8]);
			}
			if(Frame == 4)
			{
				UnitTest.Solver->GetEvolution()->GetRigidClustering().DeactivateClusterParticle(ParticleHandles[7]);
			}

			// Verify that the kinematic particle remains kinematic (InvMass == 0.0)
			// and the the dynamic particles have a non-zero inv mass
			EXPECT_NE(ParticleHandles[0]->InvM(), 0.f); // dynamic rigid
			EXPECT_EQ(ParticleHandles[1]->InvM(), 0.f); // kinematic rigid
			EXPECT_NE(ParticleHandles[2]->InvM(), 0.f); // dynamic rigid
			EXPECT_NE(ParticleHandles[3]->InvM(), 0.f); // dynamic rigid
			EXPECT_NE(ParticleHandles[4]->InvM(), 0.f); // dynamic rigid
			EXPECT_NE(ParticleHandles[5]->InvM(), 0.f); // dynamic rigid
			EXPECT_NE(ParticleHandles[6]->InvM(), 0.f); // dynamic cluster

			if(!bValidInitialState && Frame == 1)
			{
				if(ParticleHandles[0]->Disabled() == true &&
					ParticleHandles[1]->Disabled() == true &&
					ParticleHandles[2]->Disabled() == true &&
					ParticleHandles[3]->Disabled() == true &&
					ParticleHandles[4]->Disabled() == true &&
					ParticleHandles[5]->Disabled() == true &&
					ParticleHandles[6]->Disabled() == true &&
					ParticleHandles[7]->Disabled() == true &&
					ParticleHandles[8]->Disabled() == false)
				{
					bValidInitialState = true;
					Ref0 = ParticleHandles[0]->GetX();
					Ref1 = ParticleHandles[1]->GetX();
					Ref6 = ParticleHandles[6]->GetX();

					// Test static particles have valid (0.0) inverse mass and have the static object state set
					EXPECT_EQ(ParticleHandles[7]->InvM(), 0.f); // kinematic cluster
					EXPECT_EQ(ParticleHandles[7]->ObjectState(), Chaos::EObjectStateType::Static); // Static cluster
					EXPECT_EQ(ParticleHandles[8]->InvM(), 0.f); // kinematic cluster
					EXPECT_EQ(ParticleHandles[8]->ObjectState(), Chaos::EObjectStateType::Static); // Static cluster
				}
			}
			else if(bValidInitialState && !bParticle8SucessfulRelease && Frame == 2)
			{
				if(ParticleHandles[0]->Disabled() == false &&
					ParticleHandles[1]->Disabled() == true &&
					ParticleHandles[2]->Disabled() == true &&
					ParticleHandles[3]->Disabled() == true &&
					ParticleHandles[4]->Disabled() == true &&
					ParticleHandles[5]->Disabled() == true &&
					ParticleHandles[6]->Disabled() == true &&
					ParticleHandles[7]->Disabled() == false &&
					ParticleHandles[8]->Disabled() == true)
				{
					bParticle8SucessfulRelease = true;
					FVector X0 = ParticleHandles[0]->GetX();
					FVector X1 = ParticleHandles[1]->GetX();
					FVector X6 = ParticleHandles[6]->GetX();

					EXPECT_NEAR(FMath::Abs(X0.Size() - Ref0.Size()), 0, KINDA_SMALL_NUMBER);
					EXPECT_NEAR(FMath::Abs(X1.Size() - Ref1.Size()), 0, KINDA_SMALL_NUMBER);
					EXPECT_NEAR(FMath::Abs(X6.Size() - Ref6.Size()), 0, KINDA_SMALL_NUMBER);

					// Test static particles have valid (0.0) inverse mass and have the static object state set
					EXPECT_EQ(ParticleHandles[7]->InvM(), 0.f); // kinematic cluster
					EXPECT_EQ(ParticleHandles[7]->ObjectState(), Chaos::EObjectStateType::Static); // Static cluster
					EXPECT_EQ(ParticleHandles[8]->InvM(), 0.f); // kinematic cluster
					EXPECT_EQ(ParticleHandles[8]->ObjectState(), Chaos::EObjectStateType::Static); // Static cluster

					// Test that after declustering the new cluster hierarchy is what we expect
					// Tree should be:
					//
					//        7      Removed:   8 (Disabled)
					//       / \                 \
					//      6   1                 0 (Now unclustered)
					//     / \
					//    5   2
					//   / \
					//  4   3
					//
					// 8 has been removed, zero is dynamic and the remaining tree is static due to particle 1
					//
					EXPECT_EQ(ClusterMap.Num(), 3);
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[7], { ParticleHandles[6],ParticleHandles[1] }));
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[5],ParticleHandles[2] }));
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[4],ParticleHandles[3] }));
				}
			}
			else if(bParticle8SucessfulRelease && !bParticle7SucessfulRelease && Frame == 4)
			{
				if(ParticleHandles[0]->Disabled() == false &&
					ParticleHandles[1]->Disabled() == false &&
					ParticleHandles[2]->Disabled() == true &&
					ParticleHandles[3]->Disabled() == true &&
					ParticleHandles[4]->Disabled() == true &&
					ParticleHandles[5]->Disabled() == true &&
					ParticleHandles[6]->Disabled() == false &&
					ParticleHandles[7]->Disabled() == true &&
					ParticleHandles[8]->Disabled() == true)
				{
					bParticle7SucessfulRelease = true;
					FVector X0 = ParticleHandles[0]->GetX();
					FVector X1 = ParticleHandles[1]->GetX();
					FVector X6 = ParticleHandles[6]->GetX();

					// 0 is a dynamic unclustered body (was owned by cluster 8), check that it's moved since declustering
					EXPECT_GT(FMath::Abs(X0.Size() - Ref0.Size()), KINDA_SMALL_NUMBER);
					// 1 is a static unclustered body (was owned by cluster 7), check that it's stayed in place
					EXPECT_NEAR(FMath::Abs(X1.Size() - Ref1.Size()), 0, KINDA_SMALL_NUMBER);
					// 6 is a dynamic cluster (was owned by cluster 7) but it has just been declustered
					// Test that it's still near the starting position
					EXPECT_NEAR(FMath::Abs(X6.Size() - Ref6.Size()), 0, KINDA_SMALL_NUMBER);

					// Check the newly disabled 7 is still static
					EXPECT_EQ(ParticleHandles[7]->InvM(), 0.f); // Static cluster
					EXPECT_EQ(ParticleHandles[7]->ObjectState(), Chaos::EObjectStateType::Static); // Static cluster


					// Test that after declustering the new cluster hierarchy is what we expect
					// Tree should be:
					//
					//      6    Removed:  7 (disabled)
					//     / \              \
					//    5   2              1 (declustered, but Static)
					//   / \
					//  4   3
					//
					// 7 has been removed, 1 is static and the rest of the tree is dynamic as the static element is
					// no longer in the cluster
					//
					EXPECT_EQ(ClusterMap.Num(), 2);
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[5],ParticleHandles[2] }));
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[4],ParticleHandles[3] }));
				}
			}
			else if(bParticle7SucessfulRelease && !bValidFinalActiveState && Frame == 6)
			{
				if(ParticleHandles[0]->Disabled() == false &&
					ParticleHandles[1]->Disabled() == false &&
					ParticleHandles[2]->Disabled() == true &&
					ParticleHandles[3]->Disabled() == true &&
					ParticleHandles[4]->Disabled() == true &&
					ParticleHandles[5]->Disabled() == true &&
					ParticleHandles[6]->Disabled() == false &&
					ParticleHandles[7]->Disabled() == true &&
					ParticleHandles[8]->Disabled() == true)
				{
					bValidFinalActiveState = true;
					FVector X0 = ParticleHandles[0]->GetX();
					FVector X1 = ParticleHandles[1]->GetX();
					FVector X6 = ParticleHandles[6]->GetX();

					// 0 is a dynamic unclustered body (was owned by cluster 8), check that it's moved since declustering
					EXPECT_GT(FMath::Abs(X0.Size() - Ref0.Size()), KINDA_SMALL_NUMBER);
					// 1 is a static unclustered body (was owned by cluster 7), check that it's stayed in place
					EXPECT_NEAR(FMath::Abs(X1.Size() - Ref1.Size()), 0, KINDA_SMALL_NUMBER);
					// 6 is a dynamic cluster (was owned by cluster 7). Now that 1 is not a part of the cluster
					// it is dynamic, check that it has moved since declustering
					EXPECT_GT(FMath::Abs(X6.Size() - Ref6.Size()), KINDA_SMALL_NUMBER);

					// Check the previously declustered 7 is still static
					EXPECT_EQ(ParticleHandles[7]->InvM(), 0.f); // Static cluster
					EXPECT_EQ(ParticleHandles[7]->ObjectState(), Chaos::EObjectStateType::Static); // Static cluster

					// Test that the tree is still the same after the final decluster operation.
					// Tree should be:
					//
					//      6
					//     / \
					//    5   2
					//   / \
					//  4   3
					//
					EXPECT_EQ(ClusterMap.Num(), 2);
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[6], { ParticleHandles[5],ParticleHandles[2] }));
					EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[5], { ParticleHandles[4],ParticleHandles[3] }));
				}
			}
		}

		// Test our staged conditions

		// Initial state should set up the heirachy correctly, leaving correct disabled flags on frame 1
		EXPECT_TRUE(bValidInitialState);
		// After releasing particle 8, the states should be updated on frame 2
		EXPECT_TRUE(bParticle8SucessfulRelease);
		// After releasing particle 8, the states should be updated on frame 4
		EXPECT_TRUE(bParticle7SucessfulRelease);
		// After simulating post-release the states should match frame 4
		EXPECT_TRUE(bValidFinalActiveState);
	}

	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_UnionClusters)
	{
		// Test : Joining collections using the ClusterGroupIndex by a particle dynamically created within the solver.

		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody(FVector(-2, 0, 3));
		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { FLT_MAX };
		Params.ClusterGroupIndex = 1;
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();


		TSharedPtr<FGeometryCollection> RestCollection2 = CreateClusteredBody(FVector(2, 0, 3));
		CreationParameters Params2;
		Params2.RestCollection = RestCollection2;
		Params2.DynamicState = EObjectStateTypeEnum::Chaos_Object_Kinematic;
		Params2.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params2.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params2.Simulating = true;
		Params2.EnableClustering = true;
		Params2.DamageThreshold = { FLT_MAX };
		Params2.ClusterGroupIndex = 1;
		FGeometryCollectionWrapper* Collection2 = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params2)->template As<FGeometryCollectionWrapper>();
		
		UnitTest.AddSimulationObject(Collection);
		UnitTest.AddSimulationObject(Collection2);
		UnitTest.Initialize();

		TSharedPtr<FGeometryDynamicCollection> DynamicCollection = Collection->DynamicCollection;
		TSharedPtr<FGeometryDynamicCollection> DynamicCollection2 = Collection2->DynamicCollection;


		TArray<FReal> Distances;
		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();
		const auto& ParticleHandles = UnitTest.Solver->GetParticles().GetParticleHandles();

		UnitTest.Solver->RegisterSimOneShotCallback([&]()
		{
			EXPECT_EQ(ClusterMap.Num(),2);
			EXPECT_TRUE(ClusterMapContains(ClusterMap,ParticleHandles.Handle(2)->CastToRigidParticle(),{ParticleHandles.Handle(1)->CastToRigidParticle(),ParticleHandles.Handle(0)->CastToRigidParticle()}));
			EXPECT_TRUE(ClusterMapContains(ClusterMap,ParticleHandles.Handle(5)->CastToRigidParticle(),{ParticleHandles.Handle(4)->CastToRigidParticle(),ParticleHandles.Handle(3)->CastToRigidParticle()}));
		});

		for (int Frame = 0; Frame < 100; Frame++)
		{
			UnitTest.Advance();


			if (Frame == 0)
			{
				TArray<FTransform> GlobalTransform;
				GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection, GlobalTransform);

				TArray<FTransform> GlobalTransform2;
				GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection2, GlobalTransform2);

				// build relative transforms distances
				for (int32 i = 0; i < (int32)GlobalTransform.Num()-1; i++)
				{
					for (int j = 0; j < (int32)GlobalTransform2.Num()-1; j++)
					{
						Distances.Add((GlobalTransform[i].GetTranslation() - GlobalTransform2[j].GetTranslation()).Size());
					}
				}
				
				EXPECT_EQ(ClusterMap.Num(), 1);
				EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles.Handle(6)->CastToRigidParticle(), { ParticleHandles.Handle(1)->CastToRigidParticle(),ParticleHandles.Handle(0)->CastToRigidParticle(),ParticleHandles.Handle(3)->CastToRigidParticle(),ParticleHandles.Handle(4)->CastToRigidParticle() }));

			}
		}

		
		TArray<FTransform> GlobalTransform;
		GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection, GlobalTransform);

		TArray<FTransform> GlobalTransform2;
		GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection2, GlobalTransform2);

		// build relative transforms distances
		TArray<FReal> Distances2;
		for (int32 i = 0; i < (int32)GlobalTransform.Num() - 1; i++)
		{
			for (int j = 0; j < (int32)GlobalTransform2.Num() - 1; j++)
			{
				Distances2.Add((GlobalTransform[i].GetTranslation() - GlobalTransform2[j].GetTranslation()).Size());
			}
		}
		for (int i = 0; i < Distances.Num()/2.0; i++)
		{
			EXPECT_LT( FMath::Abs(Distances[i] - Distances2[i]), 0.1 );
		}
		
	}


	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_UnionClustersFalling)
	{
		// Test : Joining collections using the ClusterGroupIndex by a particle dynamically created within the solver. 		
		
		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody(FVector(-2, 0, 3));
		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { FLT_MAX };
		Params.ClusterGroupIndex = 1;
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();


		TSharedPtr<FGeometryCollection> RestCollection2 = CreateClusteredBody(FVector(2, 0, 3));
		CreationParameters Params2;
		Params2.RestCollection = RestCollection2;
		Params2.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params2.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params2.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params2.Simulating = true;
		Params2.EnableClustering = true;
		Params2.DamageThreshold = { FLT_MAX };
		Params2.ClusterGroupIndex = 1;
		FGeometryCollectionWrapper* Collection2 = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params2)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);
		UnitTest.AddSimulationObject(Collection2);
		UnitTest.Initialize();

		TSharedPtr<FGeometryDynamicCollection> DynamicCollection = Collection->DynamicCollection;
		TSharedPtr<FGeometryDynamicCollection> DynamicCollection2 = Collection2->DynamicCollection;


		TArray<FReal> Distances;

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();
		const auto& ParticleHandles = UnitTest.Solver->GetParticles().GetParticleHandles();

		UnitTest.Solver->RegisterSimOneShotCallback([&Clustering,&ClusterMap,&ParticleHandles]()
		{
			EXPECT_EQ(ClusterMap.Num(),2);
			EXPECT_TRUE(ClusterMapContains(ClusterMap,ParticleHandles.Handle(2)->CastToRigidParticle(),{ParticleHandles.Handle(1)->CastToRigidParticle(),ParticleHandles.Handle(0)->CastToRigidParticle()}));
			EXPECT_TRUE(ClusterMapContains(ClusterMap,ParticleHandles.Handle(5)->CastToRigidParticle(),{ParticleHandles.Handle(4)->CastToRigidParticle(),ParticleHandles.Handle(3)->CastToRigidParticle()}));
		});

		TArray<FTransform> PrevGlobalTransform;
		GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection, PrevGlobalTransform);

		TArray<FTransform> PrevGlobalTransform2;
		GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection2, PrevGlobalTransform2);


		for (int Frame = 0; Frame < 100; Frame++)
		{
			UnitTest.Advance();

			TArray<FTransform> GlobalTransform;
			GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection, GlobalTransform);

			TArray<FTransform> GlobalTransform2;
			GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection2, GlobalTransform2);

			EXPECT_EQ(ClusterMap.Num(), 1);
			EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles.Handle(6)->CastToRigidParticle(), { ParticleHandles.Handle(1)->CastToRigidParticle(),ParticleHandles.Handle(0)->CastToRigidParticle(),ParticleHandles.Handle(3)->CastToRigidParticle(),ParticleHandles.Handle(4)->CastToRigidParticle() }));

			EXPECT_TRUE(DynamicCollection->GetParent(0) == INDEX_NONE);
			EXPECT_TRUE(DynamicCollection->GetParent(1) == INDEX_NONE);
			EXPECT_TRUE(DynamicCollection->GetParent(2) == INDEX_NONE);

			EXPECT_TRUE(DynamicCollection2->GetParent(0) == INDEX_NONE);
			EXPECT_TRUE(DynamicCollection2->GetParent(1) == INDEX_NONE);
			EXPECT_TRUE(DynamicCollection2->GetParent(2) == INDEX_NONE);

			EXPECT_TRUE(GlobalTransform[0].GetTranslation().X == PrevGlobalTransform[0].GetTranslation().X);
			EXPECT_TRUE(GlobalTransform[1].GetTranslation().X == PrevGlobalTransform[1].GetTranslation().X);
			EXPECT_TRUE(GlobalTransform[0].GetTranslation().Y == PrevGlobalTransform[0].GetTranslation().Y);
			EXPECT_TRUE(GlobalTransform[1].GetTranslation().Y == PrevGlobalTransform[1].GetTranslation().Y);
			EXPECT_TRUE(GlobalTransform[0].GetTranslation().Z < PrevGlobalTransform[0].GetTranslation().Z);
			EXPECT_TRUE(GlobalTransform[1].GetTranslation().Z < PrevGlobalTransform[1].GetTranslation().Z);

			EXPECT_TRUE(GlobalTransform2[0].GetTranslation().X == PrevGlobalTransform2[0].GetTranslation().X);
			EXPECT_TRUE(GlobalTransform2[1].GetTranslation().X == PrevGlobalTransform2[1].GetTranslation().X);
			EXPECT_TRUE(GlobalTransform2[0].GetTranslation().Y == PrevGlobalTransform2[0].GetTranslation().Y);
			EXPECT_TRUE(GlobalTransform2[1].GetTranslation().Y == PrevGlobalTransform2[1].GetTranslation().Y);
			EXPECT_TRUE(GlobalTransform2[0].GetTranslation().Z < PrevGlobalTransform2[0].GetTranslation().Z);
			EXPECT_TRUE(GlobalTransform2[1].GetTranslation().Z < PrevGlobalTransform2[1].GetTranslation().Z);

			PrevGlobalTransform = GlobalTransform;
			PrevGlobalTransform2 = GlobalTransform2;
		}
	}


	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_UnionClusterCollisions)
	{
		// Test : Joining collections using the ClusterGroupIndex by a particle dynamically created within the solver. 		

		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody(FVector(-2, 0, 3));
		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_LevelSet;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { FLT_MAX };
		Params.ClusterGroupIndex = 1;
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		TSharedPtr<FGeometryCollection> RestCollection2 = CreateClusteredBody(FVector(2, 0, 3));
		CreationParameters Params2;
		Params2.RestCollection = RestCollection2;
		Params2.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params2.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_LevelSet;
		Params2.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params2.Simulating = true;
		Params2.EnableClustering = true;
		Params2.DamageThreshold = { FLT_MAX };
		Params2.ClusterGroupIndex = 1;
		FGeometryCollectionWrapper* Collection2 = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params2)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);
		UnitTest.AddSimulationObject(Collection2);
		UnitTest.AddSimulationObject(TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>());
		//make newsimobject set a full block filter on all shapes!
		UnitTest.Initialize();

		TSharedPtr<FGeometryDynamicCollection> DynamicCollection = Collection->DynamicCollection;
		TSharedPtr<FGeometryDynamicCollection> DynamicCollection2 = Collection2->DynamicCollection;

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();
		const auto& ParticleHandles = UnitTest.Solver->GetParticles().GetParticleHandles();
		const TArrayCollectionArray<FRigidTransform3>& ChildToParent = Clustering.GetChildToParentMap();

		FVector InitialRootPosition;
		FVector RelativeChildOffsets[4];

		UnitTest.Solver->RegisterSimOneShotCallback([&]()
		{
			EXPECT_EQ(ClusterMap.Num(),2);
			TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> CollectionParticles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2)
			};
			EXPECT_EQ(CollectionParticles.Num(),3);
			EXPECT_EQ(Collection->PhysObject->GetNumTransforms(), 3);
			EXPECT_TRUE(ClusterMapContains(ClusterMap,CollectionParticles[2],{CollectionParticles[1],CollectionParticles[0]}));

			TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> CollectionParticles2 = {
			Collection2->PhysObject->GetParticle_Internal(0),
			Collection2->PhysObject->GetParticle_Internal(1),
			Collection2->PhysObject->GetParticle_Internal(2)
			};
			EXPECT_EQ(CollectionParticles2.Num(),3);
			EXPECT_EQ(Collection2->PhysObject->GetNumTransforms(), 3);
			EXPECT_TRUE(ClusterMapContains(ClusterMap,CollectionParticles2[2],{CollectionParticles2[1],CollectionParticles2[0]}));
		});

		for (int Frame = 0; Frame < 50; Frame++)
		{
			UnitTest.Advance();

			TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> CollectionParticles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			};
			EXPECT_EQ(CollectionParticles.Num(),3);
			EXPECT_EQ(Collection->PhysObject->GetNumTransforms(), 3);

			TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> CollectionParticles2 = {
			Collection2->PhysObject->GetParticle_Internal(0),
			Collection2->PhysObject->GetParticle_Internal(1),
			Collection2->PhysObject->GetParticle_Internal(2),
			};

			const auto* Root = CollectionParticles[0]->ClusterIds().Id;

			EXPECT_EQ(ClusterMap.Num(), 1);
			EXPECT_TRUE(ClusterMapContains(ClusterMap, Root, { CollectionParticles[0],CollectionParticles[1], CollectionParticles2[0], CollectionParticles2[1] }));

			//
			// TEST
			// Validate that the relative translations stay the same
			// for the union clustered children, and that the root actually
			// moves. 
			//
			if (Frame == 0)
			{
				InitialRootPosition = Root->GetX();
				RelativeChildOffsets[0] = CollectionParticles[0]->GetX() - Root->GetX();
				RelativeChildOffsets[1] = CollectionParticles[1]->GetX() - Root->GetX();
				RelativeChildOffsets[2] = CollectionParticles2[0]->GetX() - Root->GetX();
				RelativeChildOffsets[3] = CollectionParticles2[1]->GetX() - Root->GetX();
			}
			else
			{
				FTransform RootTransform(Root->GetR(), Root->GetX());

				TArray<FTransform> GlobalTransform1;
				GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection, GlobalTransform1);

				TArray<FTransform> GlobalTransform2;
				GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection2, GlobalTransform2);

				EXPECT_TRUE(!InitialRootPosition.Equals(Root->GetX())); // root moves

				FVector ActualRelativeChildOffsets0 = GlobalTransform1[0].GetRelativeTransform(RootTransform).GetTranslation();
				FVector ActualRelativeChildOffsets1 = GlobalTransform1[1].GetRelativeTransform(RootTransform).GetTranslation();
				FVector ActualRelativeChildOffsets2 = GlobalTransform2[0].GetRelativeTransform(RootTransform).GetTranslation();
				FVector ActualRelativeChildOffsets3 = GlobalTransform2[1].GetRelativeTransform(RootTransform).GetTranslation();
				EXPECT_TRUE(RelativeChildOffsets[0].Equals(ActualRelativeChildOffsets0));
				EXPECT_TRUE(RelativeChildOffsets[1].Equals(ActualRelativeChildOffsets1));
				EXPECT_TRUE(RelativeChildOffsets[2].Equals(ActualRelativeChildOffsets2));
				EXPECT_TRUE(RelativeChildOffsets[3].Equals(ActualRelativeChildOffsets3));
			}

			//
			// TEST
			// Validate that the children have been removed from the 
			// parenting hierarchy.
			//
			EXPECT_TRUE(DynamicCollection->GetParent(0) == INDEX_NONE);
			EXPECT_TRUE(DynamicCollection->GetParent(1) == INDEX_NONE);
			EXPECT_TRUE(DynamicCollection->GetParent(2) == INDEX_NONE);

			EXPECT_TRUE(DynamicCollection2->GetParent(0) == INDEX_NONE);
			EXPECT_TRUE(DynamicCollection2->GetParent(1) == INDEX_NONE);
			EXPECT_TRUE(DynamicCollection2->GetParent(2) == INDEX_NONE);
		}
	}
	

	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_ReleaseClusterParticle_ClusteredNode)
	{
		// Test : Build to geometry collections, cluster them together, release the sub bodies of the first collection. 
		//        ... should create a internal cluster with property transform mappings. 

		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody(FVector(0, 0, 100));
		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { FLT_MAX };
		Params.ClusterGroupIndex = 1;
		Params.ClusterConnectionMethod = Chaos::FClusterCreationParameters::EConnectionMethod::DelaunayTriangulation;
		FGeometryCollectionWrapper* Collection1 = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection1);


		TSharedPtr<FGeometryCollection> RestCollection2 = CreateClusteredBody(FVector(0, 0, 200));
		CreationParameters Params2;
		Params2.RestCollection = RestCollection2;
		Params2.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params2.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
		Params2.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;
		Params2.Simulating = true;
		Params2.EnableClustering = true;
		Params2.DamageThreshold = { FLT_MAX };
		Params2.ClusterGroupIndex = 1;
		Params2.ClusterConnectionMethod = Chaos::FClusterCreationParameters::EConnectionMethod::DelaunayTriangulation;
		FGeometryCollectionWrapper* Collection2 = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params2)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection2);
		UnitTest.Initialize();

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		Chaos::FClusterUnionManager& ClusterUnionManager = Clustering.GetClusterUnionManager();

		const auto& ClusterMap = Clustering.GetChildrenMap();

		TSharedPtr<FGeometryDynamicCollection> DynamicCollection1 = Collection1->DynamicCollection;
		TSharedPtr<FGeometryDynamicCollection> DynamicCollection2 = Collection2->DynamicCollection;

		TArray<FTransform> Collection1_InitialTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection1, Collection1_InitialTM);
		TArray<FTransform> Collection2_InitialTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection2, Collection2_InitialTM);

		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> Collection1Handles = {
			Collection1->PhysObject->GetParticle_Internal(0),
			Collection1->PhysObject->GetParticle_Internal(1),
			Collection1->PhysObject->GetParticle_Internal(2),
		};
		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> Collection2Handles = {
			Collection2->PhysObject->GetParticle_Internal(0),
			Collection2->PhysObject->GetParticle_Internal(1),
			Collection2->PhysObject->GetParticle_Internal(2),
		};

		const auto& SovlerParticleHandles = UnitTest.Solver->GetParticles().GetParticleHandles();
		

		UnitTest.Solver->RegisterSimOneShotCallback([&]()
		{
			Collection1Handles = {
			Collection1->PhysObject->GetParticle_Internal(0),
			Collection1->PhysObject->GetParticle_Internal(1),
			Collection1->PhysObject->GetParticle_Internal(2),
				};
			Collection2Handles = {
				Collection2->PhysObject->GetParticle_Internal(0),
				Collection2->PhysObject->GetParticle_Internal(1),
				Collection2->PhysObject->GetParticle_Internal(2),
			};
			EXPECT_EQ(SovlerParticleHandles.Size(), 6);
			EXPECT_EQ(ClusterMap.Num(),2);
			EXPECT_TRUE(ClusterMapContains(ClusterMap,Collection1Handles[2],{Collection1Handles[1],Collection1Handles[0]}));
			EXPECT_TRUE(ClusterMapContains(ClusterMap,Collection2Handles[2],{Collection2Handles[1],Collection2Handles[0]}));
		});

		UnitTest.Advance();

		EXPECT_EQ(SovlerParticleHandles.Size(), 7);
		EXPECT_EQ(ClusterMap.Num(), 1);
		EXPECT_TRUE(ClusterMapContains(ClusterMap, SovlerParticleHandles.Handle(6)->CastToRigidParticle(), { Collection1Handles[1],Collection1Handles[0], Collection2Handles[1],Collection2Handles[0] }));
		EXPECT_TRUE(ClusterUnionManager.FindClusterUnionFromExplicitIndex(1) != nullptr);

		Chaos::FClusterUnion& ClusterUnion = *ClusterUnionManager.FindClusterUnionFromExplicitIndex(1);
		EXPECT_EQ(ClusterUnion.InternalCluster, SovlerParticleHandles.Handle(6).Get());
		// A bit of an assumption that the root particle is the 2nd index in these arrays.
		EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(Collection1Handles[0]));
		EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(Collection1Handles[1]));
		EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(Collection2Handles[0]));
		EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(Collection2Handles[1]));
		EXPECT_EQ(ClusterUnion.ChildParticles.Num(), 4);
		EXPECT_EQ(ClusterUnion.ExplicitIndex, 1);
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ClusterUnion.InternalCluster, { Collection1Handles[0], Collection1Handles[1], Collection2Handles[0], Collection2Handles[1] }));
		EXPECT_EQ(ClusterMap[ClusterUnion.InternalCluster].Num(), 4);

		TArray<FTransform> Collection1_PreReleaseTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection1, Collection1_PreReleaseTM);
		TArray<FTransform> Collection2_PreReleaseTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection2, Collection2_PreReleaseTM);
		for (int Idx = 0; Idx < Collection1_PreReleaseTM.Num() - 1; Idx++) {
			EXPECT_LT(Collection1_PreReleaseTM[Idx].GetTranslation().Z, Collection1_InitialTM[Idx].GetTranslation().Z);
			EXPECT_LT(Collection2_PreReleaseTM[Idx].GetTranslation().Z, Collection2_InitialTM[Idx].GetTranslation().Z);
		}

		UnitTest.Solver->GetEvolution()->GetRigidClustering().ReleaseClusterParticles({ Collection1Handles[0],Collection1Handles[1] });
		ClusterUnionManager.HandleDeferredClusterUnionUpdateProperties();

		EXPECT_EQ(SovlerParticleHandles.Size(), 7);
		EXPECT_FALSE(ClusterUnion.ChildParticles.Contains(Collection1Handles[0]));
		EXPECT_FALSE(ClusterUnion.ChildParticles.Contains(Collection1Handles[1]));
		EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(Collection2Handles[0]));
		EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(Collection2Handles[1]));
		EXPECT_EQ(ClusterUnion.ChildParticles.Num(), 2);
		EXPECT_EQ(ClusterUnion.ExplicitIndex, 1);
		EXPECT_FALSE(ClusterMapContains(ClusterMap, ClusterUnion.InternalCluster, { Collection1Handles[0], Collection1Handles[1] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ClusterUnion.InternalCluster, { Collection2Handles[0], Collection2Handles[1] }));
		EXPECT_EQ(ClusterMap[ClusterUnion.InternalCluster].Num(), 2);

		UnitTest.Advance();

		EXPECT_EQ(SovlerParticleHandles.Size(), 7);

		TArray<FTransform> Collection1_PostReleaseTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection1, Collection1_PostReleaseTM);
		TArray<FTransform> Collection2_PostReleaseTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection2, Collection2_PostReleaseTM);
		for (int Idx = 0; Idx < Collection1_PostReleaseTM.Num() - 1; Idx++) {
			EXPECT_LT(Collection1_PostReleaseTM[Idx].GetTranslation().Z, Collection1_PreReleaseTM[Idx].GetTranslation().Z);
			EXPECT_LT(Collection2_PostReleaseTM[Idx].GetTranslation().Z, Collection2_PreReleaseTM[Idx].GetTranslation().Z);
		}
	}


	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_ReleaseClusterParticle_ClusteredKinematicNode)
	{
		// Test : Build to geometry collections, cluster them together, release the sub bodies of the first collection. 
		// this should create a internal cluster with property transform mappings. 

		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody(FVector(0, 0, 100));
		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { FLT_MAX };
		Params.ClusterGroupIndex = 1;
		Params.ClusterConnectionMethod = Chaos::FClusterCreationParameters::EConnectionMethod::DelaunayTriangulation;
		FGeometryCollectionWrapper* Collection1 = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection1);


		TSharedPtr<FGeometryCollection> RestCollection2 = CreateClusteredBody(FVector(0, 0, 200));
		CreationParameters Params2;
		Params2.RestCollection = RestCollection2;
		Params2.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params2.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
		Params2.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;
		Params2.Simulating = true;
		Params2.EnableClustering = true;
		Params2.DamageThreshold = { FLT_MAX };
		Params2.ClusterGroupIndex = 1;
		Params2.ClusterConnectionMethod = Chaos::FClusterCreationParameters::EConnectionMethod::DelaunayTriangulation;
		FGeometryCollectionWrapper* Collection2 = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params2)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection2);

		TSharedPtr<FGeometryDynamicCollection> DynamicCollection1 = Collection1->DynamicCollection;
		TSharedPtr<FGeometryDynamicCollection> DynamicCollection2 = Collection2->DynamicCollection;
		DynamicCollection1->ModifyAttribute<uint8>("DynamicState", FGeometryCollection::TransformGroup)[1] = (uint8)EObjectStateTypeEnum::Chaos_Object_Kinematic;

		UnitTest.Initialize();

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		Chaos::FClusterUnionManager& ClusterUnionManager = Clustering.GetClusterUnionManager();
		const auto& ClusterMap = Clustering.GetChildrenMap();

		TArray<FTransform> Collection1_InitialTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection1, Collection1_InitialTM);
		TArray<FTransform> Collection2_InitialTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection2, Collection2_InitialTM);

		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> Collection1Handles = {
			Collection1->PhysObject->GetParticle_Internal(0),
			Collection1->PhysObject->GetParticle_Internal(1),
			Collection1->PhysObject->GetParticle_Internal(2),
		};
		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> Collection2Handles = {
			Collection2->PhysObject->GetParticle_Internal(0),
			Collection2->PhysObject->GetParticle_Internal(1),
			Collection2->PhysObject->GetParticle_Internal(2),
		};
		const auto& SovlerParticleHandles = UnitTest.Solver->GetParticles().GetParticleHandles();

		UnitTest.Solver->RegisterSimOneShotCallback([&]()
		{
			Collection1Handles = {
			Collection1->PhysObject->GetParticle_Internal(0),
			Collection1->PhysObject->GetParticle_Internal(1),
			Collection1->PhysObject->GetParticle_Internal(2),
				};
			Collection2Handles = {
				Collection2->PhysObject->GetParticle_Internal(0),
				Collection2->PhysObject->GetParticle_Internal(1),
				Collection2->PhysObject->GetParticle_Internal(2),
			};
			EXPECT_EQ(SovlerParticleHandles.Size(),6);
			EXPECT_EQ(ClusterMap.Num(),2);
			EXPECT_TRUE(ClusterMapContains(ClusterMap,Collection1Handles[2],{Collection1Handles[1],Collection1Handles[0]}));
			EXPECT_TRUE(ClusterMapContains(ClusterMap,Collection2Handles[2],{Collection2Handles[1],Collection2Handles[0]}));
		});

		UnitTest.Advance();

		EXPECT_EQ(SovlerParticleHandles.Size(), 7);
		EXPECT_EQ(ClusterMap.Num(), 1);
		EXPECT_TRUE(ClusterMapContains(ClusterMap, SovlerParticleHandles.Handle(6)->CastToRigidParticle(), { Collection1Handles[1],Collection1Handles[0], Collection2Handles[1],Collection2Handles[0] }));
		EXPECT_TRUE(ClusterUnionManager.FindClusterUnionFromExplicitIndex(1) != nullptr);

		Chaos::FClusterUnion& ClusterUnion = *ClusterUnionManager.FindClusterUnionFromExplicitIndex(1);
		EXPECT_EQ(ClusterUnion.InternalCluster, SovlerParticleHandles.Handle(6).Get());
		// A bit of an assumption that the root particle is the 2nd index in these arrays.
		EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(Collection1Handles[0]));
		EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(Collection1Handles[1]));
		EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(Collection2Handles[0]));
		EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(Collection2Handles[1]));
		EXPECT_EQ(ClusterUnion.ChildParticles.Num(), 4);
		EXPECT_EQ(ClusterUnion.ExplicitIndex, 1);
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ClusterUnion.InternalCluster, { Collection1Handles[0], Collection1Handles[1], Collection2Handles[0], Collection2Handles[1] }));
		EXPECT_EQ(ClusterMap[ClusterUnion.InternalCluster].Num(), 4);

		TArray<FTransform> Collection1_PreReleaseTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection1, Collection1_PreReleaseTM);
		TArray<FTransform> Collection2_PreReleaseTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection2, Collection2_PreReleaseTM);
		for (int Idx = 0; Idx < Collection1_PreReleaseTM.Num() - 1; Idx++) 
		{
			EXPECT_EQ(Collection1_PreReleaseTM[Idx].GetTranslation().Z, Collection1_InitialTM[Idx].GetTranslation().Z);
			EXPECT_EQ(Collection2_PreReleaseTM[Idx].GetTranslation().Z, Collection2_InitialTM[Idx].GetTranslation().Z);
		}

		UnitTest.Solver->GetEvolution()->GetRigidClustering().ReleaseClusterParticles({ Collection1Handles[0],Collection1Handles[1] });
		ClusterUnionManager.HandleDeferredClusterUnionUpdateProperties();

		EXPECT_EQ(SovlerParticleHandles.Size(), 7);
		EXPECT_FALSE(ClusterUnion.ChildParticles.Contains(Collection1Handles[0]));
		EXPECT_FALSE(ClusterUnion.ChildParticles.Contains(Collection1Handles[1]));
		EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(Collection2Handles[0]));
		EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(Collection2Handles[1]));
		EXPECT_EQ(ClusterUnion.ChildParticles.Num(), 2);
		EXPECT_EQ(ClusterUnion.ExplicitIndex, 1);
		EXPECT_FALSE(ClusterMapContains(ClusterMap, ClusterUnion.InternalCluster, { Collection1Handles[0], Collection1Handles[1] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, ClusterUnion.InternalCluster, { Collection2Handles[0], Collection2Handles[1] }));
		EXPECT_EQ(ClusterMap[ClusterUnion.InternalCluster].Num(), 2);

		UnitTest.Advance();

		EXPECT_EQ(SovlerParticleHandles.Size(), 7);

		// validate that DynamicCollection2 became dynamic and fell from the cluster. 

		TArray<FTransform> Collection1_PostReleaseTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection1, Collection1_PostReleaseTM);
		TArray<FTransform> Collection2_PostReleaseTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection2, Collection2_PostReleaseTM);
		for (int Idx = 0; Idx < Collection1_PostReleaseTM.Num() - 1; Idx++) 
		{
			if(Idx == 1)
			{
				EXPECT_EQ(Collection1_PostReleaseTM[Idx].GetTranslation().Z, Collection1_PreReleaseTM[Idx].GetTranslation().Z); // the original kinematic should be frozen
			}
			else
			{
				EXPECT_LT(Collection1_PostReleaseTM[Idx].GetTranslation().Z, Collection1_PreReleaseTM[Idx].GetTranslation().Z);
			}

			EXPECT_LT(Collection2_PostReleaseTM[Idx].GetTranslation().Z, Collection2_PreReleaseTM[Idx].GetTranslation().Z);
		}
	}

	// Create two boxes and wrap them in a clusterm and wrap that in a second cluster.
	// Release the two boxes, and the boxes should fall. Both clusters should be empty and therefore disabled.
	// 
	// @todo(chaos): this test is disabled. The outer cluster is not being disabled when the leafs are released.
	// Fix when this functionality is supported again...
	GTEST_TEST(AllTraits, DISABLED_GeometryCollection_RigidBodies_ClusterTest_ReleaseClusterParticles_AllLeafNodes)
	{
		// Release the leaf nodes of a cluster. This test exercises the clusters ability to deactivate from the bottom up. 

		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody_TwoParents_TwoBodies(FVector(0, 0, 100));
		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { FLT_MAX };
		Params.MaxClusterLevel = 1;
		Params.ClusterGroupIndex = 0;		
		FGeometryCollectionWrapper* Collection1 = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();
		TSharedPtr<FGeometryDynamicCollection> DynamicCollection1 = Collection1->DynamicCollection;
		DynamicCollection1->ModifyAttribute<uint8>("DynamicState", FGeometryCollection::TransformGroup)[1] = (uint8)EObjectStateTypeEnum::Chaos_Object_Kinematic;

		UnitTest.AddSimulationObject(Collection1);

		UnitTest.Initialize();

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();
		TArray<FTransform> Collection1_InitialTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection1, Collection1_InitialTM);
		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> Collection1Handles = {
			Collection1->PhysObject->GetParticle_Internal(0),
			Collection1->PhysObject->GetParticle_Internal(1),
			Collection1->PhysObject->GetParticle_Internal(2),
		};
		const auto& SovlerParticleHandles = UnitTest.Solver->GetParticles().GetParticleHandles();

		UnitTest.Solver->RegisterSimOneShotCallback([&]()
		{
			Collection1Handles = {
			Collection1->PhysObject->GetParticle_Internal(0),
			Collection1->PhysObject->GetParticle_Internal(1),
			Collection1->PhysObject->GetParticle_Internal(2),
				};
			EXPECT_EQ(SovlerParticleHandles.Size(),4);
			EXPECT_EQ(ClusterMap.Num(),2);
			EXPECT_TRUE(ClusterMapContains(ClusterMap,Collection1Handles[2],{Collection1Handles[1],Collection1Handles[0]}));
			EXPECT_TRUE(ClusterMapContains(ClusterMap,Collection1Handles[3],{Collection1Handles[2]}));
		});

		UnitTest.Advance();

		EXPECT_EQ(SovlerParticleHandles.Size(), 4);
		EXPECT_EQ(ClusterMap.Num(), 2);
		EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection1Handles[2], { Collection1Handles[1],Collection1Handles[0] }));
		EXPECT_TRUE(ClusterMapContains(ClusterMap, Collection1Handles[3], { Collection1Handles[2] }));

		TArray<FTransform> Collection1_PreReleaseTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection1, Collection1_PreReleaseTM);
		for (int Idx = 0; Idx < Collection1_PreReleaseTM.Num() - 1; Idx++)
		{
			EXPECT_EQ(Collection1_PreReleaseTM[Idx].GetTranslation().Z, Collection1_InitialTM[Idx].GetTranslation().Z);
		}

		UnitTest.Solver->GetEvolution()->GetRigidClustering().ReleaseClusterParticles({ Collection1Handles[0],Collection1Handles[1] });

		EXPECT_EQ(SovlerParticleHandles.Size(), 4);
		EXPECT_EQ(ClusterMap.Num(), 1);

		UnitTest.Advance();

		EXPECT_EQ(SovlerParticleHandles.Size(), 4);
		EXPECT_EQ(ClusterMap.Num(), 1);

		// validate that DynamicCollection1 BODY 2 became dynamic and fell from the cluster. 
		TArray<FTransform> Collection1_PostReleaseTM; GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection1, Collection1_PostReleaseTM);
		EXPECT_NEAR(Collection1_PostReleaseTM[1].GetTranslation().Z, Collection1_PreReleaseTM[1].GetTranslation().Z, KINDA_SMALL_NUMBER); // the original kinematic should be frozen
		EXPECT_LT(Collection1_PostReleaseTM[0].GetTranslation().Z, Collection1_PreReleaseTM[0].GetTranslation().Z);
	}

	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_ClusterTest_ReleaseClusterParticles_ClusterNodeAndSubClusterNode)
	{
		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody_TwoParents_TwoBodies(FVector(0, 0, 100));
		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { FLT_MAX };
		Params.MaxClusterLevel = 1;
		Params.ClusterGroupIndex = 1;
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		TSharedPtr<FGeometryDynamicCollection> DynamicCollection = Collection->DynamicCollection;
		DynamicCollection->ModifyAttribute<uint8>("DynamicState", FGeometryCollection::TransformGroup)[1] = (uint8)EObjectStateTypeEnum::Chaos_Object_Kinematic;

		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();

		// The tests below require a list of all the current particles which are abstracted away a little
		// inside the solver particles handler. This helper just lets us auto cast to rigids as we know
		// that's all that exists in the solver.
		struct FRigidParticleWrapper
		{
			FRigidParticleWrapper(TGeometryParticleHandles<FReal, 3>& InParticlesRef)
				: Particles(InParticlesRef)
			{}

			TGeometryParticleHandles<FReal, 3>& Particles;

			TPBDRigidParticleHandle<FReal, 3>* operator[](int32 InIndex)
			{
				return Particles.Handle(InIndex)->CastToRigidParticle();
			}
		};
		FRigidParticleWrapper ParticleHandles(UnitTest.Solver->GetParticles().GetParticleHandles());

		UnitTest.Advance();

		FRigidClustering& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		Chaos::FClusterUnionManager& ClusterUnionManager = Clustering.GetClusterUnionManager();

		const FRigidClustering::FClusterMap& ClusterMap = Clustering.GetChildrenMap();
		const Chaos::TArrayCollectionArray<Chaos::ClusterId>& ClusterIdsArray = Clustering.GetClusterIdsArray();

		UnitTest.Solver->RegisterSimOneShotCallback([&]()
		{
			EXPECT_EQ(ClusterMap.Num(),2);
			EXPECT_TRUE(ClusterMapContains(ClusterMap,ParticleHandles[2],{ParticleHandles[0],ParticleHandles[1]}));
			EXPECT_TRUE(ClusterMapContains(ClusterMap,ParticleHandles[4],{ParticleHandles[2]}));
		});

		EXPECT_TRUE(ClusterUnionManager.FindClusterUnionFromExplicitIndex(1) != nullptr);

		Chaos::FClusterUnion& ClusterUnion = *ClusterUnionManager.FindClusterUnionFromExplicitIndex(1);

		// Test releasing a specific unioned cluster
		// We end up with the following cluster tree
		//     4
		//     |
		//     2
		//    / \
		//   1   0
		// On frame 5 we tell particle 4 as a cluster parent to release its children (only 2) and verify the result
		for (int Frame = 1; Frame < 10; Frame++)
		{
			UnitTest.Advance();

			if (Frame == 5)
			{
				UnitTest.Solver->GetEvolution()->GetRigidClustering().ReleaseClusterParticles(ParticleHandles[4]->CastToClustered(), true);
			}
			
			if (Frame < 5)
			{

				EXPECT_TRUE(ParticleHandles[2]->Disabled());
				EXPECT_NE(ClusterIdsArray[0].Id, nullptr);
				EXPECT_EQ(ClusterIdsArray[1].Id, nullptr);
				EXPECT_EQ(ClusterIdsArray[2].Id, nullptr);

				EXPECT_TRUE(ClusterUnion.ChildParticles.Contains(ParticleHandles[2]));
				EXPECT_EQ(ClusterUnion.ChildParticles.Num(), 1);
				EXPECT_EQ(ClusterUnion.ExplicitIndex, 1);
				EXPECT_TRUE(ClusterMapContains(ClusterMap, ClusterUnion.InternalCluster, { ParticleHandles[2] }));
				EXPECT_EQ(ClusterMap[ClusterUnion.InternalCluster].Num(), 1);
				EXPECT_EQ(ClusterMap.Num(), 2);
			}
			else
			{
				EXPECT_TRUE(!ParticleHandles[2]->Disabled());
				EXPECT_EQ(ClusterIdsArray[0].Id, nullptr);
				EXPECT_EQ(ClusterIdsArray[1].Id, nullptr);
				EXPECT_EQ(ClusterIdsArray[2].Id, nullptr);

				EXPECT_FALSE(ClusterUnion.ChildParticles.Contains(ParticleHandles[2]));
				EXPECT_EQ(ClusterUnion.ChildParticles.Num(), 0);
				EXPECT_EQ(ClusterUnion.ExplicitIndex, 1);
				EXPECT_FALSE(ClusterMapContains(ClusterMap, ClusterUnion.InternalCluster, { ParticleHandles[2] }));
				EXPECT_EQ(ClusterMap[ClusterUnion.InternalCluster].Num(), 0);
				EXPECT_EQ(ClusterMap.Num(), 2);
				EXPECT_TRUE(ClusterMapContains(ClusterMap, ParticleHandles[2], { ParticleHandles[0], ParticleHandles[1] }));
			}				
		}
	}

	GTEST_TEST(AllTraits, DISABLED_GeometryCollection_RigidBodiess_ClusterTest_MaxClusterLevel_1)
	{
		FFramework UnitTest;
		
		// Create hierarchy 
		TSharedPtr<FGeometryCollection> RestCollection;
		RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(0, 0, 0)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(100, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(200, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(300, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(400, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(500, 0, 0)), FVector(1.0)));

		RestCollection->AddElements(7, FGeometryCollection::TransformGroup);
		RestCollection->Transform[10].SetTranslation(FVector3f(0,0,0));

		RestCollection->SimulationType[0] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[1] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[2] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[3] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[4] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[5] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[6] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[7] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[8] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[9] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[10] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[11] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[12] = FGeometryCollection::ESimulationTypes::FST_Clustered;

		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 6, { 11 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 7, { 3,4,5 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 8, { 6 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 9, { 7 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 10, { 9,8 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 11, { 12 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 12, { 0,1,2 });

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 100.0f, 4.0f, 2.0f };
		Params.MaxClusterLevel = 3;
		Params.ClusterGroupIndex = 0;

		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();
		UnitTest.Advance();

		FRadialFalloff* FalloffField = new FRadialFalloff();
		FalloffField->Magnitude = 104.0;
		FalloffField->Radius = 10000.0;
		FalloffField->Position = FVector(0.0, 0.0, 0.0);
		FalloffField->Falloff = EFieldFalloffType::Field_FallOff_None;

		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			Collection->PhysObject->GetParticle_Internal(4),
			Collection->PhysObject->GetParticle_Internal(5),
			Collection->PhysObject->GetParticle_Internal(6),
			Collection->PhysObject->GetParticle_Internal(7),
			Collection->PhysObject->GetParticle_Internal(8),
			Collection->PhysObject->GetParticle_Internal(9),
			Collection->PhysObject->GetParticle_Internal(10),
		};
		Chaos::FRigidClustering& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();
		auto& x = UnitTest.Solver->GetEvolution()->GetRigidClustering().GetStrainArray();


		{
			EXPECT_EQ(ClusterMap.Num(), 5);
			EXPECT_EQ(ClusterMap[ParticleHandles[6]].Num(), 3);
			EXPECT_TRUE(ClusterMap[ParticleHandles[6]].Contains(ParticleHandles[0]));
			EXPECT_TRUE(ClusterMap[ParticleHandles[6]].Contains(ParticleHandles[1]));
			EXPECT_TRUE(ClusterMap[ParticleHandles[6]].Contains(ParticleHandles[2]));
			EXPECT_EQ(ClusterMap[ParticleHandles[7]].Num(), 3);
			EXPECT_TRUE(ClusterMap[ParticleHandles[7]].Contains(ParticleHandles[3]));
			EXPECT_TRUE(ClusterMap[ParticleHandles[7]].Contains(ParticleHandles[4]));
			EXPECT_TRUE(ClusterMap[ParticleHandles[7]].Contains(ParticleHandles[5]));
			EXPECT_EQ(ClusterMap[ParticleHandles[8]].Num(), 1);
			EXPECT_TRUE(ClusterMap[ParticleHandles[8]].Contains(ParticleHandles[6]));
			EXPECT_EQ(ClusterMap[ParticleHandles[9]].Num(), 1);
			EXPECT_TRUE(ClusterMap[ParticleHandles[9]].Contains(ParticleHandles[7]));
			EXPECT_EQ(ClusterMap[ParticleHandles[10]].Num(), 2);
			EXPECT_TRUE(ClusterMap[ParticleHandles[10]].Contains(ParticleHandles[8]));
			EXPECT_TRUE(ClusterMap[ParticleHandles[10]].Contains(ParticleHandles[9]));
		}

		EXPECT_TRUE(ParticleHandles[6]->Disabled());
		EXPECT_TRUE(ParticleHandles[7]->Disabled());
		EXPECT_TRUE(ParticleHandles[8]->Disabled());
		EXPECT_TRUE(ParticleHandles[9]->Disabled());
		EXPECT_FALSE(ParticleHandles[10]->Disabled());
		// {100, 4, 2} -> {4, 2}
		FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_ExternalClusterStrain);
		UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, FalloffField->NewCopy() });
		UnitTest.Advance();
		EXPECT_TRUE(ParticleHandles[6]->Disabled());
		EXPECT_TRUE(ParticleHandles[7]->Disabled());
		EXPECT_FALSE(ParticleHandles[8]->Disabled());
		EXPECT_FALSE(ParticleHandles[9]->Disabled());
		EXPECT_TRUE(ParticleHandles[10]->Disabled());

		// {4, 2} -> {4, 2} (beyond max cluster level)
		UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, FalloffField->NewCopy() });
		UnitTest.Advance();
		EXPECT_TRUE(ParticleHandles[0]->Disabled());
		EXPECT_TRUE(ParticleHandles[1]->Disabled());
		EXPECT_TRUE(ParticleHandles[2]->Disabled());
		EXPECT_TRUE(ParticleHandles[3]->Disabled());
		EXPECT_TRUE(ParticleHandles[4]->Disabled());
		EXPECT_TRUE(ParticleHandles[5]->Disabled());
		EXPECT_TRUE(ParticleHandles[6]->Disabled());
		EXPECT_TRUE(ParticleHandles[7]->Disabled());
		EXPECT_FALSE(ParticleHandles[8]->Disabled());
		EXPECT_FALSE(ParticleHandles[9]->Disabled());
		EXPECT_TRUE(ParticleHandles[10]->Disabled());

		// {4, 2} -> {4, 2} (beyond max cluster level)
		// but this time, force internal reordering via internal strain.
		TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_InternalClusterStrain); // Note: now internal strain
		FalloffField->Magnitude = -3.0;
		UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, FalloffField->NewCopy() });
		UnitTest.Advance();
		EXPECT_TRUE(ParticleHandles[0]->Disabled());
		EXPECT_TRUE(ParticleHandles[1]->Disabled());
		EXPECT_TRUE(ParticleHandles[2]->Disabled());
		EXPECT_TRUE(ParticleHandles[3]->Disabled());
		EXPECT_TRUE(ParticleHandles[4]->Disabled());
		EXPECT_TRUE(ParticleHandles[5]->Disabled());
		EXPECT_TRUE(ParticleHandles[6]->Disabled());
		EXPECT_TRUE(ParticleHandles[7]->Disabled());
		EXPECT_FALSE(ParticleHandles[8]->Disabled());
		EXPECT_FALSE(ParticleHandles[9]->Disabled());
		EXPECT_TRUE(ParticleHandles[10]->Disabled());

		{ // Check if level 3 rigids (children) are still clustered despite internal strain breaking level 2 clusters (parent)
			//EXPECT_EQ(ClusterMap[ParticleHandles[6]].Num(), 3);
			//EXPECT_TRUE(ClusterMap[ParticleHandles[6]].Contains(ParticleHandles[0]));
			//EXPECT_TRUE(ClusterMap[ParticleHandles[6]].Contains(ParticleHandles[1]));
			//EXPECT_TRUE(ClusterMap[ParticleHandles[6]].Contains(ParticleHandles[2]));
			//EXPECT_EQ(ClusterMap[ParticleHandles[7]].Num(), 3);
			//EXPECT_TRUE(ClusterMap[ParticleHandles[7]].Contains(ParticleHandles[3]));
			//EXPECT_TRUE(ClusterMap[ParticleHandles[7]].Contains(ParticleHandles[4]));
			//EXPECT_TRUE(ClusterMap[ParticleHandles[7]].Contains(ParticleHandles[5]));
		}
	}
	
	GTEST_TEST(AllTraits, DISABLED_GeometryCollection_RigidBodiess_ClusterTest_ParticleImplicitCollisionGeometry)
	{
		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody_FracturedGeometry();

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_LevelSet;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;				
		Params.CollisionGroup = -1;
		Params.CollisionParticleFraction = 0.70f;
		Params.MinLevelSetResolution = 15;
		Params.MaxLevelSetResolution = 20;
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();
		
		// We can get the size of an original CollisionParticles by storing the Simplicials before any internal resizing. 
		typedef TUniquePtr< FCollisionStructureManager::FSimplicial > FSimplicialPointer;
		const TManagedArray<FSimplicialPointer> & Simplicials = RestCollection->template GetAttribute<FSimplicialPointer>(FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup);

		UnitTest.Advance();		// this call triggers the array resize based on the fraction

		// Test non-clustered bodies
		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			Collection->PhysObject->GetParticle_Internal(4),
			Collection->PhysObject->GetParticle_Internal(5),
			Collection->PhysObject->GetParticle_Internal(6),
			Collection->PhysObject->GetParticle_Internal(7),
			Collection->PhysObject->GetParticle_Internal(8),
			Collection->PhysObject->GetParticle_Internal(9),
			Collection->PhysObject->GetParticle_Internal(10),
		};
		int32 NumCollisionParticles, ExpectedNumCollisionParticles;
		for (int i = 0; i < Collection->PhysObject->GetNumTransforms(); i++)
		{
			if (Collection->RestCollection->SimulationType[i] == FGeometryCollection::ESimulationTypes::FST_Rigid)
			{
				NumCollisionParticles = ParticleHandles[i]->CollisionParticlesSize(); // resized
				ExpectedNumCollisionParticles = (int32)Simplicials[i]->Size() * Params.CollisionParticleFraction;
				//EXPECT_EQ(ExpectedNumCollisionParticles, NumCollisionParticles);
				//EXPECT_FALSE(NumCollisionParticles == 0.0f);
				//EXPECT_FALSE(Params.CollisionParticleFraction == 1.0f); // not defaulted
			}
		}

	}

}
