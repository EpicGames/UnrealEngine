// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventManager.h"
#include "EventsData.h"
#include "GeometryCollection/GeometryCollectionTestCollisionResolution.h"
#include "GeometryCollection/GeometryCollectionTestFramework.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollectionProxyData.h"
#include "HeadlessChaosTestUtility.h"

namespace GeometryCollectionTest
{
	using namespace ChaosTest;

	namespace CollisionEventsTests
	{
		void AddSpheres(FFramework& UnitTest)
		{
			CreationParameters Params;
			Params.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;

			Params.EnableClustering = false;

			FVector Scale(50.f);
			Params.GeomTransform.SetScale3D(Scale); // Sphere radius
			Params.EnableClustering = false;

			// Make a dynamic simplicial sphere
			Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere;
			Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;

			Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
			Params.RootTransform = FTransform(FQuat::MakeFromEuler(FVector(0)), FVector(0, 0, 75));
			FGeometryCollectionWrapper* DynamicSphereCollection =
				TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
			UnitTest.AddSimulationObject(DynamicSphereCollection);

			// Make a kinematic implicit sphere
			Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere;
			Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;

			Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Kinematic;
			Params.RootTransform = FTransform(FQuat::MakeFromEuler(FVector(0)), FVector(0));
			FGeometryCollectionWrapper* KinematicSphereCollection =
				TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
			UnitTest.AddSimulationObject(KinematicSphereCollection);

			// Hard code masstolocal on rest collection to identity
			{
				TManagedArray<FTransform>& MassToLocal =
					DynamicSphereCollection->RestCollection->template ModifyAttribute<FTransform>(
						TEXT("MassToLocal"), FTransformCollection::TransformGroup);
				check(MassToLocal.Num() == 1);
				MassToLocal[0] = FTransform::Identity;
			}
			{
				TManagedArray<FTransform>& MassToLocal =
					KinematicSphereCollection->RestCollection->template ModifyAttribute<FTransform>(
						TEXT("MassToLocal"), FTransformCollection::TransformGroup);
				check(MassToLocal.Num() == 1);
				MassToLocal[0] = FTransform::Identity;
			}
		}
	}

	// Check that we get collision events from two overlapping GC spheres
	GTEST_TEST(AllTraits, GeometryCollection_CollisionEventsTest)
	{
		// simplicial sphere to implicit sphere
		FFramework UnitTest;

		CollisionEventsTests::AddSpheres(UnitTest);

		UnitTest.Initialize();

		// Listen to collision events
		struct FTestCollisionHandler
		{
			void OnCollision(const Chaos::FCollisionEventData& CollisionEventData)
			{
				EXPECT_EQ(CollisionEventData.CollisionData.AllCollisionsArray.Num(), 1);
			};
		};
		FTestCollisionHandler CollisionHandler;
		Chaos::FEventManager* EventManager = UnitTest.Solver->GetEventManager();
		EventManager->RegisterHandler<Chaos::FCollisionEventData>(Chaos::EEventType::Collision, &CollisionHandler, &FTestCollisionHandler::OnCollision);

		FSolverCollisionFilterSettings CollisionFilterSettings;
		CollisionFilterSettings.FilterEnabled = true;
		UnitTest.Solver->SetCollisionFilterSettings(CollisionFilterSettings);

		// Should get one collision event between the two spheres
		UnitTest.Advance();
	}


	// Test collision events in async physics mode
	GTEST_TEST(AllTraits, GeometryCollection_CollisionEventsAsyncTest)
	{
		// simplicial sphere to implicit sphere
		FrameworkParameters UnitTestParams;
		UnitTestParams.ThreadingMode = Chaos::EThreadingMode::DedicatedThread;
		FFramework UnitTest;

		CollisionEventsTests::AddSpheres(UnitTest);

		UnitTest.Initialize();

		// Listen to collision events
		struct FTestCollisionHandler
		{
			void OnCollision(const Chaos::FCollisionEventData& CollisionEventData)
			{
				EXPECT_EQ(CollisionEventData.CollisionData.AllCollisionsArray.Num(), 1);
			};
		};
		FTestCollisionHandler CollisionHandler;
		Chaos::FEventManager* EventManager = UnitTest.Solver->GetEventManager();
		EventManager->RegisterHandler<Chaos::FCollisionEventData>(Chaos::EEventType::Collision, &CollisionHandler, &FTestCollisionHandler::OnCollision);

		FSolverCollisionFilterSettings CollisionFilterSettings;
		CollisionFilterSettings.FilterEnabled = true;
		UnitTest.Solver->SetCollisionFilterSettings(CollisionFilterSettings);

		// Should get one collision event between the two spheres
		UnitTest.Advance();
	}

	// Destroy a particle while async physics is running. We will still get a collision event but the proxy will
	// have been marked for destruction. The user must check GetMarkedDeleted() before using the proxy, especially
	// if the owing game-side object might have been destroyed (which will happen if GC occurs at the right time).
	GTEST_TEST(AllTraits, GeometryCollection_CollisionEventsAsyncDestroyTest)
	{
		// simplicial sphere to implicit sphere
		FrameworkParameters UnitTestParams;
		UnitTestParams.ThreadingMode = Chaos::EThreadingMode::DedicatedThread;
		UnitTestParams.AsyncDt = 1.0 / 60.0;
		FFramework UnitTest;

		CollisionEventsTests::AddSpheres(UnitTest);

		UnitTest.Initialize();

		WrapperBase* DestroyedWrapper = nullptr;

		// Listen to collision events
		struct FTestCollisionHandler
		{
			// This callback will be called twice.
			// - the first time between the two active particles
			// - the second time between the same two particles, but after the 
			//   object has been "destroyed"
			void OnCollision(const Chaos::FCollisionEventData& CollisionEventData)
			{
				// We should get one collision between the two spheres
				EXPECT_EQ(CollisionEventData.CollisionData.AllCollisionsArray.Num(), 1);
				if (CollisionEventData.CollisionData.AllCollisionsArray.Num() == 1)
				{
					// After the dynamic particle is destroyed, we get a collision with 
					// the dynamic proxy's MarkeDeleted flag set. The user can check this
					// to know that the game-side object may have been destroyed
					const FCollidingData& Collision = CollisionEventData.CollisionData.AllCollisionsArray[0];
					EXPECT_EQ(Collision.Proxy1->GetMarkedDeleted(), bIsDeleted);
				}
			};

			bool bIsDeleted = false;
		};
		FTestCollisionHandler CollisionHandler;
		Chaos::FEventManager* EventManager = UnitTest.Solver->GetEventManager();
		EventManager->RegisterHandler<Chaos::FCollisionEventData>(Chaos::EEventType::Collision, &CollisionHandler, &FTestCollisionHandler::OnCollision);

		FSolverCollisionFilterSettings CollisionFilterSettings;
		CollisionFilterSettings.FilterEnabled = true;
		UnitTest.Solver->SetCollisionFilterSettings(CollisionFilterSettings);

		// Run one tick - should get one collision event between the two spheres
		CollisionHandler.bIsDeleted = false;
		UnitTest.Advance();

		// Runa second tick, but remove the dynamic sphere mid-tick,
		// after collision resolution, but before event dispatch
		UnitTest.Solver->AdvanceAndDispatch_External(UnitTestParams.Dt);
		UnitTest.Solver->UpdateGameThreadStructures();

		// Unregister the GC and destroy the particle owner (WrapperBase).
		// The proxy will be destroyed on the physics thread in a subsequent tick,
		// but the user data will still reference the deleted owner.
		DestroyedWrapper = UnitTest.PhysicsObjects[0];
		UnitTest.RemoveSimulationObject(DestroyedWrapper);
		
		// Dispatch events - should get one collision event between the two
		// sphere, but the dynamic sphere is marked as deleted
		CollisionHandler.bIsDeleted = true;
		UnitTest.Solver->SyncEvents_GameThread();
	}
}