// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestCollisionResolution.h"
#include "GeometryCollection/GeometryCollectionTestFramework.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollectionProxyData.h"
#include "HeadlessChaosTestUtility.h"

namespace GeometryCollectionTest
{
	using namespace ChaosTest;
	GTEST_TEST(AllTraits, GeometryCollection_CollisionResolutionTest)
	{
		FFramework UnitTest;

		FGeometryCollectionWrapper* Collection = nullptr;
		{
			FVector GlobalTranslation(0, 0, 10); 
			FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(0));
			
			CreationParameters Params; 
			Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic; 
			Params.EnableClustering = false;
			Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;  
			Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere; 
			Params.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;
			Params.RootTransform = FTransform(GlobalRotation, GlobalTranslation); 
			Params.NestedTransforms = { FTransform::Identity, FTransform::Identity, FTransform::Identity };
			
			Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

			const TManagedArray<FTransform>& MassToLocal = Collection->RestCollection->GetAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);

			EXPECT_EQ(Collection->DynamicCollection->GetParent(0), 1); // is a child of index one
			EXPECT_TRUE(MassToLocal[0].Equals(FTransform::Identity)); // we are not testing MassToLocal in this test
			
			

			UnitTest.AddSimulationObject(Collection);
		}

		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();
		UnitTest.AddSimulationObject(Floor);

		UnitTest.Initialize();
		for (int i = 0; i < 10000; i++) 
		{
			UnitTest.Advance();
		}
		{
			// validate that Simplicials are null when CollisionType==Chaos_Volumetric
			EXPECT_EQ(Collection->DynamicCollection->GetNumTransforms(), 4);
			EXPECT_EQ(UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().CollisionParticles(0), nullptr);

			const FReal MaxRestingSeparation = -UnitTest.Solver->GetEvolution()->GetGravityForces().GetAcceleration(0).Z * UnitTest.Dt * UnitTest.Dt;	// PBD resting separation will be up to this
			EXPECT_LT(FMath::Abs(Collection->RestCollection->Transform[0].GetTranslation().Z)-10.f, KINDA_SMALL_NUMBER);
			EXPECT_LT(FMath::Abs(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z - 1.0), 0.1 + MaxRestingSeparation);
		}
	}

	GTEST_TEST(AllTraits,GeometryCollection_CollisionResolution_SimplicialSphereToPlane)
	{
		FFramework UnitTest;
		const FReal Radius = 100.0f; // cm

		FGeometryCollectionWrapper* Collection = nullptr;
		{
			
			FVector GlobalTranslation(0, 0, Radius + 10); FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(0));
			CreationParameters Params; Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic; Params.EnableClustering = false;
			Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;  Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere; Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
			Params.RootTransform = FTransform(GlobalRotation, GlobalTranslation); Params.NestedTransforms = { FTransform::Identity, FTransform::Identity, FTransform::Identity };
			FVector Scale(Radius);
			Params.GeomTransform.SetScale3D(Scale); // Sphere radius
			Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

			const TManagedArray<FTransform>& MassToLocal = Collection->RestCollection->GetAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);

			EXPECT_EQ(Collection->DynamicCollection->GetParent(0), 1); // is a child of index one
			EXPECT_TRUE(MassToLocal[0].Equals(FTransform::Identity)); // we are not testing MassToLocal in this test

			UnitTest.AddSimulationObject(Collection);
		}

		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();
		UnitTest.AddSimulationObject(Floor);

		UnitTest.Initialize();
		for (int i = 0; i < 1000; i++)
		{
			UnitTest.Advance();
		}
		{
			using FSimplicialPtr = TUniquePtr<FCollisionStructureManager::FSimplicial>;
			TManagedArray<FSimplicialPtr>* Simplicials = Collection->RestCollection->FindAttribute<FSimplicialPtr>(FGeometryDynamicCollection::SimplicialsAttribute, FGeometryCollection::TransformAttribute);
			check(Simplicials);

			// validate that Simplicials are null when CollisionType==Chaos_Volumetric
			EXPECT_EQ(Collection->DynamicCollection->GetNumTransforms(), 4);
			EXPECT_TRUE((*Simplicials)[0]!=nullptr);
			EXPECT_TRUE(UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().CollisionParticles(0)!=nullptr);
			EXPECT_TRUE((*Simplicials)[0]->Size() == UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().CollisionParticles(0)->Size());
			EXPECT_TRUE((*Simplicials)[0]->Size() != 0);

			EXPECT_LT(FMath::Abs(Collection->RestCollection->Transform[0].GetTranslation().Z - (Radius + 10.f)), KINDA_SMALL_NUMBER);
			EXPECT_NEAR(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, Radius, KINDA_SMALL_NUMBER); 
		}
	}



	GTEST_TEST(AllTraits,GeometryCollection_CollisionResolution_AnalyticSphereToAnalyticSphere)
	{
		// simplicial sphere to implicit sphere
		FFramework UnitTest;

		CreationParameters Params;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;

		Params.EnableClustering = false;

		FVector Scale(1.f);
		Params.GeomTransform.SetScale3D(Scale); // Sphere radius
		Params.EnableClustering = false;

		// Make a dynamic simplicial sphere
		Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere;
		//Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_None; // Fails, falls right through
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;

		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.RootTransform =
			FTransform(FQuat::MakeFromEuler(FVector(0)), FVector(0, 0, 3.0));
		FGeometryCollectionWrapper* SimplicialSphereCollection =
			TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(SimplicialSphereCollection);

		// Make a kinematic implicit sphere
		Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;

		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Kinematic;
		Params.RootTransform =
			FTransform(FQuat::MakeFromEuler(FVector(0)), FVector(0));
		FGeometryCollectionWrapper* ImplicitSphereCollection =
			TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(ImplicitSphereCollection);

		// Hard code masstolocal on rest collection to identity
		{
			TManagedArray<FTransform>& MassToLocal =
				SimplicialSphereCollection->RestCollection->template ModifyAttribute<FTransform>(
					TEXT("MassToLocal"), FTransformCollection::TransformGroup);
			check(MassToLocal.Num() == 1);
			MassToLocal[0] = FTransform::Identity;
		}
		{
			TManagedArray<FTransform>& MassToLocal =
				ImplicitSphereCollection->RestCollection->template ModifyAttribute<FTransform>(
					TEXT("MassToLocal"), FTransformCollection::TransformGroup);
			check(MassToLocal.Num() == 1);
			MassToLocal[0] = FTransform::Identity;
		}

		UnitTest.Initialize();
		EXPECT_EQ(
			SimplicialSphereCollection->DynamicCollection->GetTransform(0).GetTranslation().Z,
			ImplicitSphereCollection->DynamicCollection->GetTransform(0).GetTranslation().Z + 3);

		const FVector FirstX = FVector(SimplicialSphereCollection->DynamicCollection->GetTransform(0).GetTranslation());
		FVector PrevX = FirstX;
		for (int i = 0; i < 10; i++)
		{
			UnitTest.Advance();

			const FVector CurrX = FVector(SimplicialSphereCollection->DynamicCollection->GetTransform(0).GetTranslation());
			EXPECT_NE(CurrX.Z, FirstX.Z); // moved since init
			EXPECT_GE(PrevX.Z - CurrX.Z, -KINDA_SMALL_NUMBER); // falling in -Z, or stopped
			EXPECT_LE(FMath::Abs(CurrX.X), KINDA_SMALL_NUMBER); // straight down
			EXPECT_LE(FMath::Abs(CurrX.Y), KINDA_SMALL_NUMBER); // straight down
			PrevX = CurrX;
		}

		{
			// We expect the simplical sphere to drop by 0.1 in Z and come to rest
			// on top of the implicit sphere.
			const FVector CurrX = FVector(SimplicialSphereCollection->DynamicCollection->GetTransform(0).GetTranslation());
			EXPECT_LE(CurrX.Z - 2.0, 0.2); // Relative large fudge factor accounts for aliasing?
		}
	}

	GTEST_TEST(AllTraits, DISABLED_GeometryCollection_CollisionResolution_AnalyticCubeToAnalyticCube)
	{

		// simplicial sphere to implicit sphere
		FFramework UnitTest;

		CreationParameters Params;
		Params.EnableClustering = false;

		FVector Scale(1.f, 1.f, 1.f);
		Params.GeomTransform.SetScale3D(Scale); // Box dimensions
		Params.EnableClustering = false;

		// Make a dynamic box
		Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Box;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;

		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.RootTransform = FTransform(FQuat::MakeFromEuler(FVector(0)), FVector(0, 0, 3.0));
		FGeometryCollectionWrapper* BoxCollection0 =
			TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(BoxCollection0);

		// Make a kinematic box
		Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Box;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;

		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Kinematic;
		Params.RootTransform = FTransform(FQuat::MakeFromEuler(FVector(0)), FVector(0));
		FGeometryCollectionWrapper* BoxCollection1 =
			TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(BoxCollection1);
/*
		// Hard code masstolocal on rest collection to identity
		{
			TManagedArray<FTransform>& MassToLocal =
				BoxCollection0->RestCollection->GetAttribute<FTransform>(
					TEXT("MassToLocal"), FTransformCollection::TransformGroup);
			check(MassToLocal.Num() == 1);
			MassToLocal[0] = FTransform::Identity;
		}
		{
			TManagedArray<FTransform>& MassToLocal =
				BoxCollection1->RestCollection->GetAttribute<FTransform>(
					TEXT("MassToLocal"), FTransformCollection::TransformGroup);
			check(MassToLocal.Num() == 1);
			MassToLocal[0] = FTransform::Identity;
		}
*/
		UnitTest.Initialize();
		EXPECT_EQ(
			BoxCollection0->DynamicCollection->GetTransform(0).GetTranslation().Z,
			BoxCollection1->DynamicCollection->GetTransform(0).GetTranslation().Z + 3);

		const FVector FirstX = FVector(BoxCollection0->DynamicCollection->GetTransform(0).GetTranslation());
		FVector PrevX = FirstX;
		for (int i = 0; i < 10; i++)
		{
			UnitTest.Advance();

			const FVector& CurrX = FVector(BoxCollection0->DynamicCollection->GetTransform(0).GetTranslation());
			EXPECT_NE(CurrX.Z, FirstX.Z); // moved since init
			EXPECT_LE(CurrX.Z, PrevX.Z); // falling in -Z, or stopped
			EXPECT_LE(FMath::Abs(CurrX.X), KINDA_SMALL_NUMBER); // No deflection
			EXPECT_LE(FMath::Abs(CurrX.Y), KINDA_SMALL_NUMBER); // No deflection
			PrevX = CurrX;
		}

		{
			// We expect the simplical sphere to drop by 0.1 in Z and come to rest
			// on top of the implicit sphere.
			const FVector& CurrX = FVector(BoxCollection0->DynamicCollection->GetTransform(0).GetTranslation());
			EXPECT_LE(CurrX.Z - 2.0, 0.2); // Relative large fudge factor accounts for aliasing?
		}

	}



	GTEST_TEST(AllTraits,GeometryCollection_CollisionResolution_SimplicialSphereToAnalyticSphere)
	{
		FFramework UnitTest;
		// This should exercise CollisionResolution::ConstructLevelsetLevelsetConstraints(...) with ispc:SampleSphere* (Paticle to Analytic Sphere)

		FGeometryCollectionWrapper* Collection = nullptr;
		{
			FVector GlobalTranslation(0, 0, 10); 
			FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(0));

			CreationParameters Params; 
			Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic; 
			Params.EnableClustering = false;
			Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_LevelSet;
			Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere;
			Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
			Params.RootTransform = FTransform(GlobalRotation, GlobalTranslation); 
			Params.NestedTransforms = { FTransform::Identity, FTransform::Identity, FTransform::Identity };

			Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
			
			const TManagedArray<FTransform>& MassToLocal = Collection->RestCollection->GetAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);

			EXPECT_EQ(Collection->DynamicCollection->GetParent(0), 1); // is a child of index one
			EXPECT_TRUE(MassToLocal[0].Equals(FTransform::Identity)); // we are not testing MassToLocal in this test

			UnitTest.AddSimulationObject(Collection);
		}

		FGeometryCollectionWrapper* CollectionStaticSphere = nullptr;
		{
			FVector GlobalTranslation(0, 0, 0);
			FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(0));
			CreationParameters Params;
			Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Static; 
			Params.EnableClustering = false;
			Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
			Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere;
			Params.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;
			Params.RootTransform = FTransform(GlobalRotation, GlobalTranslation); Params.NestedTransforms = { FTransform::Identity, FTransform::Identity, FTransform::Identity };
			CollectionStaticSphere = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
			
			const TManagedArray<FTransform>& MassToLocal = Collection->RestCollection->GetAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);
			
			EXPECT_EQ(CollectionStaticSphere->DynamicCollection->GetParent(0), 1); // is a child of index one
			EXPECT_TRUE(MassToLocal[0].Equals(FTransform::Identity)); // we are not testing MassToLocal in this test

			UnitTest.AddSimulationObject(CollectionStaticSphere);
		}

		UnitTest.Initialize();

		for (int i = 0; i < 20; i++)
		{
			UnitTest.Advance();
		}
		{
			using FSimplicialPtr = TUniquePtr<FCollisionStructureManager::FSimplicial>;
			TManagedArray<FSimplicialPtr>* Simplicials = Collection->RestCollection->FindAttribute<FSimplicialPtr>(FGeometryDynamicCollection::SimplicialsAttribute, FGeometryCollection::TransformAttribute);
			check(Simplicials);

			// validate simplicials and implicits are configured correctly
			EXPECT_EQ(Collection->DynamicCollection->GetNumTransforms(), 4);
			EXPECT_TRUE((*Simplicials)[0] != nullptr);
			EXPECT_TRUE(UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().CollisionParticles(0) != nullptr);
			EXPECT_TRUE((*Simplicials)[0]->Size() == UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().CollisionParticles(0)->Size());
			EXPECT_TRUE((*Simplicials)[0]->Size() != 0);
			// The following test has been disabled because from now we remove the Implicits after initialization to free up some memory
			// EXPECT_TRUE(Collection->DynamicCollection->GetAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup)[0]->GetType() == (int32)Chaos::ImplicitObjectType::LevelSet);

			EXPECT_EQ(CollectionStaticSphere->DynamicCollection->GetNumTransforms(), 4);
			EXPECT_TRUE(UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().CollisionParticles(1) == nullptr);
			// The following test has been disabled because from now we remove the Implicits after initialization to free up some memory
			// EXPECT_TRUE(CollectionStaticSphere->DynamicCollection->GetAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup)[0]->GetType() == (int32)Chaos::ImplicitObjectType::Sphere);

			// validate the ball collides and moved away from the static ball
			EXPECT_LT(FMath::Abs(Collection->RestCollection->Transform[0].GetTranslation().Z) - 10.f, KINDA_SMALL_NUMBER);
			EXPECT_TRUE(FMath::Abs(Collection->DynamicCollection->GetTransform(0).GetTranslation().X) < 0.001); // No deflection
			EXPECT_TRUE(FMath::Abs(Collection->DynamicCollection->GetTransform(0).GetTranslation().Y) < 0.001); // No deflection
			EXPECT_LT(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, 2.1f); // ball fell
		}
	}


	GTEST_TEST(AllTraits,GeometryCollection_CollisionResolution_SimplicialSphereToImplicitSphere)
	{
		
		// simplicial sphere to implicit sphere
		FFramework UnitTest;

		CreationParameters Params; 
		Params.EnableClustering = false;

		FReal Radius = 100.0f;
		FVector Scale(Radius);
		Params.GeomTransform.SetScale3D(Scale); // Sphere radius
		Params.EnableClustering = false;

		// Make a dynamic simplicial sphere
		Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.RootTransform =FTransform(FQuat::MakeFromEuler(FVector(0)), FVector(0, 0, 2.0f * Radius + 1.0));
		FGeometryCollectionWrapper* SimplicialSphereCollection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(SimplicialSphereCollection);

		// Make a kinematic implicit sphere
		Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_LevelSet;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Kinematic;
		Params.RootTransform = FTransform(FQuat::MakeFromEuler(FVector(0)), FVector(0));
		FGeometryCollectionWrapper* ImplicitSphereCollection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(ImplicitSphereCollection);

		// Hard code masstolocal on rest collection to identity
		{
			TManagedArray<FTransform>& MassToLocal = SimplicialSphereCollection->RestCollection->template ModifyAttribute<FTransform>(TEXT("MassToLocal"), FTransformCollection::TransformGroup);
			check(MassToLocal.Num() == 1);
			MassToLocal[0] = FTransform::Identity;
		}
		{
			TManagedArray<FTransform>& MassToLocal =ImplicitSphereCollection->RestCollection->template ModifyAttribute<FTransform>(TEXT("MassToLocal"), FTransformCollection::TransformGroup);
			check(MassToLocal.Num() == 1);
			MassToLocal[0] = FTransform::Identity;
		}

		UnitTest.Initialize();
		EXPECT_EQ(SimplicialSphereCollection->DynamicCollection->GetTransform(0).GetTranslation().Z, ImplicitSphereCollection->DynamicCollection->GetTransform(0).GetTranslation().Z + 2.0f * Radius + 1.0f);

		const FVector FirstX = FVector(SimplicialSphereCollection->DynamicCollection->GetTransform(0).GetTranslation());
		FVector PrevX = FirstX;
		for (int i = 0; i < 10; i++)
		{
			UnitTest.Advance();

			const FVector& CurrX = FVector(SimplicialSphereCollection->DynamicCollection->GetTransform(0).GetTranslation());
			EXPECT_NE(CurrX.Z, FirstX.Z); // moved since init
			EXPECT_LE(FMath::Abs(CurrX.X), 0.1f); // straight down
			EXPECT_LE(FMath::Abs(CurrX.Y), 0.1f); // straight down
			PrevX = CurrX;
		}
		
		{
			// We expect the simplical sphere to drop by 0.1 in Z and come to rest
			// on top of the implicit sphere.
			const FVector& CurrX = FVector(SimplicialSphereCollection->DynamicCollection->GetTransform(0).GetTranslation());
			EXPECT_LE(FMath::Abs(CurrX.Z - 2.0f * Radius), 0.1 * Radius);
		}
	}



	GTEST_TEST(AllTraits,GeometryCollection_CollisionResolution_SimplicialCubeToImplicitCube)
	{

		// simplicial sphere to implicit sphere
		FFramework UnitTest;

		CreationParameters Params;
		Params.EnableClustering = false;
		FReal Length = 100.0f;
		FVector Scale(Length, Length, Length);
		Params.GeomTransform.SetScale3D(Scale); // Box dimensions
		Params.EnableClustering = false;

		// Make a dynamic box
		Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Box;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;

		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.RootTransform = FTransform(FQuat::MakeFromEuler(FVector(0)), FVector(0, 0, Length + 2.0f));
		FGeometryCollectionWrapper* BoxCollection0 =
			TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(BoxCollection0);

		// Make a kinematic box
		Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Box;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_LevelSet;

		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Kinematic;
		Params.RootTransform = FTransform(FQuat::MakeFromEuler(FVector(0)), FVector(0));
		FGeometryCollectionWrapper* BoxCollection1 =
			TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(BoxCollection1);
/*
		// Hard code masstolocal on rest collection to identity
		{
			TManagedArray<FTransform>& MassToLocal =
				BoxCollection0->RestCollection->GetAttribute<FTransform>(
					TEXT("MassToLocal"), FTransformCollection::TransformGroup);
			check(MassToLocal.Num() == 1);
			MassToLocal[0] = FTransform::Identity;
		}
		{
			TManagedArray<FTransform>& MassToLocal =
				BoxCollection1->RestCollection->GetAttribute<FTransform>(
					TEXT("MassToLocal"), FTransformCollection::TransformGroup);
			check(MassToLocal.Num() == 1);
			MassToLocal[0] = FTransform::Identity;
		}
*/
		UnitTest.Initialize();
		EXPECT_EQ(
			BoxCollection0->DynamicCollection->GetTransform(0).GetTranslation().Z,
			BoxCollection1->DynamicCollection->GetTransform(0).GetTranslation().Z + Length + 2.0f);

		const FVector FirstX = FVector(BoxCollection0->DynamicCollection->GetTransform(0).GetTranslation());
		FVector PrevX = FirstX;
		for (int i = 0; i < 10; i++)
		{
			UnitTest.Advance();

			const FVector& CurrX = FVector(BoxCollection0->DynamicCollection->GetTransform(0).GetTranslation());
			EXPECT_NE(CurrX.Z, FirstX.Z); // moved since init
			EXPECT_LE(CurrX.Z, PrevX.Z); // falling in -Z, or stopped
			EXPECT_LE(FMath::Abs(CurrX.X), KINDA_SMALL_NUMBER); // straight down
			EXPECT_LE(FMath::Abs(CurrX.Y), KINDA_SMALL_NUMBER); // straight down
			PrevX = CurrX;
		}

		{
			// We expect the simplical cube to drop in Z direction and come to rest
			// on top of the implicit cube.
			const FVector& CurrX = FVector(BoxCollection0->DynamicCollection->GetTransform(0).GetTranslation());
			EXPECT_LE(FMath::Abs(CurrX.Z - Length), 0.2 * Length); // Relative large fudge factor accounts for spatial aliasing and contact location averaging.
		}
	}





	GTEST_TEST(AllTraits,GeometryCollection_CollisionResolution_SimplicialTetrahedronWithNonUniformMassToFloor)
	{
		FFramework UnitTest;

		FReal  Scale = 100.0f;

		FGeometryCollectionWrapper* Collection = nullptr;
		{
			FVector GlobalTranslation(0, 0, Scale + 10); FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(0));
			CreationParameters Params;
			Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
			Params.EnableClustering = false;
			Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_LevelSet;
			Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Tetrahedron;
			Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
			Params.GeomTransform = FTransform(GlobalRotation, GlobalTranslation);
			FVector TetraHedronScale(Scale);
			Params.GeomTransform.SetScale3D(TetraHedronScale); // Tetrahedron dimensions
			Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

			const TManagedArray<FTransform>& MassToLocal = Collection->RestCollection->GetAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);

			EXPECT_EQ(Collection->DynamicCollection->GetParent(0), -1); // is a child of index one
			EXPECT_NEAR((MassToLocal[0].GetTranslation()-FVector(0,0,Scale + 10)).Size(),0,KINDA_SMALL_NUMBER);

			UnitTest.AddSimulationObject(Collection);
		}

		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();
		UnitTest.AddSimulationObject(Floor);

		UnitTest.Initialize();

		for (int i = 0; i < 10; i++)
		{
			UnitTest.Advance();
		}
		{
			// Expected resting distance depends on the collision solver implementation.
			const FReal ExpectedRestingDistance = 0.0f;
			//const FReal ExpectedRestingDistance = UnitTest.Solver->GetEvolution()->GetGravityForces().GetAcceleration().Size() * UnitTest.Dt * UnitTest.Dt;

			// The error in the resting distance depends on the number of pushout iterations, which is very low by default
			// It also depends on whether manifolds are used for collision (no manifolds means larger errors)
			const FReal RestingDistanceTolerance = 2.0f;

			// validate the tetahedron collides and moved away from the static floor
			FVec3 RestTranslation = Collection->RestCollection->Transform[0].GetTranslation();
			FVec3 DynamicTranslation = Collection->DynamicCollection->GetTransform(0).GetTranslation();
			EXPECT_EQ(RestTranslation.Z, 0.f);
			EXPECT_NEAR(FMath::Abs(DynamicTranslation.X), 0.f, RestingDistanceTolerance);
			EXPECT_NEAR(FMath::Abs(DynamicTranslation.Y), 0.f, RestingDistanceTolerance);
			EXPECT_NEAR(DynamicTranslation.Z, -10.f + ExpectedRestingDistance, RestingDistanceTolerance);
		}
	}



} // namespace GeometryCollectionTest
