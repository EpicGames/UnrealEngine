// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestCreation.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/TransformCollection.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

DEFINE_LOG_CATEGORY_STATIC(GCTCR_Log, Verbose, All);


namespace GeometryCollectionTest
{
	void  CheckClassTypes()
	{
		FManagedArrayCollection ArrayCollection;
		EXPECT_TRUE(ArrayCollection.IsA<FManagedArrayCollection>());
		FManagedArrayCollection* Result = ArrayCollection.Cast<FManagedArrayCollection>();
		EXPECT_TRUE(ArrayCollection.Cast<FManagedArrayCollection>()!=nullptr);
		EXPECT_TRUE(ArrayCollection.Cast<FTransformCollection>()==nullptr);
		EXPECT_TRUE(ArrayCollection.Cast<FGeometryCollection>()==nullptr);

		FTransformCollection TransformCollection;
		EXPECT_TRUE(TransformCollection.IsA<FManagedArrayCollection>());
		EXPECT_TRUE(TransformCollection.Cast<FManagedArrayCollection>()!=nullptr);
		EXPECT_TRUE(TransformCollection.IsA<FTransformCollection>());
		EXPECT_TRUE(TransformCollection.Cast<FTransformCollection>()!=nullptr);
		EXPECT_TRUE(TransformCollection.Cast<FGeometryCollection>()==nullptr);

		FGeometryCollection GeometryCollection;
		EXPECT_TRUE(GeometryCollection.IsA<FManagedArrayCollection>());
		EXPECT_TRUE(GeometryCollection.Cast<FManagedArrayCollection>() != nullptr);
		EXPECT_TRUE(GeometryCollection.IsA<FTransformCollection>());
		EXPECT_TRUE(GeometryCollection.Cast<FTransformCollection>() != nullptr);
		EXPECT_TRUE(GeometryCollection.IsA<FGeometryCollection>());
		EXPECT_TRUE(GeometryCollection.Cast<FGeometryCollection>() != nullptr);
	}


	void CheckIncrementMask()
	{
		{
			TArray<int32> Mask;
			GeometryCollectionAlgo::BuildIncrementMask({ 2 }, 5, Mask);
			EXPECT_EQ(Mask[2], 0);
			EXPECT_EQ(Mask[3], 1);
		}
		{
			TArray<int32> Mask;
			GeometryCollectionAlgo::BuildIncrementMask({ 0 }, 5, Mask);
			EXPECT_EQ(Mask[0], 0);
			EXPECT_EQ(Mask[1], 1);
		}
		{
			TArray<int32> Mask;
			GeometryCollectionAlgo::BuildIncrementMask({ 1,2 }, 5, Mask);
			EXPECT_EQ(Mask[0], 0);
			EXPECT_EQ(Mask[1], 0);
			EXPECT_EQ(Mask[2], 1);
			EXPECT_EQ(Mask[3], 2);
			EXPECT_EQ(Mask[4], 2);
		}
	}

	void Creation()
	{
		TSharedPtr<FGeometryCollection> Collection(new FGeometryCollection());

		GeometryCollection::SetupCubeGridExample(Collection);

		EXPECT_TRUE(Collection->HasGroup(FTransformCollection::TransformGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::VerticesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::FacesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::MaterialGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::GeometryGroup));

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 1000);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 8000);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 12000);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 1000);

		int HalfTheFaces = Collection->NumElements(FGeometryCollection::FacesGroup) / 2;
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, HalfTheFaces * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));

		EXPECT_TRUE(Collection->HasContiguousFaces());
		EXPECT_TRUE(Collection->HasContiguousVertices());
		EXPECT_TRUE(Collection->HasContiguousRenderFaces());
	}

	void Empty()
	{
		// Set up Collection, empty it, then set it back up again.
		// Rebuild Collection should be the same as the initial set up.
		
		TSharedPtr<FGeometryCollection> Collection(new FGeometryCollection());

		GeometryCollection::SetupCubeGridExample(Collection);

		Collection->Empty();

		EXPECT_TRUE(Collection->HasGroup(FTransformCollection::TransformGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::VerticesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::FacesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::MaterialGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::GeometryGroup));

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 0);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 0);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 0);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 0);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 0);

		GeometryCollection::SetupCubeGridExample(Collection);

		EXPECT_TRUE(Collection->HasGroup(FTransformCollection::TransformGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::VerticesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::FacesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::MaterialGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::GeometryGroup));

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 1000);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 8000);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 12000);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 1000);

		int HalfTheFaces = Collection->NumElements(FGeometryCollection::FacesGroup) / 2;
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, HalfTheFaces * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));

		EXPECT_TRUE(Collection->HasContiguousFaces());
		EXPECT_TRUE(Collection->HasContiguousVertices());
		EXPECT_TRUE(Collection->HasContiguousRenderFaces());

	}

	void AppendTransformHierarchy()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 90.)), FVector(0, 10, 0)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 90.)), FVector(0, 10, 0)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 90.)), FVector(0, 10, 0)), FVector(1.0)));

		TSharedPtr<FGeometryCollection> Collection2 = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 90.)), FVector(0, 10, 0)), FVector(1.0), 4);
		Collection2->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 90.)), FVector(0, 10, 0)), FVector(1.0), 4));
		Collection2->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 90.)), FVector(0, 10, 0)), FVector(1.0), 4));


		//  0
		//  ...1
		//  ......2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);
		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);
		(Collection->Parent)[2] = 1;

		//  0
		//  ...1
		//  ...2
		(Collection2->Parent)[0] = -1;
		(Collection2->Children)[0].Add(1);
		(Collection2->Parent)[1] = 0;
		(Collection2->Children)[0].Add(2);
		(Collection2->Parent)[2] = 0;

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 3);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 24);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 36);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 3);

		EXPECT_EQ(Collection2->NumElements(FTransformCollection::TransformGroup), 3);
		EXPECT_EQ(Collection2->NumElements(FGeometryCollection::VerticesGroup), 24);
		EXPECT_EQ(Collection2->NumElements(FGeometryCollection::FacesGroup), 36);
		EXPECT_EQ(Collection2->NumElements(FGeometryCollection::MaterialGroup), 4);
		EXPECT_EQ(Collection2->NumElements(FGeometryCollection::GeometryGroup), 3);

		Collection->AppendGeometry(*Collection2);

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 6);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 48);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 72);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 4); // union of the 2/4 materials
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 6);

		EXPECT_EQ(Collection->Parent[0], -1);
		EXPECT_EQ(Collection->Parent[1], 0);
		EXPECT_EQ(Collection->Parent[2], 1);
		EXPECT_EQ(Collection->Parent[3], -1);
		EXPECT_EQ(Collection->Parent[4], 3);
		EXPECT_EQ(Collection->Parent[5], 3);

		EXPECT_EQ(Collection->Children[0].Num(), 1);
		EXPECT_EQ(Collection->Children[1].Num(), 1);
		EXPECT_EQ(Collection->Children[2].Num(), 0);
		EXPECT_EQ(Collection->Children[3].Num(), 2);
		EXPECT_EQ(Collection->Children[4].Num(), 0);
		EXPECT_EQ(Collection->Children[5].Num(), 0);

		EXPECT_EQ(Collection->Children[0].Array()[0], 1);
		EXPECT_EQ(Collection->Children[1].Array()[0], 2);
		EXPECT_EQ(Collection->Children[3].Array()[0], 4);
		EXPECT_EQ(Collection->Children[3].Array()[1], 5);

		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, 18 + 9);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, (Collection->Sections)[0].FirstIndex + (Collection->Sections)[0].NumTriangles * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, 18 + 9);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[2].MaterialID, 2);
		EXPECT_EQ((Collection->Sections)[2].FirstIndex, (Collection->Sections)[1].FirstIndex + (Collection->Sections)[1].NumTriangles * 3);
		EXPECT_EQ((Collection->Sections)[2].NumTriangles, 9);
		EXPECT_EQ((Collection->Sections)[2].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[2].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[3].MaterialID, 3);
		EXPECT_EQ((Collection->Sections)[3].FirstIndex, (Collection->Sections)[2].FirstIndex + (Collection->Sections)[2].NumTriangles * 3);
		EXPECT_EQ((Collection->Sections)[3].NumTriangles, 9);
		EXPECT_EQ((Collection->Sections)[3].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[3].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		// GeometryGroup Tests
		{
			EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 6);

			EXPECT_EQ((Collection->TransformIndex)[0], 0);
			EXPECT_EQ((Collection->TransformIndex)[1], 1);
			EXPECT_EQ((Collection->TransformIndex)[2], 2);
			EXPECT_EQ((Collection->TransformIndex)[3], 3);
			EXPECT_EQ((Collection->TransformIndex)[4], 4);
			EXPECT_EQ((Collection->TransformIndex)[5], 5);

			EXPECT_EQ((Collection->TransformToGeometryIndex)[0], 0);
			EXPECT_EQ((Collection->TransformToGeometryIndex)[1], 1);
			EXPECT_EQ((Collection->TransformToGeometryIndex)[2], 2);
			EXPECT_EQ((Collection->TransformToGeometryIndex)[3], 3);
			EXPECT_EQ((Collection->TransformToGeometryIndex)[4], 4);
			EXPECT_EQ((Collection->TransformToGeometryIndex)[5], 5);

			EXPECT_EQ((Collection->FaceStart)[0], 0);
			EXPECT_EQ((Collection->FaceStart)[1], 12);
			EXPECT_EQ((Collection->FaceStart)[2], 24);
			EXPECT_EQ((Collection->FaceStart)[3], 36);
			EXPECT_EQ((Collection->FaceStart)[4], 48);
			EXPECT_EQ((Collection->FaceStart)[5], 60);

			EXPECT_EQ((Collection->FaceCount)[0], 12);
			EXPECT_EQ((Collection->FaceCount)[1], 12);
			EXPECT_EQ((Collection->FaceCount)[2], 12);
			EXPECT_EQ((Collection->FaceCount)[3], 12);
			EXPECT_EQ((Collection->FaceCount)[4], 12);
			EXPECT_EQ((Collection->FaceCount)[5], 12);
			EXPECT_EQ((Collection->Indices).Num(), 72);

			EXPECT_EQ((Collection->VertexStart)[0], 0);
			EXPECT_EQ((Collection->VertexStart)[1], 8);
			EXPECT_EQ((Collection->VertexStart)[2], 16);
			EXPECT_EQ((Collection->VertexStart)[3], 24);
			EXPECT_EQ((Collection->VertexStart)[4], 32);
			EXPECT_EQ((Collection->VertexStart)[5], 40);

			EXPECT_EQ((Collection->VertexCount)[0], 8);
			EXPECT_EQ((Collection->VertexCount)[1], 8);
			EXPECT_EQ((Collection->VertexCount)[2], 8);
			EXPECT_EQ((Collection->VertexCount)[3], 8);
			EXPECT_EQ((Collection->VertexCount)[4], 8);
			EXPECT_EQ((Collection->VertexCount)[5], 8);
			EXPECT_EQ((Collection->Vertex).Num(), 48);
		}

		EXPECT_TRUE(Collection->HasContiguousFaces());
		EXPECT_TRUE(Collection->HasContiguousVertices());
		EXPECT_TRUE(Collection->HasContiguousRenderFaces());
		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));
	}

	void ContiguousElementsTest()
	{
		{
			TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform::Identity, FVector(1.0));
			EXPECT_TRUE(Collection->HasContiguousFaces());
			EXPECT_TRUE(Collection->HasContiguousVertices());
			Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
			EXPECT_TRUE(Collection->HasContiguousFaces());
			EXPECT_TRUE(Collection->HasContiguousVertices());
			Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
			EXPECT_TRUE(Collection->HasContiguousFaces());
			EXPECT_TRUE(Collection->HasContiguousVertices());
		}
		{
			TSharedPtr<FGeometryCollection> Collection(new FGeometryCollection());
			GeometryCollection::SetupCubeGridExample(Collection);
			EXPECT_TRUE(Collection->HasContiguousFaces());
			EXPECT_TRUE(Collection->HasContiguousVertices());
		}
	}

	void DeleteFromEnd()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform::Identity, FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 20)), FVector(1.0)));

		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);

		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);

		(Collection->Parent)[2] = 1;

		EXPECT_TRUE(Collection->HasGroup(FTransformCollection::TransformGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::VerticesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::FacesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::MaterialGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::GeometryGroup));

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 3);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 24);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 36);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 3);

		int HalfTheFaces = Collection->NumElements(FGeometryCollection::FacesGroup) / 2;
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, HalfTheFaces * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 3);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], 0);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], 1);
		EXPECT_EQ(Collection->TransformToGeometryIndex[2], 2);

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));

		TArray<int32> DelList = { 2 };
		Collection->RemoveElements(FTransformCollection::TransformGroup, DelList);

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 2);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 16);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 24);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 2);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], 0);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], 1);

		for (int Index = 0; Index < Collection->NumElements(FGeometryCollection::VerticesGroup); Index++)
		{
			EXPECT_TRUE((Collection->BoneMap)[Index] < Collection->NumElements(FGeometryCollection::TransformGroup));
		}
		for (int Index = 0; Index < Collection->NumElements(FGeometryCollection::FacesGroup); Index++)
		{
			EXPECT_TRUE((Collection->Indices)[Index][0] < Collection->NumElements(FGeometryCollection::VerticesGroup));
			EXPECT_TRUE((Collection->Indices)[Index][1] < Collection->NumElements(FGeometryCollection::VerticesGroup));
			EXPECT_TRUE((Collection->Indices)[Index][2] < Collection->NumElements(FGeometryCollection::VerticesGroup));
		}

		EXPECT_EQ((Collection->Transform)[0].GetTranslation().Z, 0.f);
		EXPECT_EQ((Collection->Transform)[1].GetTranslation().Z, 10.f);

		HalfTheFaces = Collection->NumElements(FGeometryCollection::FacesGroup) / 2;
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, HalfTheFaces * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		// GeometryGroup Updated Tests
		{
			EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 2);

			EXPECT_EQ((Collection->TransformIndex)[0], 0);
			EXPECT_EQ((Collection->TransformIndex)[1], 1);

			EXPECT_EQ((Collection->FaceStart)[0], 0);
			EXPECT_EQ((Collection->FaceStart)[1], 12);

			EXPECT_EQ((Collection->FaceCount)[0], 12);
			EXPECT_EQ((Collection->FaceCount)[1], 12);
			EXPECT_EQ((Collection->Indices).Num(), 24);

			EXPECT_EQ((Collection->VertexStart)[0], 0);
			EXPECT_EQ((Collection->VertexStart)[1], 8);

			EXPECT_EQ((Collection->VertexCount)[0], 8);
			EXPECT_EQ((Collection->VertexCount)[1], 8);
			EXPECT_EQ((Collection->Vertex).Num(), 16);
		}

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));
		EXPECT_TRUE(Collection->HasContiguousFaces());
		EXPECT_TRUE(Collection->HasContiguousVertices());
		EXPECT_TRUE(Collection->HasContiguousRenderFaces());
	}

	void DeleteFromStart()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform::Identity, FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 20)), FVector(1.0)));

		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);

		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);

		(Collection->Parent)[2] = 1;

		EXPECT_TRUE(Collection->HasGroup(FTransformCollection::TransformGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::VerticesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::FacesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::MaterialGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::GeometryGroup));

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 3);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 24);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 36);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 3);

		int HalfTheFaces = Collection->NumElements(FGeometryCollection::FacesGroup) / 2;
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, HalfTheFaces * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 3);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], 0);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], 1);
		EXPECT_EQ(Collection->TransformToGeometryIndex[2], 2);

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));

		TArray<int32> DelList = { 0 };
		Collection->RemoveElements(FTransformCollection::TransformGroup, DelList);

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 2);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 16);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 24);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 2);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], 0);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], 1);

		for (int Index = 0; Index < Collection->NumElements(FGeometryCollection::VerticesGroup); Index++)
		{
			EXPECT_LT((Collection->BoneMap)[Index], Collection->NumElements(FGeometryCollection::TransformGroup));
		}
		for (int Index = 0; Index < Collection->NumElements(FGeometryCollection::FacesGroup); Index++)
		{
			EXPECT_LT((Collection->Indices)[Index][0], Collection->NumElements(FGeometryCollection::VerticesGroup));
			EXPECT_LT((Collection->Indices)[Index][1], Collection->NumElements(FGeometryCollection::VerticesGroup));
			EXPECT_LT((Collection->Indices)[Index][2], Collection->NumElements(FGeometryCollection::VerticesGroup));
		}

		EXPECT_EQ((Collection->Transform)[0].GetTranslation().Z, 10.f);
		EXPECT_EQ((Collection->Transform)[1].GetTranslation().Z, 20.f);

		HalfTheFaces = Collection->NumElements(FGeometryCollection::FacesGroup) / 2;
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, HalfTheFaces * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		// GeometryGroup Updated Tests
		{
			EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 2);

			EXPECT_EQ((Collection->TransformIndex)[0], 0);
			EXPECT_EQ((Collection->TransformIndex)[1], 1);

			EXPECT_EQ((Collection->FaceStart)[0], 0);
			EXPECT_EQ((Collection->FaceStart)[1], 12);

			EXPECT_EQ((Collection->FaceCount)[0], 12);
			EXPECT_EQ((Collection->FaceCount)[1], 12);
			EXPECT_EQ((Collection->Indices).Num(), 24);

			EXPECT_EQ((Collection->VertexStart)[0], 0);
			EXPECT_EQ((Collection->VertexStart)[1], 8);

			EXPECT_EQ((Collection->VertexCount)[0], 8);
			EXPECT_EQ((Collection->VertexCount)[1], 8);
			EXPECT_EQ((Collection->Vertex).Num(), 16);
		}

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));
		EXPECT_TRUE(Collection->HasContiguousFaces());
		EXPECT_TRUE(Collection->HasContiguousVertices());
		EXPECT_TRUE(Collection->HasContiguousRenderFaces());
	}

	void DeleteFromMiddle()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform::Identity, FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 20)), FVector(1.0)));

		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);

		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);

		(Collection->Parent)[2] = 1;

		EXPECT_TRUE(Collection->HasGroup(FTransformCollection::TransformGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::VerticesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::FacesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::MaterialGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::GeometryGroup));

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 3);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 24);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 36);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 3);

		int HalfTheFaces = Collection->NumElements(FGeometryCollection::FacesGroup) / 2;
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, HalfTheFaces * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 3);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], 0);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], 1);
		EXPECT_EQ(Collection->TransformToGeometryIndex[2], 2);

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));

		TArray<int32> DelList = { 1 };
		Collection->RemoveElements(FTransformCollection::TransformGroup, DelList);

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 2);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 16);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 24);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 2);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], 0);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], 1);

		for (int Index = 0; Index < Collection->NumElements(FGeometryCollection::VerticesGroup); Index++)
		{
			EXPECT_LT((Collection->BoneMap)[Index], Collection->NumElements(FGeometryCollection::TransformGroup));
		}
		for (int Index = 0; Index < Collection->NumElements(FGeometryCollection::FacesGroup); Index++)
		{
			EXPECT_LT((Collection->Indices)[Index][0], Collection->NumElements(FGeometryCollection::VerticesGroup));
			EXPECT_LT((Collection->Indices)[Index][1], Collection->NumElements(FGeometryCollection::VerticesGroup));
			EXPECT_LT((Collection->Indices)[Index][2], Collection->NumElements(FGeometryCollection::VerticesGroup));
		}

		EXPECT_EQ((Collection->Transform)[0].GetTranslation().Z, 0.f);
		EXPECT_EQ((Collection->Transform)[1].GetTranslation().Z, 30.f);

		HalfTheFaces = Collection->NumElements(FGeometryCollection::FacesGroup) / 2;
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, HalfTheFaces * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		// GeometryGroup Updated Tests
		{
			EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 2);

			EXPECT_EQ((Collection->TransformIndex)[0], 0);
			EXPECT_EQ((Collection->TransformIndex)[1], 1);

			EXPECT_EQ((Collection->FaceStart)[0], 0);
			EXPECT_EQ((Collection->FaceStart)[1], 12);

			EXPECT_EQ((Collection->FaceCount)[0], 12);
			EXPECT_EQ((Collection->FaceCount)[1], 12);
			EXPECT_EQ((Collection->Indices).Num(), 24);

			EXPECT_EQ((Collection->VertexStart)[0], 0);
			EXPECT_EQ((Collection->VertexStart)[1], 8);

			EXPECT_EQ((Collection->VertexCount)[0], 8);
			EXPECT_EQ((Collection->VertexCount)[1], 8);
			EXPECT_EQ((Collection->Vertex).Num(), 16);
		}

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));
		EXPECT_TRUE(Collection->HasContiguousFaces());
		EXPECT_TRUE(Collection->HasContiguousVertices());
		EXPECT_TRUE(Collection->HasContiguousRenderFaces());
	}

	void DeleteBranch()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform::Identity, FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));

		//  0
		//  ...1
		//  ......3
		//  ...2
		//  ......4
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);
		(Collection->Children)[0].Add(2);
		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(3);
		(Collection->Parent)[2] = 0;
		(Collection->Children)[2].Add(4);
		(Collection->Parent)[3] = 1;
		(Collection->Parent)[4] = 2;

		EXPECT_TRUE(Collection->HasGroup(FTransformCollection::TransformGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::VerticesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::FacesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::MaterialGroup));

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 5);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 40);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 60);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 5);

		int HalfTheFaces = Collection->NumElements(FGeometryCollection::FacesGroup) / 2;
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, HalfTheFaces * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 5);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], 0);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], 1);
		EXPECT_EQ(Collection->TransformToGeometryIndex[2], 2);
		EXPECT_EQ(Collection->TransformToGeometryIndex[3], 3);
		EXPECT_EQ(Collection->TransformToGeometryIndex[4], 4);

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));

		//  0
		//  ...2
		//  ......4
		TArray<int32> DelList = { 1, 3 };
		Collection->RemoveElements(FTransformCollection::TransformGroup, DelList);

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 3);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 24);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 36);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);

		EXPECT_EQ((Collection->Parent)[0], -1);
		EXPECT_EQ((Collection->Children)[0].Num(), 1);
		EXPECT_TRUE((Collection->Children)[0].Contains(1));
		EXPECT_EQ((Collection->Parent)[1], 0);
		EXPECT_EQ((Collection->Children)[1].Num(), 1);
		EXPECT_TRUE((Collection->Children)[1].Contains(2));
		EXPECT_EQ((Collection->Parent)[2], 1);
		EXPECT_EQ((Collection->Children)[2].Num(), 0);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 3);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], 0);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], 1);
		EXPECT_EQ(Collection->TransformToGeometryIndex[2], 2);

		for (int Index = 0; Index < Collection->NumElements(FGeometryCollection::VerticesGroup); Index++)
		{
			EXPECT_LT((Collection->BoneMap)[Index], Collection->NumElements(FGeometryCollection::TransformGroup));
		}
		for (int Index = 0; Index < Collection->NumElements(FGeometryCollection::FacesGroup); Index++)
		{
			EXPECT_LT((Collection->Indices)[Index][0], Collection->NumElements(FGeometryCollection::VerticesGroup));
			EXPECT_LT((Collection->Indices)[Index][1], Collection->NumElements(FGeometryCollection::VerticesGroup));
			EXPECT_LT((Collection->Indices)[Index][2], Collection->NumElements(FGeometryCollection::VerticesGroup));
		}

		EXPECT_EQ((Collection->Transform)[0].GetTranslation().Z, 0.f);
		EXPECT_EQ((Collection->Transform)[1].GetTranslation().Z, 10.f);

		HalfTheFaces = Collection->NumElements(FGeometryCollection::FacesGroup) / 2;
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, HalfTheFaces * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		// GeometryGroup Updated Tests
		{
			EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 3);

			EXPECT_EQ((Collection->TransformIndex)[0], 0);
			EXPECT_EQ((Collection->TransformIndex)[1], 1);
			EXPECT_EQ((Collection->TransformIndex)[2], 2);

			EXPECT_EQ((Collection->FaceStart)[0], 0);
			EXPECT_EQ((Collection->FaceStart)[1], 12);
			EXPECT_EQ((Collection->FaceStart)[2], 24);

			EXPECT_EQ((Collection->FaceCount)[0], 12);
			EXPECT_EQ((Collection->FaceCount)[1], 12);
			EXPECT_EQ((Collection->FaceCount)[2], 12);
			EXPECT_EQ((Collection->Indices).Num(), 36);

			EXPECT_EQ((Collection->VertexStart)[0], 0);
			EXPECT_EQ((Collection->VertexStart)[1], 8);
			EXPECT_EQ((Collection->VertexStart)[2], 16);

			EXPECT_EQ((Collection->VertexCount)[0], 8);
			EXPECT_EQ((Collection->VertexCount)[1], 8);
			EXPECT_EQ((Collection->VertexCount)[2], 8);
			EXPECT_EQ((Collection->Vertex).Num(), 24);
		}

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));
		EXPECT_TRUE(Collection->HasContiguousFaces());
		EXPECT_TRUE(Collection->HasContiguousVertices());
		EXPECT_TRUE(Collection->HasContiguousRenderFaces());
	}

	void DeleteRootLeafMiddle()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform::Identity, FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));

		//  0
		//  ...1
		//  ...5
		//  ......6
		//  ......3
		//  ...2
		//  ......7
		//  .........4
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);
		(Collection->Children)[0].Add(5);
		(Collection->Children)[0].Add(2);
		(Collection->Parent)[1] = 0;
		(Collection->Parent)[2] = 0;
		(Collection->Children)[2].Add(7);
		(Collection->Parent)[3] = 5;
		(Collection->Parent)[4] = 7;
		(Collection->Parent)[5] = 0;
		(Collection->Children)[5].Add(6);
		(Collection->Children)[5].Add(3);
		(Collection->Parent)[6] = 5;
		(Collection->Parent)[7] = 2;
		(Collection->Children)[7].Add(4);

		(Collection->BoneName)[0] = "0";
		(Collection->BoneName)[1] = "1";
		(Collection->BoneName)[2] = "2";
		(Collection->BoneName)[3] = "3";
		(Collection->BoneName)[4] = "4";
		(Collection->BoneName)[5] = "5";
		(Collection->BoneName)[6] = "6";
		(Collection->BoneName)[7] = "7";


		EXPECT_TRUE(Collection->HasGroup(FTransformCollection::TransformGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::VerticesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::FacesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::MaterialGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::GeometryGroup));

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 8);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 64);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 96);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 8);

		int HalfTheFaces = Collection->NumElements(FGeometryCollection::FacesGroup) / 2;
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, HalfTheFaces * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 8);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], 0);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], 1);
		EXPECT_EQ(Collection->TransformToGeometryIndex[2], 2);
		EXPECT_EQ(Collection->TransformToGeometryIndex[3], 3);
		EXPECT_EQ(Collection->TransformToGeometryIndex[4], 4);
		EXPECT_EQ(Collection->TransformToGeometryIndex[5], 5);
		EXPECT_EQ(Collection->TransformToGeometryIndex[6], 6);
		EXPECT_EQ(Collection->TransformToGeometryIndex[7], 7);

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));

		//  1
		//  6
		//  3
		//  2
		//  ...4
		TArray<int32> DelList = { 0,5,7 };
		Collection->RemoveElements(FTransformCollection::TransformGroup, DelList);

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 5);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 40);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 60);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 2);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 5);

		EXPECT_EQ((Collection->Parent)[0], -1);
		EXPECT_EQ((Collection->Children)[0].Num(), 0);
		EXPECT_EQ((Collection->Parent)[1], -1);
		EXPECT_EQ((Collection->Children)[1].Num(), 1);
		EXPECT_TRUE((Collection->Children)[1].Contains(3));
		EXPECT_EQ((Collection->Parent)[2], -1);
		EXPECT_EQ((Collection->Children)[2].Num(), 0);
		EXPECT_EQ((Collection->Parent)[3], 1);
		EXPECT_EQ((Collection->Children)[3].Num(), 0);
		EXPECT_EQ((Collection->Parent)[4], -1);
		EXPECT_EQ((Collection->Children)[4].Num(), 0);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 5);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], 0);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], 1);
		EXPECT_EQ(Collection->TransformToGeometryIndex[2], 2);
		EXPECT_EQ(Collection->TransformToGeometryIndex[3], 3);
		EXPECT_EQ(Collection->TransformToGeometryIndex[4], 4);

		int32 Index0 = Collection->BoneName.Find("0");
		int32 Index1 = Collection->BoneName.Find("1");
		int32 Index2 = Collection->BoneName.Find("2");
		int32 Index3 = Collection->BoneName.Find("3");
		int32 Index4 = Collection->BoneName.Find("4");
		int32 Index6 = Collection->BoneName.Find("6");

		EXPECT_EQ(Index0, -1);
		EXPECT_NE(Index6, -1);
		EXPECT_EQ((Collection->Parent)[Index1], -1);
		EXPECT_EQ((Collection->Parent)[Index2], -1);
		EXPECT_EQ((Collection->Children)[Index2].Num(), 1);
		EXPECT_TRUE((Collection->Children)[Index2].Contains(Index4));
		EXPECT_EQ((Collection->Parent)[Index4], Index2);
		EXPECT_EQ((Collection->Children)[Index4].Num(), 0);

		for (int Index = 0; Index < Collection->NumElements(FGeometryCollection::VerticesGroup); Index++)
		{
			EXPECT_LT((Collection->BoneMap)[Index], Collection->NumElements(FGeometryCollection::TransformGroup));
		}
		for (int Index = 0; Index < Collection->NumElements(FGeometryCollection::FacesGroup); Index++)
		{
			EXPECT_LT((Collection->Indices)[Index][0], Collection->NumElements(FGeometryCollection::VerticesGroup));
			EXPECT_LT((Collection->Indices)[Index][1], Collection->NumElements(FGeometryCollection::VerticesGroup));
			EXPECT_LT((Collection->Indices)[Index][2], Collection->NumElements(FGeometryCollection::VerticesGroup));
		}

		EXPECT_EQ((Collection->Transform)[Index1].GetTranslation().Z, 10.f);
		EXPECT_EQ((Collection->Transform)[Index2].GetTranslation().Z, 10.f);
		EXPECT_EQ((Collection->Transform)[Index3].GetTranslation().Z, 20.f);
		EXPECT_EQ((Collection->Transform)[Index4].GetTranslation().Z, 20.f);
		EXPECT_EQ((Collection->Transform)[Index6].GetTranslation().Z, 20.f);

		HalfTheFaces = Collection->NumElements(FGeometryCollection::FacesGroup) / 2;
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].FirstIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[0].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[0].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);

		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].FirstIndex, HalfTheFaces * 3);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, HalfTheFaces);
		EXPECT_EQ((Collection->Sections)[1].MinVertexIndex, 0);
		EXPECT_EQ((Collection->Sections)[1].MaxVertexIndex, Collection->NumElements(FGeometryCollection::VerticesGroup) - 1);


		// GeometryGroup Updated Tests
		{
			EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 5);


			EXPECT_EQ((Collection->TransformIndex)[Index1], 0);
			EXPECT_EQ((Collection->TransformIndex)[Index2], 1);
			EXPECT_EQ((Collection->TransformIndex)[Index3], 2);
			EXPECT_EQ((Collection->TransformIndex)[Index4], 3);
			EXPECT_EQ((Collection->TransformIndex)[Index6], 4);

			EXPECT_EQ((Collection->FaceStart)[Index1], 0);
			EXPECT_EQ((Collection->FaceStart)[Index2], 12);
			EXPECT_EQ((Collection->FaceStart)[Index3], 24);
			EXPECT_EQ((Collection->FaceStart)[Index4], 36);
			EXPECT_EQ((Collection->FaceStart)[Index6], 48);

			EXPECT_EQ((Collection->FaceCount)[Index1], 12);
			EXPECT_EQ((Collection->FaceCount)[Index2], 12);
			EXPECT_EQ((Collection->FaceCount)[Index3], 12);
			EXPECT_EQ((Collection->FaceCount)[Index4], 12);
			EXPECT_EQ((Collection->FaceCount)[Index6], 12);
			EXPECT_EQ((Collection->Indices).Num(), 60);

			EXPECT_EQ((Collection->VertexStart)[Index1], 0);
			EXPECT_EQ((Collection->VertexStart)[Index2], 8);
			EXPECT_EQ((Collection->VertexStart)[Index3], 16);
			EXPECT_EQ((Collection->VertexStart)[Index4], 24);
			EXPECT_EQ((Collection->VertexStart)[Index6], 32);

			EXPECT_EQ((Collection->VertexCount)[Index1], 8);
			EXPECT_EQ((Collection->VertexCount)[Index2], 8);
			EXPECT_EQ((Collection->VertexCount)[Index3], 8);
			EXPECT_EQ((Collection->VertexCount)[Index4], 8);
			EXPECT_EQ((Collection->VertexCount)[Index6], 8);
			EXPECT_EQ((Collection->Vertex).Num(), 40);
		}

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));
		EXPECT_TRUE(Collection->HasContiguousFaces());
		EXPECT_TRUE(Collection->HasContiguousVertices());
		EXPECT_TRUE(Collection->HasContiguousRenderFaces());
	}

	void DeleteEverything()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform::Identity, FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0, 0, 10)), FVector(1.0)));

		//  0
		//  ...1
		//  ...5
		//  ......6
		//  ......3
		//  ...2
		//  ......7
		//  .........4
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);
		(Collection->Children)[0].Add(5);
		(Collection->Children)[0].Add(2);
		(Collection->Parent)[1] = 0;
		(Collection->Parent)[2] = 0;
		(Collection->Children)[2].Add(7);
		(Collection->Parent)[3] = 5;
		(Collection->Parent)[4] = 7;
		(Collection->Parent)[5] = 0;
		(Collection->Children)[5].Add(6);
		(Collection->Children)[5].Add(3);
		(Collection->Parent)[6] = 5;
		(Collection->Parent)[7] = 2;
		(Collection->Children)[7].Add(4);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 8);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], 0);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], 1);
		EXPECT_EQ(Collection->TransformToGeometryIndex[2], 2);
		EXPECT_EQ(Collection->TransformToGeometryIndex[3], 3);
		EXPECT_EQ(Collection->TransformToGeometryIndex[4], 4);
		EXPECT_EQ(Collection->TransformToGeometryIndex[5], 5);
		EXPECT_EQ(Collection->TransformToGeometryIndex[6], 6);
		EXPECT_EQ(Collection->TransformToGeometryIndex[7], 7);

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));

		TArray<int32> DelList = { 0,1,2,3,4,5,6,7 };
		Collection->RemoveElements(FTransformCollection::TransformGroup, DelList);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 0);

		EXPECT_TRUE(Collection->HasGroup(FTransformCollection::TransformGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::VerticesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::FacesGroup));
		EXPECT_TRUE(Collection->HasGroup(FGeometryCollection::MaterialGroup));

		EXPECT_EQ(Collection->NumElements(FTransformCollection::TransformGroup), 0);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::VerticesGroup), 0);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::FacesGroup), 0);
		EXPECT_EQ(Collection->NumElements(FGeometryCollection::MaterialGroup), 0);

		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 0);
		EXPECT_EQ((Collection->Indices).Num(), 0);
		EXPECT_EQ((Collection->Vertex).Num(), 0);

		EXPECT_TRUE(GeometryCollectionAlgo::HasValidGeometryReferences(Collection.Get()));
		EXPECT_TRUE(Collection->HasContiguousFaces());
		EXPECT_TRUE(Collection->HasContiguousVertices());
		EXPECT_TRUE(Collection->HasContiguousRenderFaces());
	}

	void ParentTransformTest()
	{
		FGeometryCollection* Collection = new FGeometryCollection();

		int32 TransformIndex = Collection->AddElements(1, FGeometryCollection::TransformGroup);
		(Collection->Transform)[TransformIndex].SetTranslation(FVector3f(13));
		(Collection->Parent)[TransformIndex]= -1;
		EXPECT_EQ(TransformIndex, 0);

		TransformIndex = Collection->AddElements(1, FGeometryCollection::TransformGroup);
		(Collection->Transform)[TransformIndex].SetTranslation(FVector3f(7));
		(Collection->Parent)[TransformIndex] = -1;
		EXPECT_EQ(TransformIndex, 1);

		//
		// Parent a transform
		//
		GeometryCollectionAlgo::ParentTransform(Collection, 1, 0);
		EXPECT_EQ((Collection->Children)[0].Num(), 0);
		EXPECT_EQ((Collection->Parent)[0], 1);
		EXPECT_EQ((Collection->Children)[1].Num(), 1);
		EXPECT_TRUE((Collection->Children)[1].Contains(0));
		EXPECT_EQ((Collection->Parent)[1], -1);

		TArray<FTransform> GlobalTransform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransform);
		EXPECT_LT(((Collection->Transform)[0].GetTranslation() - FVector3f(6)).Size(), KINDA_SMALL_NUMBER);
		EXPECT_LT((GlobalTransform[0].GetTranslation()-FVector(13)).Size(), KINDA_SMALL_NUMBER);

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 2);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], -1);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], -1);

		//
		// Add some geometry
		//
		TransformIndex = Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(3)), FVector(1.0)));
		EXPECT_LT(((Collection->Transform)[TransformIndex].GetTranslation() - FVector3f(3)).Size(), KINDA_SMALL_NUMBER);
		EXPECT_EQ((Collection->TransformIndex).Num(), 1);
		EXPECT_EQ((Collection->TransformIndex)[0], TransformIndex);
		EXPECT_EQ((Collection->VertexStart)[0], 0);
		EXPECT_EQ((Collection->VertexCount)[0], 8);
		for (int i = (Collection->VertexStart)[0]; i < (Collection->VertexCount)[0]; i++)
		{
			EXPECT_EQ((Collection->BoneMap)[i], TransformIndex);
		}

		EXPECT_EQ(Collection->TransformToGeometryIndex.Num(), 3);
		EXPECT_EQ(Collection->TransformToGeometryIndex[0], -1);
		EXPECT_EQ(Collection->TransformToGeometryIndex[1], -1);
		EXPECT_EQ(Collection->TransformToGeometryIndex[2], 0);

		//
		// Parent the geometry
		//
		GeometryCollectionAlgo::ParentTransform(Collection, 0, TransformIndex);
		EXPECT_EQ((Collection->Children)[0].Num(), 1);
		EXPECT_EQ((Collection->Parent)[0], 1);
		EXPECT_EQ((Collection->Children)[1].Num(), 1);
		EXPECT_TRUE((Collection->Children)[1].Contains(0));
		EXPECT_EQ((Collection->Parent)[1], -1);
		EXPECT_LT(((Collection->Transform)[TransformIndex].GetTranslation() - FVector3f(-10)).Size(), KINDA_SMALL_NUMBER);
		EXPECT_EQ((Collection->TransformIndex).Num(), 1);
		EXPECT_EQ((Collection->TransformIndex)[0], TransformIndex);
		EXPECT_EQ((Collection->VertexStart)[0], 0);
		EXPECT_EQ((Collection->VertexCount)[0], 8);
		for (int i = (Collection->VertexStart)[0]; i < (Collection->VertexCount)[0]; i++)
		{
			EXPECT_EQ((Collection->BoneMap)[i], TransformIndex);
		}

		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransform);
		EXPECT_LT((GlobalTransform[0].GetTranslation() - FVector(13)).Size(), KINDA_SMALL_NUMBER);
		EXPECT_LT((GlobalTransform[2].GetTranslation() - FVector(3)).Size(), KINDA_SMALL_NUMBER);


		//
		// Force a circular parent
		//
		EXPECT_FALSE(GeometryCollectionAlgo::HasCycle((Collection->Parent), TransformIndex));
		(Collection->Children)[0].Add(2);
		(Collection->Parent)[0] = 2;
		(Collection->Children)[2].Add(0);
		(Collection->Parent)[2] = 0;
		EXPECT_TRUE(GeometryCollectionAlgo::HasCycle((Collection->Parent), TransformIndex));

		delete Collection;
	}

	void ReindexMaterialsTest()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform::Identity, FVector(1.0));
		
		EXPECT_EQ(Collection->Sections.Num(), 2);

		Collection->ReindexMaterials();

		// Reindexing doesn't change the number of sections
		EXPECT_EQ(Collection->Sections.Num(), 2);

		// Ensure material selections have correct material ids after reindexing
		for (int i = 0; i < 12; i++)
		{
			if (i < 6)
			{
				EXPECT_EQ((Collection->MaterialID)[i], 0);
			}
			else
			{
				EXPECT_EQ((Collection->MaterialID)[i], 1);
			}
		}

		// Delete vertices for a single material id
		TArray<int32> DelList = { 0,1,2,3,4,5 };
		Collection->RemoveElements(FGeometryCollection::FacesGroup, DelList);

		Collection->ReindexMaterials();

		// Ensure we now have 1 section
		EXPECT_EQ(Collection->Sections.Num(), 1);
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, 6);		
		 
		// Add a copy of the geometry and reindex
		TSharedPtr<FGeometryCollection> Collection2 = GeometryCollection::MakeCubeElement(FTransform::Identity, FVector(1.0));
		Collection->AppendGeometry(*Collection2.Get());
		Collection->ReindexMaterials();

		// test that sections created are consolidated
		EXPECT_EQ(Collection->Sections.Num(), 2);
		EXPECT_EQ((Collection->Sections)[0].MaterialID, 0);
		EXPECT_EQ((Collection->Sections)[0].NumTriangles, 6);
		EXPECT_EQ((Collection->Sections)[1].MaterialID, 1);
		EXPECT_EQ((Collection->Sections)[1].NumTriangles, 12);
	}

	void AttributeTransferTest()
	{
		TSharedPtr<FGeometryCollection> Collection1 = GeometryCollection::MakeCubeElement(FTransform::Identity, FVector(1.0));
		TSharedPtr<FGeometryCollection> Collection2 = GeometryCollection::MakeCubeElement(FTransform::Identity, FVector(1.0));
		TSharedPtr<FGeometryCollection> Collection3 = GeometryCollection::MakeCubeElement(FTransform::Identity, FVector(2.0));
		Collection2->AppendGeometry(*Collection3.Get());
		// set color one 1
		for (int i = 0; i < Collection1->NumElements(FGeometryCollection::VerticesGroup); ++i)
		{
			(Collection1->Color)[i] = FLinearColor(1, 0, 1, 1);
		}

		// transfer color to 2
		FName Attr("Color");
		GeometryCollection::AttributeTransfer<FLinearColor>(Collection1.Get(), Collection2.Get(), Attr, Attr);

		// test color is set correctly on 2
		for (int i = 0; i < Collection2->NumElements(FGeometryCollection::VerticesGroup); ++i)
		{
			EXPECT_TRUE((Collection2->Color)[i].Equals(FColor(1, 0, 1, 1)));
		}
	}

	void AttributeDependencyTest()
	{
		FGeometryCollection* Collection = new FGeometryCollection();

		TManagedArray<FTransform> Transform;
		TManagedArray<int32> Int32s;

		FName Group1 = "Group1";
		FName Group2 = "Group2";
		FName Group3 = "Group3";
		FName Group4 = "Group4";
		FName Group5 = "Group5";

		FManagedArrayCollection::FConstructionParameters TransformDependency(FTransformCollection::TransformGroup);

		// valid dependency
		// (A)G1
		// |
		// _______
		// |      |
		// (B)G2  (D)G4
		// |
		// (C)G3
		Collection->AddExternalAttribute<FTransform>("AttributeA", Group1, Transform);
		Collection->AddExternalAttribute<FTransform>("AttributeB", Group2, Transform, Group1);
		Collection->AddExternalAttribute<FTransform>("AttributeC", Group3, Transform, Group2);
		Collection->AddExternalAttribute<FTransform>("AttributeD", Group4, Transform, Group1);

		// Force a circular group dependency - from G1 to G3 (these are now allowed!)
		constexpr bool bAllowCircularDependency = true;
		Collection->SetDependency("AttributeA", Group1, Group3, bAllowCircularDependency);

		delete Collection;
	}


	void IntListReindexOnDeletionTest()
	{
		//--gtest_filter=GeometryCollection_CreationTest.IntListReindexOnDeletionTest

		//
		// Build two arrays in GroupB:
		//   - One array is an index into GroupA. (dependent on  A)
		//   - The other array stores a local copy of the value in A. 
		// As elements are deleted from GroupA, the indices will be 
		// shifted within GroupB, and missing references will be 
		// updated to INDEX_NONE. 
		//
		// At the end of the test all indices in GroupB will be INDEX_NONE.

		FManagedArrayCollection* Collection = new FManagedArrayCollection();

		FManagedArrayCollection::FConstructionParameters DependencyOnA("GroupA");
		Collection->AddGroup("GroupA");
		Collection->AddGroup("GroupB");

		Collection->AddElements(10, "GroupA");
		Collection->AddElements(10, "GroupB");

		TManagedArray<float>& AttrInA = Collection->AddAttribute<float>("Floats", "GroupA");

		// int32
		TManagedArray<int32>& IntInB = Collection->AddAttribute<int32>("Int", "GroupB", DependencyOnA);
		TManagedArray<float>& IntInBVal = Collection->AddAttribute<float>("Intf", "GroupB");
		//FIntVector
		TManagedArray<FIntVector>& VecInB = Collection->AddAttribute<FIntVector>("IntVec", "GroupB", DependencyOnA);
		TManagedArray< TArray<float> >& VecInBVal = Collection->AddAttribute< TArray<float> >("IntVecf", "GroupB");
		//FIntVector2
		TManagedArray<FIntVector2>& Vec2InB = Collection->AddAttribute<FIntVector2>("IntVec2", "GroupB", DependencyOnA);
		TManagedArray< TArray<float> >& Vec2InBVal = Collection->AddAttribute< TArray<float> >("IntVec2f", "GroupB");
		// TArray<FIntVector2>
		TManagedArray<TArray<FIntVector2>>& VecArray2InB = Collection->AddAttribute<TArray<FIntVector2>>("IntVecArray2", "GroupB", DependencyOnA);
		TManagedArray< TArray<FVector2f> >& VecArray2InBVal = Collection->AddAttribute< TArray<FVector2f> >("IntVecArray2f", "GroupB");
		// FIntVector4
		TManagedArray<FIntVector4>& Vec4InB = Collection->AddAttribute<FIntVector4>("Int4", "GroupB", DependencyOnA);
		TManagedArray< TArray<float> >& Vec4InBVal = Collection->AddAttribute< TArray<float> >("Int4f", "GroupB");
		// TArray<int32>
		TManagedArray<TArray<int32>>& ArrayNInB = Collection->AddAttribute<TArray<int32>>("IntArray", "GroupB", DependencyOnA);
		TManagedArray<TArray<float>>& ArrayNInBVal = Collection->AddAttribute<TArray<float>>("IntArrayf", "GroupB");


		for (int i = 0; i < 10; i++)
		{
			AttrInA[i] = i * 0.1;
		}

		for (int i = 0; i < 10; i++)
		{
			// int32
			IntInB[i] = i;
			IntInBVal[i] = AttrInA[IntInB[i]];

			//FIntVector
			VecInB[i] = FIntVector(FMath::Clamp(i - 1, 0, 9), FMath::Clamp(i, 0, 9), FMath::Clamp(i + 1, 0, 9));
			VecInBVal[i].SetNum(3);
			VecInBVal[i][0] = AttrInA[VecInB[i][0]];
			VecInBVal[i][1] = AttrInA[VecInB[i][1]];
			VecInBVal[i][2] = AttrInA[VecInB[i][2]];

			//FIntVector4
			Vec2InB[i] = FIntVector2(FMath::Clamp(i - 1, 0, 9), FMath::Clamp(i, 0, 9));
			Vec2InBVal[i].SetNum(2);
			Vec2InBVal[i][0] = AttrInA[Vec2InB[i][0]];
			Vec2InBVal[i][1] = AttrInA[Vec2InB[i][1]];

			// TArray<FIntVector2>
			VecArray2InB[i].SetNum(i);
			VecArray2InBVal[i].SetNum(i);
			for (int j = 0; j < i; j++)
			{
				VecArray2InB[i][j] = FIntVector2(FMath::Clamp(j - 1, 0, 9), FMath::Clamp(j, 0, 9));
				VecArray2InBVal[i][j] = FVector2f(AttrInA[VecArray2InB[i][j][0]], AttrInA[VecArray2InB[i][j][1]]);
			}

			//FIntVector4
			Vec4InB[i] = FIntVector4(FMath::Clamp(i - 1, 0, 9), FMath::Clamp(i, 0, 9), FMath::Clamp(i + 1, 0, 9), FMath::Clamp(i + 2, 0, 9));
			Vec4InBVal[i].SetNum(4);
			Vec4InBVal[i][0] = AttrInA[Vec4InB[i][0]];
			Vec4InBVal[i][1] = AttrInA[Vec4InB[i][1]];
			Vec4InBVal[i][2] = AttrInA[Vec4InB[i][2]];
			Vec4InBVal[i][3] = AttrInA[Vec4InB[i][3]];

			//TArray<int32>
			ArrayNInB[i].SetNum(i);
			ArrayNInBVal[i].SetNum(i);
			for (int j = 0; j < i; j++)
			{
				ArrayNInB[i][j] = FMath::Clamp(j, 0, 9);
				ArrayNInBVal[i][j] = AttrInA[ArrayNInB[i][j]];
			}
		}

		auto RemoveIndicesOn = [&](const TArray<int32>& Elements)
		{
			bool HasValidValues = false;
			Collection->RemoveElements("GroupA", Elements);
			for (int i = 0; i < 10; i++)
			{
				// int32
				if (IntInB[i] != INDEX_NONE)
				{
					HasValidValues = true;
					EXPECT_NEAR(AttrInA[IntInB[i]], IntInBVal[i], FLT_EPSILON);
				}

				//FIntVector
				for (int j = 0; j < 3; j++)
				{
					if (VecInB[i][j] != INDEX_NONE)
					{
						HasValidValues = true;
						EXPECT_NEAR(AttrInA[VecInB[i][j]], VecInBVal[i][j], FLT_EPSILON);
					}
				}

				//FIntVector2
				for (int j = 0; j < 2; j++)
				{
					if (Vec2InB[i][j] != INDEX_NONE)
					{
						HasValidValues = true;
						EXPECT_NEAR(AttrInA[Vec2InB[i][j]], Vec2InBVal[i][j], FLT_EPSILON);
					}
				}

				// TArray<FIntVector2>
				for (int j = 0; j < i; j++)
				{
					if (VecArray2InB[i][j][0] != INDEX_NONE)
					{
						HasValidValues = true;
						EXPECT_NEAR(AttrInA[VecArray2InB[i][j][0]], VecArray2InBVal[i][j][0], FLT_EPSILON);
					}
					if (VecArray2InB[i][j][1] != INDEX_NONE)
					{
						HasValidValues = true;
						ensure(FMath::IsNearlyEqual(AttrInA[VecArray2InB[i][j][1]], VecArray2InBVal[i][j][1], FLT_EPSILON));
						EXPECT_NEAR(AttrInA[VecArray2InB[i][j][1]], VecArray2InBVal[i][j][1], FLT_EPSILON);
					}
				}


				//FIntVector4
				for (int j = 0; j < 4; j++)
				{
					if (Vec4InB[i][j] != INDEX_NONE)
					{
						HasValidValues = true;
						EXPECT_NEAR(AttrInA[Vec4InB[i][j]], Vec4InBVal[i][j], FLT_EPSILON);
					}
				}

				//TArray<int32>
				for (int j = 0; j < i; j++)
				{
					if (ArrayNInB[i][j] != INDEX_NONE)
					{
						HasValidValues = true;
						EXPECT_NEAR(AttrInA[ArrayNInB[i][j]], ArrayNInBVal[i][j], FLT_EPSILON);
					}
				}
			}
			return HasValidValues;
		};

		EXPECT_TRUE(RemoveIndicesOn({ 3 }));
		EXPECT_TRUE(RemoveIndicesOn({ 3,4,5 }));
		EXPECT_TRUE(RemoveIndicesOn({ 5 }));
		EXPECT_TRUE(RemoveIndicesOn({ 0 }));
		EXPECT_TRUE(RemoveIndicesOn({ 1,3 }));
		EXPECT_TRUE(RemoveIndicesOn({ 1 }));
		EXPECT_FALSE(RemoveIndicesOn({ 0 }));

		delete Collection;
	}

	void IntListSelfDependencyTest()
	{
		//--gtest_filter=GeometryCollection_CreationTest.IntListSelfDependencyTest
		FManagedArrayCollection Collection;

		constexpr bool bSaved = true;
		constexpr bool bAllowCircularDependency = true;
		FManagedArrayCollection::FConstructionParameters DependencyOnA("GroupA", bSaved, bAllowCircularDependency);
		Collection.AddGroup("GroupA");

		TManagedArray<float>& Values = Collection.AddAttribute<float>("Values", "GroupA");

		//
		// Build two arrays:
		//   - One array is an index. (dependent on  A)
		//   - The other array stores a local copy of the values. 
		// As elements are deleted from GroupA, the indices will be 
		// shifted, and missing references will be 
		// updated to INDEX_NONE. 
		// 

		// int32
		TManagedArray<int32>& Int = Collection.AddAttribute<int32>("Int", "GroupA", DependencyOnA);
		TManagedArray<float>& IntValues = Collection.AddAttribute<float>("IntValues", "GroupA");
		// TArray<int32>
		TManagedArray<TArray<int32>>& Array = Collection.AddAttribute<TArray<int32>>("Array", "GroupA", DependencyOnA);
		TManagedArray<TArray<float>>& ArrayValues = Collection.AddAttribute<TArray<float>>("ArrayValues", "GroupA");
		//FIntVector
		TManagedArray<FIntVector>& Vec = Collection.AddAttribute<FIntVector>("IntVec", "GroupA", DependencyOnA);
		TManagedArray<FVector3f>& VecValues = Collection.AddAttribute<FVector3f>("IntVecValues", "GroupA");
		//FIntVector2
		TManagedArray<FIntVector2>& Vec2 = Collection.AddAttribute<FIntVector2>("IntVec2", "GroupA", DependencyOnA);
		TManagedArray<FVector2f>& Vec2Values = Collection.AddAttribute<FVector2f>("IntVec2Values", "GroupA");
		// TArray<FIntVector2>
		TManagedArray<TArray<FIntVector2>>& ArrayVec2 = Collection.AddAttribute<TArray<FIntVector2>>("ArrayIntVec2", "GroupA", DependencyOnA);
		TManagedArray<TArray<FVector2f>>& ArrayVec2Values = Collection.AddAttribute<TArray<FVector2f>>("ArrayIntVec2Values", "GroupA");
		//FIntVector4
		TManagedArray<FIntVector4>& Vec4 = Collection.AddAttribute<FIntVector4>("IntVec4", "GroupA", DependencyOnA);
		TManagedArray<FVector4f>& Vec4Values = Collection.AddAttribute<FVector4f>("IntVec4Values", "GroupA");

		auto CheckIndexNoneRange = [&](int32 StartIndex, int32 EndIndex)
		{
			for (int32 Index = StartIndex; Index < EndIndex; ++Index)
			{
				EXPECT_NEAR(Values[Index], 0.f, FLT_EPSILON);
				EXPECT_EQ(Int[Index], INDEX_NONE);
				EXPECT_EQ(Array[Index].Num(), 0);
				EXPECT_EQ(Vec[Index], FIntVector(INDEX_NONE));
				EXPECT_EQ(Vec2[Index], FIntVector2(INDEX_NONE));
				EXPECT_EQ(ArrayVec2[Index].Num(), 0);
				EXPECT_EQ(Vec4[Index], FIntVector4(INDEX_NONE));
			}
		};

		// Add elements and check indices are initialized
		Collection.AddElements(10, "GroupA");
		CheckIndexNoneRange(0, 10);

		// Set initial values
		for (int i = 0; i < 10; i++)
		{
			Values[i] = 1.f + i * 0.1f;
		}

		for (int i = 0; i < 10; i++)
		{
			// int32
			Int[i] = i;
			IntValues[i] = Values[Int[i]];

			// TArray<int32>
			Array[i].SetNum(i);
			ArrayValues[i].SetNum(i);
			for (int j = 0; j < i; ++j)
			{
				Array[i][j] = (j + i) % 10;
				ArrayValues[i][j] = Values[Array[i][j]];
			}

			//FIntVector
			Vec[i] = FIntVector(i, (i + 1) % 10, (i + 2) % 10);
			VecValues[i] = FVector3f(Values[Vec[i][0]], Values[Vec[i][1]], Values[Vec[i][2]]);

			//FIntVector2
			Vec2[i] = FIntVector2((i + 3) % 10, (i + 4) % 10);
			Vec2Values[i] = FVector2f(Values[Vec2[i][0]], Values[Vec2[i][1]]);

			// TArray<FIntVector2>
			ArrayVec2[i].SetNum(i);
			ArrayVec2Values[i].SetNum(i);
			for (int j = 0; j < i; ++j)
			{
				ArrayVec2[i][j] = FIntVector2((j + i + 3) % 10, (j + i + 4) % 10);
				ArrayVec2Values[i][j] = FVector2f(Values[ArrayVec2[i][j][0]], Values[ArrayVec2[i][j][1]]);
			}

			//FIntVector4
			Vec4[i] = FIntVector4((i + 5) % 10, (i + 6) % 10, (i + 7) % 10, (i + 8) % 10);
			Vec4Values[i] = FVector4f(Values[Vec4[i][0]], Values[Vec4[i][1]], Values[Vec4[i][2]], Values[Vec4[i][3]]);
		}

		auto CheckValues = [&](int32 Size)
		{
			for (int i = 0; i < Size; i++)
			{
				// int32
				if (Int[i] != INDEX_NONE)
				{
					EXPECT_NEAR(IntValues[i], Values[Int[i]], FLT_EPSILON);
				}

				// TArray<int32>
				for (int j = 0; j < Array[i].Num(); ++j)
				{
					if (Array[i][j] != INDEX_NONE)
					{
						EXPECT_NEAR(ArrayValues[i][j], Values[Array[i][j]], FLT_EPSILON);
					}
				}

				//FIntVector
				for (int j = 0; j < 3; ++j)
				{
					if (Vec[i][j] != INDEX_NONE)
					{
						EXPECT_NEAR(VecValues[i][j], Values[Vec[i][j]], FLT_EPSILON);
					}
				}

				//FIntVector2
				for (int j = 0; j < 2; ++j)
				{
					if (Vec2[i][j] != INDEX_NONE)
					{
						EXPECT_NEAR(Vec2Values[i][j], Values[Vec2[i][j]], FLT_EPSILON);
					}
				}

				// TArray<FIntVector2>
				for (int j = 0; j < ArrayVec2[i].Num(); ++j)
				{
					for (int k = 0; k < 2; ++k)
					{
						if (ArrayVec2[i][j][k] != INDEX_NONE)
						{
							EXPECT_NEAR(ArrayVec2Values[i][j][k], Values[ArrayVec2[i][j][k]], FLT_EPSILON);
						}
					}
				}

				//FIntVector4
				for (int j = 0; j < 4; ++j)
				{
					if (Vec4[i][j] != INDEX_NONE)
					{
						EXPECT_NEAR(Vec4Values[i][j], Values[Vec4[i][j]], FLT_EPSILON);
					}
				}
			}
		};
		CheckValues(10);

		// Insert some elements
		Collection.InsertElements(5, 2, "GroupA");
		CheckIndexNoneRange(5, 7);
		CheckValues(Collection.NumElements("GroupA"));

		// Remove some elements
		Collection.RemoveElements("GroupA", 3, 2);
		CheckValues(Collection.NumElements("GroupA"));

		// Remove some more elements
		Collection.RemoveElements("GroupA", { 1,3,5 });
		CheckValues(Collection.NumElements("GroupA"));
	}


	void AppendManagedArrayCollectionTest()
	{
		//--gtest_filter=GeometryCollection_CreationTest.AppendManagedArrayCollectionTest

		//
		// Build two ManagedArryaCollections then append CollectionA to CollectionB. 
		//
		// At the end of the test CollectionA should be moved and CollectionB
		// should be at the start of the arrays. 
		//
		
		auto Build = [](FManagedArrayCollection& InCollection, int Offset, FName GroupName)
		{
			FManagedArrayCollection::FConstructionParameters DependencyOnShared("Shared");

			// single array with 10 entries from 0 to 1.0
			InCollection.AddGroup("Shared");
			InCollection.AddElements(10, "Shared");
			TManagedArray<float>& IntInShared = InCollection.AddAttribute<float>("Float", "Shared");
			for (int i = 0; i < 10; i++)
			{
				IntInShared[i] = i * 0.1 + Offset;
			}

			// two arrays, where Int is dependent on Shared, and Intf is a copy of shared.
			// Intf is used to check the reordering of the groups. 
			InCollection.AddGroup(GroupName);
			InCollection.AddElements(10, GroupName);
			TManagedArray<int32>& IntIn = InCollection.AddAttribute<int32>("Int", GroupName, DependencyOnShared);
			TManagedArray<float>& SharedValue = InCollection.AddAttribute<float>("SharedValue", GroupName);
			for (int i = 0; i < 10; i++)
			{
				// int32
				IntIn[i] = i;
				SharedValue[i] = IntInShared[IntIn[i]];
			}
		};

		FManagedArrayCollection CollectionA; Build(CollectionA, 0, FName("GroupA"));
		FManagedArrayCollection CollectionB; Build(CollectionB, 1, FName("GroupB"));

		CollectionA.Append(CollectionB);

		// check the union of shared
		const TManagedArray<float>& Shared = CollectionA.GetAttribute<float>("Float", "Shared");
		EXPECT_TRUE(CollectionA.NumElements("Shared") == 20);
		for (int i = 0; i < 10; i++) // 0 to 9 greater than one
		{
			float val = (float)i * 0.1 + 1;
			EXPECT_TRUE( FMath::IsNearlyEqual(Shared[i], val));
		}
		for (int i = 10; i < 20; i++) // 10 to 19 lass than one
		{
			float val = (float)(i-10) * 0.1;
			EXPECT_TRUE(FMath::IsNearlyEqual(Shared[i], val));
		}
		
		// check reindexing of GroupA
		const TManagedArray<int32>& IntInA = CollectionA.GetAttribute<int32>("Int", "GroupA");
		for (int i = 0; i < 10; i++)
		{
			EXPECT_TRUE(IntInA[i] == i + 10);
		}

		// check indexing of GroupB
		const TManagedArray<int32>& IntInB = CollectionA.GetAttribute<int32>("Int", "GroupB");
		for (int i = 0; i < 10; i++)
		{
			EXPECT_TRUE(IntInB[i] == i);
		}

		// check reindexing into shared for A
		const TManagedArray<float>& SharedValueA = CollectionA.GetAttribute<float>("SharedValue", "GroupA");
		for (int i = 0; i < 10; i++)
		{
			EXPECT_TRUE(SharedValueA[i] == Shared[IntInA[i]]);
		}

		// check reindexing into shared for B
		const TManagedArray<float>& SharedValueB = CollectionA.GetAttribute<float>("SharedValue", "GroupB");
		for (int i = 0; i < 10; i++)
		{
			EXPECT_TRUE(SharedValueB[i] == Shared[IntInB[i]]);
		}

		EXPECT_TRUE(true);
	}


	void AppendTransformCollectionTest()
	{
		//--gtest_filter=GeometryCollection_CreationTest.AppendTransformCollectionTest

		//
		// Build two TranformCollections then append CollectionA to CollectionB. 
		//
		// At the end of the test CollectionA should be moved and CollectionB
		// should be at the start of the arrays. The parenting of CollectionA
		// should be updated to maintain the parent-child relationships.  
		//
		auto Build = [](FTransformCollection& InCollection, int32 Offset)
		{
			InCollection.AddElements(8, FTransformCollection::TransformGroup);

			//  0
			//  ...1
			//  ...5
			//  ......6
			//  ......3
			//  ...2
			//  ......7
			//  .........4
			(InCollection.Parent)[0] = -1;
			(InCollection.Children)[0].Add(1);
			(InCollection.Children)[0].Add(5);
			(InCollection.Children)[0].Add(2);
			(InCollection.Parent)[1] = 0;
			(InCollection.Parent)[2] = 0;
			(InCollection.Children)[2].Add(7);
			(InCollection.Parent)[3] = 5;
			(InCollection.Parent)[4] = 7;
			(InCollection.Parent)[5] = 0;
			(InCollection.Children)[5].Add(6);
			(InCollection.Children)[5].Add(3);
			(InCollection.Parent)[6] = 5;
			(InCollection.Parent)[7] = 2;
			(InCollection.Children)[7].Add(4);

			for (int i = 0; i < 8; i++)
			{
				(InCollection.BoneName)[i] = FString::FromInt(i);
			}
		};

		FTransformCollection CollectionA; Build(CollectionA, 0);
		FTransformCollection CollectionB; Build(CollectionB, 10);

		// add some random dependent attribute to the transform group. 
		FManagedArrayCollection::FConstructionParameters TransformDependency(FTransformCollection::TransformGroup);
		TManagedArray<int32>& ExtraA = CollectionA.AddAttribute<int32>("UserAttr", FTransformCollection::TransformGroup, TransformDependency);
		TManagedArray<int32>& ExtraB = CollectionB.AddAttribute<int32>("UserAttr", FTransformCollection::TransformGroup, TransformDependency);
		for (int i = ExtraA.Num()-1, j = 0; i >= 0; i--, j++) { ExtraA[i] = j; ExtraB[i] = j; } // count down

		// add some random transform dependent attribute to a different group. 
		FManagedArrayCollection::FConstructionParameters OtherDependency("Other");
		CollectionA.AddGroup("Other"); CollectionB.AddGroup("Other");
		CollectionA.AddElements(5, "Other"); CollectionB.AddElements(6, "Other");
		TManagedArray<int32>& OtherA = CollectionA.AddAttribute<int32>("UserAttr", "Other", TransformDependency);
		TManagedArray<int32>& OtherB = CollectionB.AddAttribute<int32>("UserAttr", "Other", TransformDependency);
		for (int i = 0; i < OtherA.Num();  i++) { OtherA[i] = i;} // count up
		for (int i = 0; i < OtherB.Num(); i++) { OtherB[i] = i; } // count up

		// What we are testing! - Append
		CollectionA.Append(CollectionB);

		// validate that the collectionB is at the start of A, and A is indexed correctly
		int32 Size = CollectionA.NumElements(FTransformCollection::TransformGroup);
		EXPECT_TRUE(Size==16);

		// validate appended transform hierarchy
		for (int32 Idx = 0, Sdx = Size/2; Idx < Size/2; Idx++, Sdx++)
		{
			EXPECT_TRUE(CollectionA.BoneName[Idx]== CollectionA.BoneName[Sdx]);
			if (CollectionA.Parent[Idx] == INDEX_NONE)
				EXPECT_TRUE(CollectionA.Parent[Sdx] == INDEX_NONE);
			else
				EXPECT_TRUE(CollectionA.Parent[Idx] == CollectionA.Parent[Sdx] - Size / 2);
			EXPECT_TRUE(CollectionA.Children[Idx].Num() == CollectionA.Children[Sdx].Num());
			for (int i : CollectionA.Children[Sdx])
				EXPECT_TRUE(CollectionA.Children[Idx].Contains(i - Size / 2) );

			// validate dependent user attributes in transform group
			EXPECT_TRUE(ExtraA[Idx] == ExtraA[Sdx]-Size/2);

		}

		EXPECT_TRUE(CollectionA.NumElements("Other") == 11);
		for (int32 Idx = 0; Idx < CollectionA.NumElements("Other"); Idx++)
		{
			if (Idx <= 5)
			{
				EXPECT_TRUE(OtherA[Idx] == OtherB[Idx]);
			}
			else
			{
				EXPECT_TRUE(OtherA[Idx] == Idx+2);
			}
		}


		EXPECT_TRUE(true);

	}


	void CollectionCycleTest()
	{
		//--gtest_filter=GeometryCollection_CreationTest.CollectionCycleTest

		//
		// Build dependent attributes and check for connections using HasCycle
		// 
		//
		FManagedArrayCollection Collection;
		Collection.AddGroup("A");
		Collection.AddGroup("B");
		Collection.AddGroup("C");
		Collection.AddGroup("D");

		EXPECT_FALSE(Collection.IsConnected("A", "C"));
		EXPECT_FALSE(Collection.IsConnected("A", "B"));
		EXPECT_FALSE(Collection.IsConnected("A", "D"));
		EXPECT_FALSE(Collection.IsConnected("B", "C"));

		//  A
		//  |
		//  |--->B "A depends on B"
		//  |    |
		//  |    |--->C "B depends on C"
		//  | 
		//  |--->D  "A depends on D
		FManagedArrayCollection::FConstructionParameters ADependency("A");
		FManagedArrayCollection::FConstructionParameters BDependency("B");
		FManagedArrayCollection::FConstructionParameters CDependency("C");
		FManagedArrayCollection::FConstructionParameters DDependency("D");

		Collection.AddAttribute<int32>("AOnB", "A", BDependency);
		Collection.AddAttribute<int32>("AOnD", "A", DDependency);
		Collection.AddAttribute<int32>("BOnC", "B", CDependency);
		Collection.AddAttribute<int32>("C", "C"); // None
		Collection.AddAttribute<int32>("D", "D"); // None

		EXPECT_TRUE(Collection.IsConnected("A", "B"));
		EXPECT_TRUE(Collection.IsConnected("A", "C"));
		EXPECT_TRUE(Collection.IsConnected("A", "D"));
		EXPECT_TRUE(Collection.IsConnected("B", "C"));

		EXPECT_FALSE(Collection.IsConnected("A", "A"));
		EXPECT_FALSE(Collection.IsConnected("B", "A"));
		EXPECT_FALSE(Collection.IsConnected("B", "B"));
		EXPECT_FALSE(Collection.IsConnected("B", "D"));
		EXPECT_FALSE(Collection.IsConnected("C", "A"));
		EXPECT_FALSE(Collection.IsConnected("C", "B"));
		EXPECT_FALSE(Collection.IsConnected("C", "C"));
		EXPECT_FALSE(Collection.IsConnected("C", "D"));
		EXPECT_FALSE(Collection.IsConnected("D", "A"));
		EXPECT_FALSE(Collection.IsConnected("D", "B"));
		EXPECT_FALSE(Collection.IsConnected("D", "C"));
		EXPECT_FALSE(Collection.IsConnected("D", "D"));

		// Now create a cycle by connecting C back to A
		constexpr bool bAllowCircularDependency = true;
		Collection.SetDependency("C", "C", "A", bAllowCircularDependency);

		EXPECT_TRUE(Collection.IsConnected("A", "A"));
		EXPECT_TRUE(Collection.IsConnected("A", "B"));
		EXPECT_TRUE(Collection.IsConnected("A", "C"));
		EXPECT_TRUE(Collection.IsConnected("A", "D"));
		EXPECT_TRUE(Collection.IsConnected("B", "A"));
		EXPECT_TRUE(Collection.IsConnected("B", "B"));
		EXPECT_TRUE(Collection.IsConnected("B", "C"));
		EXPECT_TRUE(Collection.IsConnected("B", "D"));
		EXPECT_TRUE(Collection.IsConnected("C", "A"));
		EXPECT_TRUE(Collection.IsConnected("C", "B"));
		EXPECT_TRUE(Collection.IsConnected("C", "C"));
		EXPECT_TRUE(Collection.IsConnected("C", "D"));

		EXPECT_FALSE(Collection.IsConnected("D", "A"));
		EXPECT_FALSE(Collection.IsConnected("D", "B"));
		EXPECT_FALSE(Collection.IsConnected("D", "C"));
		EXPECT_FALSE(Collection.IsConnected("D", "D"));
	}

}
