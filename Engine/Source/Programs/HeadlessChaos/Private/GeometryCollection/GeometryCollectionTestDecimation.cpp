// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestDecimation.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/TransformCollection.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"

#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/Particles.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/Vector.h"

#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "../Resource/BoxGeometry.h"
#include "../Resource/CylinderGeometry.h"
#include "../Resource/EllipsoidGeometry.h"
#include "../Resource/EllipsoidGeometry2.h"
#include "../Resource/EllipsoidGeometry3.h"
#include "../Resource/FracturedGeometry.h"
#include "../Resource/SphereGeometry.h"
#include "../Resource/TorusGeometry.h"

//#define VERBOSE
#define WRITE_OBJ_FILES

namespace GeometryCollectionTest
{
	
	using namespace Chaos;

	Chaos::FParticles
	BuildParticlesFeomGeomCollection(FGeometryCollection *TestCollection)
	{
		TManagedArray<FVector3f> &Vertex = TestCollection->Vertex;
		const int numParticles = Vertex.Num();
		Chaos::FParticles particles;
		particles.AddParticles(numParticles);
		for (int i = 0; i < numParticles; i++)
			particles.SetX(i, FVec3(Vertex[i][0], Vertex[i][1], Vertex[i][2]));
		return particles;
	}

	Chaos::FTriangleMesh
	BuildTriMeshFromGeomCollection(FGeometryCollection *TestCollection)
	{
		TManagedArray<FIntVector>& Indices = TestCollection->Indices;
		const int numTris = Indices.Num();
		TArray<Chaos::TVec3<int32>> tris;
		tris.SetNumUninitialized(numTris);
		for (int i = 0; i < numTris; i++)
			tris[i] = Chaos::TVec3<int32>(Indices[i][0], Indices[i][1], Indices[i][2]);

		Chaos::FTriangleMesh triMesh(MoveTemp(tris));
		return triMesh;
	}

	void
	PrintBoolArray(const TManagedArray<bool> &bArray)
	{
#ifdef VERBOSE
		for (int j = 0; j < bArray.Num(); j++)
			std::cout << (bArray[j] ? "1" : "0");
		std::cout << std::endl;
#endif
	}

	void
	WriteImportanceOrderObjs(
		FGeometryCollection *TestCollection,
		const TArray<int32> &importance,
		const TArray<int32> &coincidentVertices,
		const char *baseName, 
		const char *path)
	{
		// Add an array to the vertices group for visibility
		TManagedArray<bool>& visibility = TestCollection->AddAttribute<bool>("VertexVisibility", FGeometryCollection::VerticesGroup);
		// Initialize visibility
		const int numParticles = importance.Num();// visibility.Num();
		check(numParticles <= visibility.Num());
		const int numGoodParticles = numParticles - coincidentVertices.Num();
		check(numParticles >= numGoodParticles);
		for (int i = 0; i < numParticles; i++)
			visibility[i] = false;

#ifdef VERBOSE
		std::cout << baseName << " - Num points: " << numParticles 
			<< " Num coincident: " << coincidentVertices.Num()
			<< " - visibility:" << std::endl;
#endif
#ifdef WRITE_OBJ_FILES
		TestCollection->WriteDataToOBJFile(FString(baseName), path, true, false);
#endif
		int i = 0;
		for (; i < FMath::Min(4, numGoodParticles); i++)
			visibility[importance[i]] = true;
#ifdef WRITE_OBJ_FILES
		TestCollection->WriteDataToOBJFile(FString(baseName)+"_4", path, false, true);
#endif
		PrintBoolArray(visibility);

		for (; i < FMath::Min(8, numGoodParticles); i++)
			visibility[importance[i]] = true;
#ifdef WRITE_OBJ_FILES
		TestCollection->WriteDataToOBJFile(FString(baseName) +"_8", path, false, true);
#endif
		PrintBoolArray(visibility);

		for (; i < static_cast<int32>(ceil(numGoodParticles * .1)); i++)
			visibility[importance[i]] = true;
#ifdef WRITE_OBJ_FILES
		TestCollection->WriteDataToOBJFile(FString(baseName) +"_10pct", path, false, true);
#endif
		PrintBoolArray(visibility);

		for (; i < static_cast<int32>(ceil(numGoodParticles * .25)); i++)
			visibility[importance[i]] = true;
#ifdef WRITE_OBJ_FILES
		TestCollection->WriteDataToOBJFile(FString(baseName) +"_25pct", path, false, true);
#endif
		PrintBoolArray(visibility);

		for (; i < static_cast<int32>(ceil(numGoodParticles * .5)); i++)
			visibility[importance[i]] = true;
#ifdef WRITE_OBJ_FILES
		TestCollection->WriteDataToOBJFile(FString(baseName) +"_50pct", path, false, true);
#endif
		PrintBoolArray(visibility);
	}

	bool
	RunGeomDecimationTest(FGeometryCollection* TestCollection, const char *BaseName, const char *OutputDir, const uint32 ExpectedHash, const bool RestrictToLocalIndexRange = false)
	{
		Chaos::FParticles Particles = BuildParticlesFeomGeomCollection(TestCollection);
		Chaos::FTriangleMesh TriMesh = BuildTriMeshFromGeomCollection(TestCollection);

		TArray<int32> CoincidentVertices;
		const TArray<int32> Importance = TriMesh.GetVertexImportanceOrdering(Particles.X(), &CoincidentVertices, RestrictToLocalIndexRange);
		check(CoincidentVertices.Num() < Importance.Num());

		const int numParticles = Particles.Size();

		// got the right number of indices
		if (RestrictToLocalIndexRange)
		{
			EXPECT_LE(Importance.Num(), numParticles);
		}
		else 
		{
			EXPECT_EQ(Importance.Num(), numParticles);
		}

		EXPECT_EQ(TSet<int32>(Importance).Num(), Importance.Num()); // indices were unique

		WriteImportanceOrderObjs(
			TestCollection, Importance, CoincidentVertices, BaseName, OutputDir);

		auto CalcImportanceHash = [](const TArray<int32>& InImportance)
		{
			uint32 Hash = 0;
			for(const int32 Value : InImportance)
			{
				Hash = HashCombine(Hash, GetTypeHash(Value));
			}
			return Hash;
		};

		const uint32 hash = CalcImportanceHash(Importance);
#ifdef VERBOSE
		std::cout << BaseName << " importance ordering hash: " << hash << std::endl;
#endif
		if (hash != ExpectedHash)
		{
			std::cout << "GeometryCollectionTestDecimation - " << BaseName
				<< " - expected importance ordering hash: " << ExpectedHash
				<< " got: " << hash << ".  Failing." << std::endl;
		}
		return hash == ExpectedHash;
		//return true;	//todo: update hash
	}

	template <class TGeom>
	bool
	RunGeomDecimationTest(const char *BaseName, const char *OutputDir, const uint32 ExpectedHash, const bool RestrictToLocalIndexRange = false)
	{
		TGeom Geom;
		FGeometryCollection* TestCollection =
			FGeometryCollection::NewGeometryCollection(
				Geom.RawVertexArray,
				Geom.RawIndicesArray);
		return RunGeomDecimationTest(TestCollection, BaseName, OutputDir, ExpectedHash, RestrictToLocalIndexRange);
	}

	template <class TGeom>
	bool
	RunGlobalGeomDecimationTest(const char *BaseName, const char *OutputDir, const uint32 ExpectedHash, const bool RestrictToLocalIndexRange = true)
	{
		TGeom Geom;
		FGeometryCollection* TestCollection =
			FGeometryCollection::NewGeometryCollection(
				Geom.RawVertexArray,
				Geom.RawIndicesArray1);
		return RunGeomDecimationTest(TestCollection, BaseName, OutputDir, ExpectedHash, RestrictToLocalIndexRange);
	}

	void TestGeometryDecimation()
	{
		bool Success = true;
		// If E:\TestGeometry\Decimation doesn't already exist, the files aren't written.

		// Standalone point pools.
		Success &= RunGeomDecimationTest<BoxGeometry>("box", "E:\\TestGeometry\\Decimation\\", 4024338882);
		Success &= RunGeomDecimationTest<CylinderGeometry>("cylinder", "E:\\TestGeometry\\Decimation\\", 2477299646);
		Success &= RunGeomDecimationTest<EllipsoidGeometry>("ellipsoid", "E:\\TestGeometry\\Decimation\\", 1158371240);
		Success &= RunGeomDecimationTest<EllipsoidGeometry2>("ellipsoid2", "E:\\TestGeometry\\Decimation\\", 554754926);
		Success &= RunGeomDecimationTest<EllipsoidGeometry3>("ellipsoid3", "E:\\TestGeometry\\Decimation\\", 2210765036);
		Success &= RunGeomDecimationTest<FracturedGeometry>("fractured", "E:\\TestGeometry\\Decimation\\", 2030682536);
		Success &= RunGeomDecimationTest<SphereGeometry>("sphere", "E:\\TestGeometry\\Decimation\\", 4119721232);
		Success &= RunGeomDecimationTest<TorusGeometry>("torus", "E:\\TestGeometry\\Decimation\\", 2519379615);
		
		// Geometry in a global point pool.
		Success &= RunGeomDecimationTest<GlobalFracturedGeometry>("globalFractured", "E:\\TestGeometry\\Decimation\\", 4227374796, true);
		Success &= RunGlobalGeomDecimationTest<GlobalFracturedGeometry>("globalFracturedMerged", "E:\\TestGeometry\\Decimation\\", 4227374796, true);

		EXPECT_TRUE(Success);
	}
	
}

