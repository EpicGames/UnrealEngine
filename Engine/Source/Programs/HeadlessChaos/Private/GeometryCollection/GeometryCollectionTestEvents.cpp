// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestEvents.h"
#include "EventManager.h"
#include "EventsData.h"
#include "PBDRigidsSolver.h"
#include "ChaosSolversModule.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/ChaosScene.h"

namespace GeometryCollectionTest
{
	using namespace ChaosTest;

	// These were previously integers outside of range of event enum, deliberately not using existing events. API changed,
	// and we need to provide enum. Casting int out of range of enum to enum is undefined, so just using existing events. 
	static const EEventType CustomEvent1 = EEventType::Breaking;
	static const EEventType CustomEvent2 = EEventType::Sleeping;

	struct EventTestData
	{
		EventTestData(int InData1, FVector InData2) : Data1(InData1), Data2(InData2) {}
		EventTestData() : Data1(0xBADF000D), Data2(FVector(0xBADF000D, 0xBADF000D, 0xBADF000D)) {}

		bool operator==(const EventTestData& Other) const
		{
			return (Data1 == Other.Data1) && (Data2 == Other.Data2);
		}

		void Reset()
		{
			Data1 = 0xBADF000D;
			Data2 = FVector(0xBADF000D, 0xBADF000D, 0xBADF000D);
		}

		int Data1;
		FVector Data2;
	};

	template<typename PayloadType>
	const TMap<IPhysicsProxyBase*, TArray<int32>>* GetProxyToIndexMap(const PayloadType* Buffer)
	{
		return nullptr;
	}

	template<typename PayloadType>
	bool IsEventDataEmpty(const PayloadType* Buffer)
	{
		return false;
	}

	class MyEventHandler
	{
	public:

		MyEventHandler(Chaos::FEventManager& InEventManager) : EventManager(InEventManager)
		{
		}
		~MyEventHandler()
		{
			EventManager.Reset();
		}

		void HandleEvent(const EventTestData& EventData)
		{
			ResultFromHandler = EventData;
		}

		void HandleEvent(const TArray<EventTestData>& EventData)
		{
			ResultFromHandler2 = EventData;
		}

		void RegisterHandler1()
		{
			EventManager.template RegisterHandler<EventTestData>(CustomEvent1, this, &MyEventHandler::HandleEvent);
		}

		void RegisterHandler2()
		{
			EventManager.template RegisterHandler<TArray<EventTestData>>(CustomEvent2, this, &MyEventHandler::HandleEvent);
		}

		void UnregisterHandler1()
		{
			EventManager.UnregisterHandler(CustomEvent1, this);
		}

		void UnregisterHandler2()
		{
			EventManager.UnregisterHandler(CustomEvent2, this);
		}

		// Prove data is reference to original and not copy passed to the event handler
		EventTestData ResultFromHandler;

		// the data dispatched can also be a frames worth of events
		TArray<EventTestData> ResultFromHandler2;

	private:
		Chaos::FEventManager& EventManager;
	};


	GTEST_TEST(AllTraits, GeometryCollection_EventBufferTest_Event_Handler)
	{
		Chaos::FEventManager EventManager(Chaos::EMultiBufferMode::Single);
		Chaos::FPBDRigidsSolver* Solver = FChaosSolversModule::GetModule()->CreateSolver(nullptr, /*AsyncDt=*/-1);

		MyEventHandler HandlerTest(EventManager);
		MyEventHandler AnotherHandlerTest(EventManager);

		// the data injected into the buffer for CustomEvent1 will be whatever is currently the variable TestData
		EventTestData TestData;
		EventTestData* TestDataPtr = &TestData;
		EventManager.template RegisterEvent<EventTestData>(CustomEvent1, [TestDataPtr]
		(const auto* Solver, EventTestData& MyData, bool ResetData)
		{
			MyData = *TestDataPtr;
		});

		// the data injected into the buffer for CustomEvent2 will be whatever is currently the variable TestArrayData
		TArray<EventTestData> TestArrayData;
		TArray<EventTestData>* TestDataPtr2 = &TestArrayData;
		EventManager.template RegisterEvent<TArray<EventTestData>>(CustomEvent2, [TestDataPtr2]
		(const auto* Solver, TArray<EventTestData>& MyData, bool ResetData)
		{
			MyData = *TestDataPtr2;
		});

		HandlerTest.RegisterHandler1();
		HandlerTest.RegisterHandler2();
		AnotherHandlerTest.RegisterHandler2();

		TestData.Data1 = 123;
		TestData.Data2 = FVector(1, 2, 3);

		EventManager.FillProducerData(Solver);
		EventManager.DispatchEvents();
		EXPECT_EQ(HandlerTest.ResultFromHandler, TestData);

		TestData.Data1 = 789;
		TestData.Data2 = FVector(7, 8, 9);

		EventManager.FillProducerData(Solver);
		EventManager.DispatchEvents();
		EXPECT_EQ(HandlerTest.ResultFromHandler, TestData);

		// Unregister - data should no longer update
		HandlerTest.UnregisterHandler1();

		EventTestData OriginalTestData = TestData;
		TestData.Data1 = 999;
		TestData.Data2 = FVector(9, 9, 9);

		EventManager.FillProducerData(Solver);
		EventManager.DispatchEvents();
		EXPECT_EQ(HandlerTest.ResultFromHandler, OriginalTestData);

		EventTestData Data;
		TestArrayData.Push(EventTestData(123, FVector(1, 2, 3)));
		TestArrayData.Push(EventTestData(456, FVector(4, 5, 6)));
		TestArrayData.Push(EventTestData(789, FVector(7, 8, 9)));

		EventManager.FillProducerData(Solver);
		EventManager.DispatchEvents();
		// dispatched to multiple handlers
		EXPECT_EQ(HandlerTest.ResultFromHandler2, TestArrayData);
		EXPECT_EQ(AnotherHandlerTest.ResultFromHandler2, TestArrayData);

		// Unregister one of the handlers from the events
		HandlerTest.UnregisterHandler2();

		TArray<EventTestData> OriginalTestArrayData = TestArrayData;
		TestArrayData.Reset();
		TestArrayData.Push(EventTestData(999, FVector(9, 9, 9)));

		EventManager.FillProducerData(Solver);
		EventManager.DispatchEvents();
		EXPECT_EQ(HandlerTest.ResultFromHandler2, OriginalTestArrayData); // Unregistered - data should no longer update
		EXPECT_EQ(AnotherHandlerTest.ResultFromHandler2, TestArrayData); // Still registered so should get updates

		HandlerTest.RegisterHandler2();

		EventManager.FillProducerData(Solver);
		EventManager.DispatchEvents();

		EXPECT_EQ(HandlerTest.ResultFromHandler2, TestArrayData);
		EXPECT_EQ(AnotherHandlerTest.ResultFromHandler2, TestArrayData);
	}


	struct SimpleEventHandler
	{	
		void HandleEvent(const EventTestData& EventData)
		{
			ResultFromHandler = EventData;
		}
		EventTestData ResultFromHandler;
	};

	GTEST_TEST(AllTraits, EventTestBuffer)
	{
		for (EMultiBufferMode BufferMode : {EMultiBufferMode::Single, EMultiBufferMode::Double, EMultiBufferMode::Triple/*, EMultiBufferMode::TripleGuarded*/})
		{
			const FReal FixedDT = 1;
			FChaosScene Scene(nullptr, FixedDT);
			Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

			Chaos::FEventManager* EventManager = Scene.GetSolver()->GetEventManager();
			EventManager->SetBufferMode(BufferMode);

			Chaos::FPBDRigidsSolver* Solver = Scene.GetSolver();

			EventTestData TestData;
			TestData.Data1 = 123;
			TestData.Data2 = FVector(1, 2, 3);
			EventTestData* TestDataPtr = &TestData;
			EventManager->template RegisterEvent<EventTestData>(CustomEvent1, [TestDataPtr]
			(const auto* Solver, EventTestData& MyData, bool ResetData)
				{
					MyData = *TestDataPtr;
				});

			SimpleEventHandler EventHandler;
			EventManager->template RegisterHandler<EventTestData>(CustomEvent1, &EventHandler, &SimpleEventHandler::HandleEvent);

			FVec3 Grav(0, 0, -1);
			Scene.SetUpForFrame(&Grav, 1, 0, 99999, 99999, 10, false);
			Scene.StartFrame();
			Scene.EndFrame();

			EXPECT_EQ(EventHandler.ResultFromHandler, TestData);
		}
	}

	GTEST_TEST(AllTraits, EventTestBuffer_SeveralTicks)
	{
		for (EMultiBufferMode BufferMode : {EMultiBufferMode::Single, EMultiBufferMode::Double, EMultiBufferMode::Triple})
		{
			const FReal FixedDT = 1;
			FChaosScene Scene(nullptr, FixedDT);
			Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

			Chaos::FEventManager* EventManager = Scene.GetSolver()->GetEventManager();
			EventManager->SetBufferMode(BufferMode);

			Chaos::FPBDRigidsSolver* Solver = Scene.GetSolver();

			EventTestData TestData;
			TestData.Data1 = 0;
			TestData.Data2 = FVector(0, 0, 0);
			EventTestData* TestDataPtr = &TestData;
			EventManager->template RegisterEvent<EventTestData>(CustomEvent1, [TestDataPtr]
			(const auto* Solver, EventTestData& MyData, bool ResetData)
				{
					MyData = *TestDataPtr;
				});


			SimpleEventHandler EventHandler;
			EventHandler.ResultFromHandler = TestData;
			EventManager->template RegisterHandler<EventTestData>(CustomEvent1, &EventHandler, &SimpleEventHandler::HandleEvent);

			FVec3 Grav(0, 0, -1);
			Scene.SetUpForFrame(&Grav, 1, 0, 99999, 99999, 10, false);


			for (int32 Index = 0; Index < 3; Index++)
			{
				TestData.Data1 += 1;
				TestData.Data2 += FVec3(1);
				Scene.StartFrame();
				Scene.EndFrame();
				EXPECT_EQ(EventHandler.ResultFromHandler, TestData);
			}
		}
	}
}


