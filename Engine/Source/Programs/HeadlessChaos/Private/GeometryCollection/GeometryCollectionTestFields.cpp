// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestFields.h"

#include "Chaos/Real.h"
#include "Chaos/Utilities.h"
#include "Field/FieldSystem.h"
#include "Field/FieldSystemNodes.h"
#include "Field/FieldSystemCoreAlgo.h"
#include "Field/FieldSystemNoiseAlgo.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/TransformCollection.h"
#include "Misc/FileHelper.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

DEFINE_LOG_CATEGORY_STATIC(GCTF_Log, Verbose, All);

namespace GeometryCollectionTest
{
	void Fields_NoiseSample()
	{
		int Bounds = 100;
		FFieldExecutionDatas ExecutionDatas;
		
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, Bounds*Bounds);
		ExecutionDatas.SamplePositions.Init(FVector(0.f), Bounds*Bounds);
		for (int32 Index = 0, i=0; i < Bounds; i++)
		{
			for (int32 j = 0; j < Bounds; j++)
			{
				ExecutionDatas.SamplePositions[Index] = 1000.*FVector(i,j,0);
				Index++;
			}
		}

		Chaos::FReal MinDoimain = -1.0, MaxDomain = 1.0;
		FTransform Transform(FQuat::MakeFromEuler(FVector(45,45,45)), FVector(100, 0, 0), FVector(2, 1, 1));
		FNoiseField * NoiseField = new FNoiseField(MinDoimain, MaxDomain,Transform);

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(), 
			0.0
		};

		TArray<float> ResultsArray;
		ResultsArray.Init(false, Bounds*Bounds);
		TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());
		NoiseField->Evaluate(Context, ResultsView);

		Chaos::FReal min = FLT_MAX, max = -FLT_MAX;
		double avg = 0.0;
		Chaos::Utilities::GetMinAvgMax(ResultsView, min, avg, max);
		EXPECT_GE(min, MinDoimain);
		EXPECT_LE(max, MaxDomain);
		EXPECT_LT(min, max);

		Chaos::FReal variance = Chaos::Utilities::GetVariance(ResultsView, avg);
		Chaos::FReal stdDev = Chaos::Utilities::GetStandardDeviation(variance);
		EXPECT_GT(variance, 0.0); // If variance is 0, then all values are the same.
		EXPECT_GT(stdDev, 0.0);
		EXPECT_LT(stdDev, 0.5);

		delete NoiseField;
	}

	void Fields_RadialIntMask()
	{
		FFieldExecutionDatas ExecutionDatas;

		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, 10);
		ExecutionDatas.SamplePositions.Init(FVector(0.f), 10);
		for (int32 Index = 0; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index] = FVector(Index);
		}

		FRadialIntMask * RadialMask = new FRadialIntMask(0.0, FVector(), 1.0, 0.0, ESetMaskConditionType::Field_Set_Always);
		RadialMask->Position = FVector(0.0, 0.0, 0.0);
		RadialMask->Radius = 5.0;

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<int32> ResultsArray;
		ResultsArray.Init(false, 10.0);
		TFieldArrayView<int32> ResultsView(ResultsArray, 0, ResultsArray.Num());
		RadialMask->Evaluate(Context, ResultsView);

		for (int32 Index = 0; Index < 10; Index++)
		{
			bool bValueAsBool = ResultsView[Index] != 0;
			if (Index <= 2)
			{
				EXPECT_TRUE(bValueAsBool);
			}
			else
			{
				EXPECT_FALSE(bValueAsBool);
			}
			//UE_LOGF(GCTF_Log, Error, "[%d] %d", Index, ResultsView[Index]);
		}
		delete RadialMask;
	}

	void Fields_RadialFalloff()
	{
		FFieldExecutionDatas ExecutionDatas;

		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, 10);
		ExecutionDatas.SamplePositions.Init(FVector(0.f), 10);
		for (int32 Index = 0; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index] = FVector(Index, 0, 0);
		}

		FRadialFalloff * RadialFalloff = new FRadialFalloff();
		RadialFalloff->Position = FVector(0.0, 0.0, 0.0);
		RadialFalloff->Radius = 5.0;
		RadialFalloff->Magnitude = 3.0;

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<float> ResultsArray;
		ResultsArray.Init(false, 10);
		TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());
		RadialFalloff->Evaluate(Context, ResultsView);

		for (int32 Index = 0; Index < 10; Index++)
		{
			Chaos::FReal ExpectedVal = RadialFalloff->Magnitude * Chaos::FReal(1.0 - Index / RadialFalloff->Radius);

			if (Index < RadialFalloff->Radius)
			{
				EXPECT_LT(FMath::Abs(ResultsView[Index] - ExpectedVal), KINDA_SMALL_NUMBER);
			}
			else
			{
				EXPECT_FALSE(ResultsView[Index]);
			}
			//UE_LOGF(GCTF_Log, Error, "[%d] sample:%3.5f (%3.5f,%3.5f,%3.5f) %3.5f", Index,
			//	ExpectedVal ,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ResultsView[Index]);
		}
		delete RadialFalloff;
	}

	void Fields_PlaneFalloff()
	{
		FFieldExecutionDatas ExecutionDatas;

		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, 10);
		ExecutionDatas.SamplePositions.Init(FVector(0.f), 10);
		for (int32 Index = 0; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index] = FVector(0, 0, Index-5);
		}

		FPlaneFalloff * PlaneFalloff = new FPlaneFalloff();
		PlaneFalloff->Position = FVector(0.0, 0.0, 0.0);
		PlaneFalloff->Normal = FVector(0.0, 0.0, 1.0);
		PlaneFalloff->Magnitude = 3.0;
		PlaneFalloff->Distance = 10.0;

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<float> ResultsArray;
		ResultsArray.Init(false, 10);
		TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());
		PlaneFalloff->Evaluate(Context, ResultsView);

		FPlane Plane(PlaneFalloff->Position, PlaneFalloff->Normal);
		for (int32 Index = 0; Index < 10; Index++)
		{
			Chaos::FReal ExpectedVal = 0;
			Chaos::FReal Distance = Plane.PlaneDot(ExecutionDatas.SamplePositions[Index]);
			if (Distance < -SMALL_NUMBER && Distance > -PlaneFalloff->Distance)
			{
				ExpectedVal = PlaneFalloff->Magnitude * ( 1.0 + Distance / PlaneFalloff->Distance);
			}

			EXPECT_LT(FMath::Abs(ResultsView[Index] - ExpectedVal), KINDA_SMALL_NUMBER);
		}
		delete PlaneFalloff;
	}

	void Fields_UniformVector()
	{
		FFieldExecutionDatas ExecutionDatas;
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, 10);

		ExecutionDatas.SamplePositions.Init(FVector(0.f), 10);
		for (int32 Index = 0; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index] = FVector(Index, 0, 0);
		}

		FUniformVector * UniformVector = new FUniformVector();
		UniformVector->Direction = FVector(3, 5, 7);
		UniformVector->Magnitude = 1.0;

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<FVector> ResultsArray;
		ResultsArray.Init(FVector(0), 10);
		TFieldArrayView<FVector> ResultsView(ResultsArray, 0, ResultsArray.Num());
		UniformVector->Evaluate(Context, ResultsView);

		for (int32 Index = 0; Index < 10; Index++)
		{
			FVector ExpectedVal = UniformVector->Magnitude * UniformVector->Direction;
			EXPECT_LT((ResultsView[Index] - ExpectedVal).Size(), KINDA_SMALL_NUMBER);
			//UE_LOGF(GCTF_Log, Error, "[%d] sample:(%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f)", Index,
			//	ExpectedVal.X, ExpectedVal.Y, ExpectedVal.Z,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ResultsView[Index].X, ResultsView[Index].Y, ResultsView[Index].Z);
		}
		delete UniformVector;
	}

	void Fields_RaidalVector()
	{
		FFieldExecutionDatas ExecutionDatas;
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, 10);
		ExecutionDatas.SamplePositions.Init(FVector(0.f), 10);
		for (int32 Index = 0; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index] = FVector(100.*(FMath::SRand() - 0.5), 100.*(FMath::SRand() - 0.5), 100.*(FMath::SRand() - 0.5));
		}

		FRadialVector * RadialVector = new FRadialVector();
		RadialVector->Position = FVector(3, 4, 5);
		RadialVector->Magnitude = 1.0;

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<FVector> ResultsArray;
		ResultsArray.Init(FVector(0), 10);
		TFieldArrayView<FVector> ResultsView(ResultsArray, 0, ResultsArray.Num());
		RadialVector->Evaluate(Context, ResultsView);

		for (int32 Index = 0; Index < 10; Index++)
		{
			FVector ExpectedVal = RadialVector->Magnitude * (ExecutionDatas.SamplePositions[Index] - RadialVector->Position).GetSafeNormal();
			EXPECT_LT((ResultsView[Index] - ExpectedVal).Size(), KINDA_SMALL_NUMBER);
			//UE_LOGF(GCTF_Log, Error, "[%d] sample:(%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f)", Index,
			//	ExpectedVal.X, ExpectedVal.Y, ExpectedVal.Z,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ResultsView[Index].X, ResultsView[Index].Y, ResultsView[Index].Z);
		}
		delete RadialVector;
	}

	void Fields_SumVectorFullMult()
	{
		FFieldExecutionDatas ExecutionDatas;
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, 10);
		
		Chaos::FReal AverageSampleLength = 0.0;
		ExecutionDatas.SamplePositions.Init(FVector(0.f), 10);
		for (int32 Index = 0; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index] = FVector(0);
			if (Index > 0)
			{
				ExecutionDatas.SamplePositions[Index] = FVector(100. * (FMath::SRand() - 0.5), 100. * (FMath::SRand() - 0.5), 100. * (FMath::SRand() - 0.5));
			}
			AverageSampleLength += ExecutionDatas.SamplePositions[Index].Size();
		}
		AverageSampleLength /= 10.0;

		FRadialFalloff* RadialFalloff = new FRadialFalloff();
		RadialFalloff->Position = FVector(0.0, 0.0, 0.0);
		RadialFalloff->Radius = AverageSampleLength;
		RadialFalloff->Magnitude = 3.0;

		FRadialVector* RadialVector = new FRadialVector();
		RadialVector->Position = FVector(0);
		RadialVector->Magnitude = 1.0;

		FUniformVector* UniformVector = new FUniformVector();
		UniformVector->Direction = FVector(3, 5, 7);
		UniformVector->Magnitude = 1.0;

		FSumVector* SumVector = new FSumVector(1.0, RadialFalloff, RadialVector, UniformVector, EFieldOperationType::Field_Multiply);

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<FVector> ResultsArray;
		ResultsArray.Init(FVector(0), 10);
		TFieldArrayView<FVector> ResultsView(ResultsArray, 0, ResultsArray.Num());
		SumVector->Evaluate(Context, ResultsView);

		Chaos::FReal RadialFalloffSize = RadialFalloff->Radius;
		for (int32 Index = 0; Index < 10; Index++)
		{
			FVector RightResult = UniformVector->Magnitude * UniformVector->Direction;
			FVector LeftResult = RadialVector->Magnitude * (ExecutionDatas.SamplePositions[Index] - RadialVector->Position).GetSafeNormal();
			Chaos::FReal RadialFalloffDelta = (ExecutionDatas.SamplePositions[Index] - RadialFalloff->Position).Size();
			Chaos::FReal ScalarResult = RadialFalloff->Magnitude * (1.0 - RadialFalloffDelta / RadialFalloffSize);
			if (RadialFalloffDelta >= RadialFalloffSize)
				ScalarResult = 0.f;

			FVector ExpectedVal = ScalarResult * (LeftResult * RightResult);

			//UE_LOGF(GCTF_Log, Error, "[%d:%3.5f] sample:(%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f)", Index,
			//	(ResultsView[Index] - ExpectedVal).Size(),
			//	ExpectedVal.X, ExpectedVal.Y, ExpectedVal.Z,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ResultsView[Index].X, ResultsView[Index].Y, ResultsView[Index].Z);

			EXPECT_LT((ResultsView[Index] - ExpectedVal).Size(), KINDA_SMALL_NUMBER);
		}
		delete SumVector;
	}

	void Fields_SumVectorFullDiv()
	{
		FFieldExecutionDatas ExecutionDatas;
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, 10);
		


		Chaos::FReal AverageSampleLength = 0.0;
		
		ExecutionDatas.SamplePositions.Init(FVector(0.f), 10);
		for (int32 Index = 0; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index] = FVector(0);
			if (Index > 0)
			{
				ExecutionDatas.SamplePositions[Index] = FVector(100.*(FMath::SRand() - 0.5), 100.*(FMath::SRand() - 0.5), 100.*(FMath::SRand() - 0.5));
			}
			AverageSampleLength += ExecutionDatas.SamplePositions[Index].Size();
		}
		AverageSampleLength /= 10.0;

		FRadialFalloff * RadialFalloff = new FRadialFalloff();
		RadialFalloff->Position = FVector(0.0, 0.0, 0.0);
		RadialFalloff->Radius = AverageSampleLength;
		RadialFalloff->Magnitude = 3.0;

		FRadialVector * RadialVector = new FRadialVector();
		RadialVector->Position = FVector(0);
		RadialVector->Magnitude = 1.0;

		FUniformVector * UniformVector = new FUniformVector();
		UniformVector->Direction = FVector(3, 5, 7);
		UniformVector->Magnitude = 1.0;

		FSumVector * SumVector = new FSumVector(1.0, RadialFalloff, UniformVector, RadialVector,  EFieldOperationType::Field_Divide);

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<FVector> ResultsArray;
		ResultsArray.Init(FVector(0), 10);
		TFieldArrayView<FVector> ResultsView(ResultsArray, 0, ResultsArray.Num());
		SumVector->Evaluate(Context, ResultsView);

		Chaos::FReal RadialFalloffSize = RadialFalloff->Radius;
		for (int32 Index = 0; Index < 10; Index++)
		{
			FVector RightResult = UniformVector->Magnitude * UniformVector->Direction;
			FVector LeftResult = RadialVector->Magnitude  * (ExecutionDatas.SamplePositions[Index] - RadialVector->Position).GetSafeNormal();
			Chaos::FReal RadialFalloffDelta = (ExecutionDatas.SamplePositions[Index] - RadialFalloff->Position).Size();
			Chaos::FReal ScalarResult = RadialFalloff->Magnitude * (1.0 - RadialFalloffDelta/ RadialFalloffSize);
			if (RadialFalloffDelta >= RadialFalloffSize)
				ScalarResult = 0.f;

			FVector ExpectedVal = ScalarResult * (LeftResult / RightResult);

			//UE_LOGF(GCTF_Log, Error, "[%d:%3.5f] sample:(%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f)", Index,
			//	(ResultsView[Index] - ExpectedVal).Size(),
			//	ExpectedVal.X, ExpectedVal.Y, ExpectedVal.Z,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ResultsView[Index].X, ResultsView[Index].Y, ResultsView[Index].Z);

			EXPECT_LT((ResultsView[Index] - ExpectedVal).Size(), KINDA_SMALL_NUMBER);
		}
		delete SumVector;
	}

	void Fields_SumVectorFullAdd()
	{
		FFieldExecutionDatas ExecutionDatas;
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, 10);
		
		Chaos::FReal AverageSampleLength = 0.0;
		ExecutionDatas.SamplePositions.Init(FVector(0.f), 10);
		for (int32 Index = 0; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index] = FVector(0);
			if (Index > 0)
			{
				ExecutionDatas.SamplePositions[Index] = FVector(100.*(FMath::SRand() - 0.5), 100.*(FMath::SRand() - 0.5), 100.*(FMath::SRand() - 0.5));
			}
			AverageSampleLength += ExecutionDatas.SamplePositions[Index].Size();
		}
		AverageSampleLength /= 10.0;

		FRadialFalloff * RadialFalloff = new FRadialFalloff();
		RadialFalloff->Position = FVector(0.0, 0.0, 0.0);
		RadialFalloff->Radius = AverageSampleLength;
		RadialFalloff->Magnitude = 3.0;

		FRadialVector * RadialVector = new FRadialVector();
		RadialVector->Position = FVector(0);
		RadialVector->Magnitude = 1.0;

		FUniformVector * UniformVector = new FUniformVector();
		UniformVector->Direction = FVector(3, 5, 7);
		UniformVector->Magnitude = 1.0;

		FSumVector * SumVector = new FSumVector(1.0, RadialFalloff, UniformVector, RadialVector, EFieldOperationType::Field_Add);

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<FVector> ResultsArray;
		ResultsArray.Init(FVector(0), 10);
		TFieldArrayView<FVector> ResultsView(ResultsArray, 0, ResultsArray.Num());
		SumVector->Evaluate(Context, ResultsView);

		Chaos::FReal RadialFalloffSize = RadialFalloff->Radius;
		for (int32 Index = 0; Index < 10; Index++)
		{
			FVector RightResult = UniformVector->Magnitude * UniformVector->Direction;
			FVector LeftResult = RadialVector->Magnitude  * (ExecutionDatas.SamplePositions[Index] - RadialVector->Position).GetSafeNormal();
			Chaos::FReal RadialFalloffDelta = (ExecutionDatas.SamplePositions[Index] - RadialFalloff->Position).Size();
			Chaos::FReal ScalarResult = RadialFalloff->Magnitude * (1.0 - RadialFalloffDelta / RadialFalloffSize);
			if (RadialFalloffDelta >= RadialFalloffSize)
				ScalarResult = 0.f;

			FVector ExpectedVal = ScalarResult * (LeftResult + RightResult);

			//UE_LOGF(GCTF_Log, Error, "[%d:%3.5f] sample:(%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f)", Index,
			//	(ResultsView[Index] - ExpectedVal).Size(),
			//	ExpectedVal.X, ExpectedVal.Y, ExpectedVal.Z,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ResultsView[Index].X, ResultsView[Index].Y, ResultsView[Index].Z);

			EXPECT_LT((ResultsView[Index] - ExpectedVal).Size(), KINDA_SMALL_NUMBER);
		}
		delete SumVector;
	}

	void Fields_SumVectorFullSub()
	{
		FFieldExecutionDatas ExecutionDatas;
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, 10);

		Chaos::FReal AverageSampleLength = 0.0;
		ExecutionDatas.SamplePositions.Init(FVector(0.f), 10);
		for (int32 Index = 0; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index] = FVector(0);
			if (Index > 0)
			{
				ExecutionDatas.SamplePositions[Index] = FVector(100.*(FMath::SRand() - 0.5), 100.*(FMath::SRand() - 0.5), 100.*(FMath::SRand() - 0.5));
			}
			AverageSampleLength += ExecutionDatas.SamplePositions[Index].Size();
		}
		AverageSampleLength /= 10.0;

		FRadialFalloff * RadialFalloff = new FRadialFalloff();
		RadialFalloff->Position = FVector(0.0, 0.0, 0.0);
		RadialFalloff->Radius = AverageSampleLength;
		RadialFalloff->Magnitude = 3.0;

		FRadialVector * RadialVector = new FRadialVector();
		RadialVector->Position = FVector(0);
		RadialVector->Magnitude = 1.0;

		FUniformVector * UniformVector = new FUniformVector();
		UniformVector->Direction = FVector(3, 5, 7);
		UniformVector->Magnitude = 1.0;

		FSumVector * SumVector = new FSumVector(1.0, RadialFalloff, UniformVector, RadialVector, EFieldOperationType::Field_Substract);

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<FVector> ResultsArray;
		ResultsArray.Init(FVector(0), 10);
		TFieldArrayView<FVector> ResultsView(ResultsArray, 0, ResultsArray.Num());
		SumVector->Evaluate(Context, ResultsView);

		Chaos::FReal RadialFalloffSize = RadialFalloff->Radius;
		for (int32 Index = 0; Index < 10; Index++)
		{
			FVector RightResult = UniformVector->Magnitude * UniformVector->Direction;
			FVector LeftResult = RadialVector->Magnitude  * (ExecutionDatas.SamplePositions[Index] - RadialVector->Position).GetSafeNormal();
			Chaos::FReal RadialFalloffDelta = (ExecutionDatas.SamplePositions[Index] - RadialFalloff->Position).Size();
			Chaos::FReal ScalarResult = RadialFalloff->Magnitude * (1.0 - RadialFalloffDelta / RadialFalloffSize);
			if (RadialFalloffDelta >= RadialFalloffSize)
				ScalarResult = 0.0;

			FVector ExpectedVal = ScalarResult * (LeftResult - RightResult);

			//UE_LOGF(GCTF_Log, Error, "[%d:%3.5f] sample:(%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f)", Index,
			//	(ResultsView[Index] - ExpectedVal).Size(),
			//	ExpectedVal.X, ExpectedVal.Y, ExpectedVal.Z,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ResultsView[Index].X, ResultsView[Index].Y, ResultsView[Index].Z);

			EXPECT_LT((ResultsView[Index] - ExpectedVal).Size(), KINDA_SMALL_NUMBER);
		}
		delete SumVector;
	}

	void Fields_SumVectorLeftSide()
	{
		FFieldExecutionDatas ExecutionDatas;
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, 10);

		Chaos::FReal AverageSampleLength = 0.0;
		ExecutionDatas.SamplePositions.Init(FVector(0.f), 10);
		for (int32 Index = 0; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index] = FVector(0);
			if (Index > 0)
			{
				ExecutionDatas.SamplePositions[Index] = FVector(100.*(FMath::SRand() - 0.5), 100.*(FMath::SRand() - 0.5), 100.*(FMath::SRand() - 0.5));
			}
			AverageSampleLength += ExecutionDatas.SamplePositions[Index].Size();
		}
		AverageSampleLength /= 10.0;

		FRadialFalloff * RadialFalloff = new FRadialFalloff();
		RadialFalloff->Position = FVector(0.0, 0.0, 0.0);
		RadialFalloff->Radius = AverageSampleLength;
		RadialFalloff->Magnitude = 3.0;

		FRadialVector * RadialVector = new FRadialVector();
		RadialVector->Position = FVector(0);
		RadialVector->Magnitude = 1.0;

		FUniformVector * UniformVector = new FUniformVector();
		UniformVector->Direction = FVector(3, 5, 7);
		UniformVector->Magnitude = 1.0;

		FSumVector * SumVector = new FSumVector(1.0, RadialFalloff, nullptr, RadialVector, EFieldOperationType::Field_Multiply);

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<FVector> ResultsArray;
		ResultsArray.Init(FVector(0), 10);
		TFieldArrayView<FVector> ResultsView(ResultsArray, 0, ResultsArray.Num());
		SumVector->Evaluate(Context, ResultsView);

		Chaos::FReal RadialFalloffSize = RadialFalloff->Radius;
		for (int32 Index = 0; Index < 10; Index++)
		{
			FVector RightResult = UniformVector->Magnitude * UniformVector->Direction;
			FVector LeftResult = RadialVector->Magnitude  * (ExecutionDatas.SamplePositions[Index] - RadialVector->Position).GetSafeNormal();
			Chaos::FReal RadialFalloffDelta = (ExecutionDatas.SamplePositions[Index] - RadialFalloff->Position).Size();
			Chaos::FReal ScalarResult = RadialFalloff->Magnitude * (1.0 - RadialFalloffDelta / RadialFalloffSize);
			if (RadialFalloffDelta >= RadialFalloffSize)
				ScalarResult = 0.f;

			FVector ExpectedVal = ScalarResult * (LeftResult);

			//UE_LOGF(GCTF_Log, Error, "[%d:%3.5f] sample:(%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f)", Index,
			//	(ResultsView[Index] - ExpectedVal).Size(),
			//	ExpectedVal.X, ExpectedVal.Y, ExpectedVal.Z,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ResultsView[Index].X, ResultsView[Index].Y, ResultsView[Index].Z);

			EXPECT_LT((ResultsView[Index] - ExpectedVal).Size(), KINDA_SMALL_NUMBER);
		}
		delete SumVector;
		delete UniformVector;
	}

	void Fields_SumVectorRightSide()
	{
		FFieldExecutionDatas ExecutionDatas;
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, 10);

		Chaos::FReal AverageSampleLength = 0.0;
		ExecutionDatas.SamplePositions.Init(FVector(0.f), 10);
		for (int32 Index = 0; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index] = FVector(0);
			if (Index > 0)
			{
				ExecutionDatas.SamplePositions[Index] = FVector(100.*(FMath::SRand() - 0.5), 100.*(FMath::SRand() - 0.5), 100.*(FMath::SRand() - 0.5));
			}
			AverageSampleLength += ExecutionDatas.SamplePositions[Index].Size();
		}
		AverageSampleLength /= 10.0;

		FRadialFalloff * RadialFalloff = new FRadialFalloff();
		RadialFalloff->Position = FVector(0.0, 0.0, 0.0);
		RadialFalloff->Radius = AverageSampleLength;
		RadialFalloff->Magnitude = 3.0;

		FRadialVector * RadialVector = new FRadialVector();
		RadialVector->Position = FVector(0);
		RadialVector->Magnitude = 1.0;

		FUniformVector * UniformVector = new FUniformVector();
		UniformVector->Direction = FVector(3, 5, 7);
		UniformVector->Magnitude = 1.0;

		FSumVector * SumVector = new FSumVector(1.0, RadialFalloff, UniformVector, nullptr, EFieldOperationType::Field_Multiply);

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<FVector> ResultsArray;
		ResultsArray.Init(FVector(0), 10);
		TFieldArrayView<FVector> ResultsView(ResultsArray, 0, ResultsArray.Num());
		SumVector->Evaluate(Context, ResultsView);

		Chaos::FReal RadialFalloffSize = RadialFalloff->Radius;
		for (int32 Index = 0; Index < 10; Index++)
		{
			FVector RightResult = UniformVector->Magnitude * UniformVector->Direction;
			FVector LeftResult = RadialVector->Magnitude  * (ExecutionDatas.SamplePositions[Index] - RadialVector->Position).GetSafeNormal();
			Chaos::FReal RadialFalloffDelta = (ExecutionDatas.SamplePositions[Index] - RadialFalloff->Position).Size();
			Chaos::FReal ScalarResult = RadialFalloff->Magnitude * (1.0 - RadialFalloffDelta / RadialFalloffSize);
			if (RadialFalloffDelta >= RadialFalloffSize)
				ScalarResult = 0.0;

			FVector ExpectedVal = ScalarResult * (RightResult);

			//UE_LOGF(GCTF_Log, Error, "[%d:%3.5f] sample:(%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f) (%3.5f,%3.5f,%3.5f)", Index,
			//	(ResultsView[Index] - ExpectedVal).Size(),
			//	ExpectedVal.X, ExpectedVal.Y, ExpectedVal.Z,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ResultsView[Index].X, ResultsView[Index].Y, ResultsView[Index].Z);

			EXPECT_LT((ResultsView[Index] - ExpectedVal).Size(), KINDA_SMALL_NUMBER);
		}
		delete SumVector;
		delete RadialVector;
	}

	void Fields_SumScalar()
	{
		int32 NumPoints = 20;
		FFieldExecutionDatas ExecutionDatas;
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumPoints);
		
		ExecutionDatas.SamplePositions.Init(FVector(0.f), NumPoints);
		for (int32 Index = -10; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index + 10] = FVector(Index, 0, 0);
		}
		
		FRadialFalloff * RadialFalloff = new FRadialFalloff();
		RadialFalloff->Position = FVector(5.0, 0.0, 0.0);
		RadialFalloff->Radius = 10;
		RadialFalloff->Magnitude = 3.0;
		Chaos::FReal RadialFalloffRadius = RadialFalloff->Radius;

		FRadialFalloff * RadialFalloff2 = new FRadialFalloff();
		RadialFalloff2->Position = FVector(5.0, 0.0, 0.0);
		RadialFalloff2->Radius = 10;
		RadialFalloff2->Magnitude = 3.0;
		Chaos::FReal RadialFalloff2Radius = RadialFalloff2->Radius;

		FSumScalar * SumScalar = new FSumScalar(1.f, RadialFalloff, RadialFalloff2,EFieldOperationType::Field_Multiply);

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<float> ResultsArray;
		ResultsArray.Init(0.f, ExecutionDatas.SamplePositions.Num());
		TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());
		SumScalar->Evaluate(Context, ResultsView);

		for (int32 Index = 0; Index < NumPoints; Index++)
		{
			Chaos::FReal ScalarRight = 0.0;
			{// FRadialFalloff::Evaluate
				Chaos::FReal  RadialFalloffDelta = (RadialFalloff->Position - ExecutionDatas.SamplePositions[Index]).Size();
				if (RadialFalloffDelta < RadialFalloffRadius)
				{
					ScalarRight = RadialFalloff->Magnitude * (1.0 - RadialFalloffDelta / RadialFalloffRadius );
				}
			}

			Chaos::FReal ScalarLeft = 0.0;
			{ //  FRadialFalloff2::Evaluate
				Chaos::FReal RadialFalloff2Delta = (RadialFalloff2->Position - ExecutionDatas.SamplePositions[Index]).Size();
				if (RadialFalloff2Delta < RadialFalloff2Radius)
				{
					ScalarLeft = RadialFalloff2->Magnitude * (1.0 - RadialFalloff2Delta / RadialFalloff2Radius);
				}
			}

			Chaos::FReal ExpectedVal = ScalarLeft * ScalarRight;

			//UE_LOGF(GCTF_Log, Error, "[%d:%3.5f] sample:(%3.5f,%3.5f,%3.5f) %3.5f -> %3.5f", Index,
			//	ResultsView[Index] - ExpectedVal,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ExpectedVal, ResultsView[Index]);

			EXPECT_LT(FMath::Abs(ResultsView[Index] - ExpectedVal), KINDA_SMALL_NUMBER);
		}
		delete SumScalar;
	}

	void Fields_SumScalarRightSide()
	{
		int32 NumPoints = 20;
		FFieldExecutionDatas ExecutionDatas;
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumPoints);
		
		ExecutionDatas.SamplePositions.Init(FVector(0.f), NumPoints);
		for (int32 Index = -10; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index + 10] = FVector(Index, 0, 0);
		}

		FRadialFalloff * RadialFalloff = new FRadialFalloff();
		RadialFalloff->Position = FVector(5.0, 0.0, 0.0);
		RadialFalloff->Radius = 10;
		RadialFalloff->Magnitude = 3.0;
		Chaos::FReal RadialFalloffRadius = RadialFalloff->Radius;

		FSumScalar * SumScalar = new FSumScalar(1.f, RadialFalloff, nullptr, EFieldOperationType::Field_Multiply);

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<float> ResultsArray;
		ResultsArray.Init(0.f, ExecutionDatas.SamplePositions.Num());
		TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());
		SumScalar->Evaluate(Context, ResultsView);

		for (int32 Index = 0; Index < NumPoints; Index++)
		{
			Chaos::FReal ScalarRight = 0.0;
			{// FRadialFalloff::Evaluate
				Chaos::FReal RadialFalloffDelta = (RadialFalloff->Position - ExecutionDatas.SamplePositions[Index]).Size();
				if (RadialFalloffDelta < RadialFalloffRadius)
				{
					ScalarRight = RadialFalloff->Magnitude * (1.0 - RadialFalloffDelta / RadialFalloffRadius);
				}
			}

			Chaos::FReal ScalarLeft = 1.0;
			Chaos::FReal ExpectedVal = ScalarLeft * ScalarRight;

			//UE_LOGF(GCTF_Log, Error, "[%d:%3.5f] sample:(%3.5f,%3.5f,%3.5f) %3.5f -> %3.5f", Index,
			//	ResultsView[Index] - ExpectedVal,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ExpectedVal, ResultsView[Index]);

			EXPECT_LT(FMath::Abs(ResultsView[Index] - ExpectedVal), KINDA_SMALL_NUMBER);
		}
		delete SumScalar;
	}

	void Fields_SumScalarLeftSide()
	{
		int32 NumPoints = 20;
		FFieldExecutionDatas ExecutionDatas;
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumPoints);

		ExecutionDatas.SamplePositions.Init(FVector(0.f), NumPoints);
		for (int32 Index = -10; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index + 10] = FVector(Index, 0, 0);
		}

		FRadialFalloff * RadialFalloff = new FRadialFalloff();
		RadialFalloff->Position = FVector(5.0, 0.0, 0.0);
		RadialFalloff->Radius = 10;
		RadialFalloff->Magnitude = 3.0;
		Chaos::FReal RadialFalloffRadius = RadialFalloff->Radius;

		FSumScalar * SumScalar = new FSumScalar(1.f, nullptr, RadialFalloff, EFieldOperationType::Field_Multiply);

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<float> ResultsArray;
		ResultsArray.Init(0.f, ExecutionDatas.SamplePositions.Num());
		TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());
		SumScalar->Evaluate(Context, ResultsView);

		for (int32 Index = 0; Index < NumPoints; Index++)
		{
			Chaos::FReal ScalarRight = 1.0;

			Chaos::FReal ScalarLeft = 0.0;
			{ //  FRadialFalloff2::Evaluate
				Chaos::FReal RadialFalloffDelta = (RadialFalloff->Position - ExecutionDatas.SamplePositions[Index]).Size();
				if (RadialFalloffDelta < RadialFalloffRadius)
				{
					ScalarLeft = RadialFalloff->Magnitude * (1.0 - RadialFalloffDelta / RadialFalloffRadius );
				}
			}

			Chaos::FReal ExpectedVal = ScalarLeft * ScalarRight;

			//UE_LOGF(GCTF_Log, Error, "[%d:%3.5f] sample:(%3.5f,%3.5f,%3.5f) %3.5f -> %3.5f", Index,
			//	ResultsView[Index] - ExpectedVal,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ExpectedVal, ResultsView[Index]);

			EXPECT_LT(FMath::Abs(ResultsView[Index] - ExpectedVal), KINDA_SMALL_NUMBER);
		}
		delete SumScalar;
	}

	void Fields_Culling()
	{
		int32 NumPoints = 20;
		FFieldExecutionDatas ExecutionDatas;
		FFieldContextIndex::ContiguousIndices(ExecutionDatas.SampleIndices, NumPoints);

		ExecutionDatas.SamplePositions.Init(FVector(0.f), NumPoints);
		for (int32 Index = -10; Index < 10; Index++)
		{
			ExecutionDatas.SamplePositions[Index + 10] = FVector(Index, 0, 0);
		}

		FRadialFalloff * RadialFalloff = new FRadialFalloff();
		RadialFalloff->Position = FVector(0.0, 0.0, 0.0);
		RadialFalloff->Radius = 4.;
		RadialFalloff->Magnitude = 3.0;
		Chaos::FReal RadialFalloffRadius = RadialFalloff->Radius;

		FRadialFalloff * RadialFalloff2 = new FRadialFalloff();
		RadialFalloff2->Position = FVector(5.0, 0.0, 0.0);
		RadialFalloff2->Radius = 10;
		RadialFalloff2->Magnitude = 3.0;
		Chaos::FReal RadialFalloff2Radius = RadialFalloff2->Radius;

		FCullingField<float> * CullingField = new FCullingField<float>(RadialFalloff, RadialFalloff2, EFieldCullingOperationType::Field_Culling_Outside);

		FFieldContext Context{
			ExecutionDatas,
			FFieldContext::UniquePointerMap(),
			0.0
		};

		TArray<float> ResultsArray;
		ResultsArray.Init(0.f, ExecutionDatas.SamplePositions.Num());
		TFieldArrayView<float> ResultsView(ResultsArray, 0, ResultsArray.Num());
		CullingField->Evaluate(Context, ResultsView);

		for (int32 Index = 0; Index < NumPoints; Index++)
		{
			Chaos::FReal ScalarRight = 1.0;
			{ 
				Chaos::FReal RadialFalloffDelta = (RadialFalloff->Position - ExecutionDatas.SamplePositions[Index]).Size();
				if (RadialFalloffDelta >= RadialFalloffRadius)
				{
					ScalarRight = 0.0;
				}
			}

			Chaos::FReal ScalarLeft = 0.0;
			{ //  FCullingField::Evaluate
				Chaos::FReal RadialFalloff2Delta = (RadialFalloff2->Position - ExecutionDatas.SamplePositions[Index]).Size();
				if (RadialFalloff2Delta < RadialFalloff2Radius)
				{
					ScalarLeft = RadialFalloff2->Magnitude * (1.0 - RadialFalloff2Delta / RadialFalloff2Radius);
				}
			}

			Chaos::FReal ExpectedVal = ScalarLeft * ScalarRight;

			//UE_LOGF(GCTF_Log, Error, "[%d:%3.5f] sample:(%3.5f,%3.5f,%3.5f) %3.5f -> %3.5f", Index,
			//	ResultsView[Index] - ExpectedVal,
			//	ExecutionDatas.SamplePositions[Index].X, ExecutionDatas.SamplePositions[Index].Y, ExecutionDatas.SamplePositions[Index].Z,
			//	ExpectedVal, ResultsView[Index]);

			EXPECT_LT(FMath::Abs(ResultsView[Index] - ExpectedVal), KINDA_SMALL_NUMBER);
		}
		delete CullingField;
	}

	FFieldSystemCommand SaveAndLoad(FFieldSystemCommand & CommandOut)
	{
		FBufferArchive Ar;
		Ar.SetIsSaving(true);
		Ar.SetIsLoading(false);
		FString Filename = "Fields_SerializeAPI.tmp";

		FFieldSystemCommand CommandIn;

		CommandOut.Serialize(Ar);
		FFileHelper::SaveArrayToFile(Ar, *Filename);
		Ar.FlushCache();
		Ar.Empty();

		TArray<uint8> InputArray;
		FFileHelper::LoadFileToArray(InputArray, *Filename);
		FMemoryReader InputArchive = FMemoryReader(InputArray, true);
		InputArchive.Seek(0);
		CommandIn.Serialize(InputArchive);

		return CommandIn;
	}

	// Helper for testing FFieldSystemCommand equality. Defined very similarly to the == operator but with more granular
	// checks at each stage of testing. 
	bool TestFFieldSystemEquality(const FFieldSystemCommand& CommandIn, const FFieldSystemCommand& CommandOut)
	{
		EXPECT_TRUE(CommandOut.TargetAttribute.IsEqual(CommandIn.TargetAttribute));
		if (CommandOut.TargetAttribute.IsEqual(CommandIn.TargetAttribute))
		{
			EXPECT_EQ(CommandOut.RootNode.IsValid(), CommandIn.RootNode.IsValid());
			if (CommandOut.RootNode.IsValid() == CommandIn.RootNode.IsValid())
			{
				if (CommandOut.RootNode.IsValid())
				{
					EXPECT_EQ(CommandOut.RootNode->SerializationType(), CommandIn.RootNode->SerializationType());
					if (CommandOut.RootNode->SerializationType() == CommandIn.RootNode->SerializationType())
					{
						EXPECT_TRUE(CommandOut.RootNode->operator==(*CommandIn.RootNode));
						if (CommandOut.RootNode->operator==(*CommandIn.RootNode))
						{
							return true;
						}
					}
				}
				else
				{
					return true;
				}
			}
		}
		return false;
	}

	void Fields_SerializeAPI()
	{
		{
			FFieldSystemCommand CommandOut("FUniformInteger", new FUniformInteger(3));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

		{
			FFieldSystemCommand CommandOut("FRadialIntMask", new FRadialIntMask(1.f,FVector(3,5,7),11,13) );
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

		{
			FFieldSystemCommand CommandOut("FUniformScalar", new FUniformScalar(13));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

		{
			FFieldSystemCommand CommandOut("FRadialFalloff", new FRadialFalloff(1.f,3.f,5.f,7.f,11.f,FVector(13, 17, 19)));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

		{
			FFieldSystemCommand CommandOut("FPlaneFalloff", new FPlaneFalloff(1.f, 3.f, 5.f, 7.f, 100.f, FVector(9, 11, 13), FVector(17, 19, 23)));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}
		
		{
			FFieldSystemCommand CommandOut("FBoxFalloff", new FBoxFalloff(1.f, 7.f, 9.f, 13.f, FTransform::Identity));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}
		
		{
			FFieldSystemCommand CommandOut("FNoiseField", new	FNoiseField(1.f, 3.f));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

		{
			FFieldSystemCommand CommandOut("FUniformVector", new FUniformVector(1.f, FVector(3, 5, 7)));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

		{
			FFieldSystemCommand CommandOut("FRadialVector", new FRadialVector(1.f,FVector(3, 5, 7)));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

		{
			FFieldSystemCommand CommandOut("FRandomVector", new FRandomVector(1.f));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

		{
			FUniformScalar* UniformScalar = new FUniformScalar(1.f);
			FRadialFalloff* RadialScalar = new FRadialFalloff(1.f, 3.f, 5.f, 7.f, 11.f, FVector(13, 17, 19));
			FFieldSystemCommand CommandOut("FSumScalar", new FSumScalar(1.f, UniformScalar, RadialScalar, EFieldOperationType::Field_Substract));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

		{
			FUniformScalar* UniformScalar = new FUniformScalar(41.f);
			FUniformVector* UniformVector = new FUniformVector(31.f, FVector(3, 5, 7));
			FRadialVector* RadialVector = new FRadialVector(21.f, FVector(3, 5, 7));
			FFieldSystemCommand CommandOut("FSumVector", new FSumVector(1.f, UniformScalar, UniformVector, RadialVector, EFieldOperationType::Field_Divide));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

		// conversion fields
		{
			FUniformScalar* UniformScalar = new FUniformScalar(41.f);
			FFieldSystemCommand CommandOut("FConversionField", new FConversionField<float,int32>(UniformScalar));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}
		{
			FUniformInteger* UniformInteger = new FUniformInteger(3);
			FFieldSystemCommand CommandOut("FConversionField", new FConversionField<int32, float>(UniformInteger));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

		// culling fields
		{
			FUniformInteger* UniformInteger = new FUniformInteger(3);
			FRadialFalloff* RadialScalar = new FRadialFalloff(1.f, 3.f, 5.f, 7.f, 11.f, FVector(13, 17, 19));
			FFieldSystemCommand CommandOut("FCullingField", new FCullingField<int32>(RadialScalar,UniformInteger));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}
		{
			FUniformScalar* UniformScalar = new FUniformScalar(3.f);
			FRadialFalloff* RadialScalar = new FRadialFalloff(1.f, 3.f, 5.f, 7.f, 11.f, FVector(13, 17, 19));
			FFieldSystemCommand CommandOut("FCullingField", new FCullingField<float>(RadialScalar, UniformScalar));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}
		{
			FUniformVector* UniformVector = new FUniformVector(3.f);
			FRadialFalloff* RadialScalar = new FRadialFalloff(1.f, 3.f, 5.f, 7.f, 11.f, FVector(13, 17, 19));
			FFieldSystemCommand CommandOut("FCullingField", new FCullingField<FVector>(RadialScalar, UniformVector));
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

		// terminals
		{
			FFieldSystemCommand CommandOut("FReturnResultsTerminal", new FReturnResultsTerminal<int32>());
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}
		{
			FFieldSystemCommand CommandOut("FReturnResultsTerminal", new FReturnResultsTerminal<float>());
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}
		{
			FFieldSystemCommand CommandOut("FReturnResultsTerminal", new FReturnResultsTerminal<FVector>());
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}


		// depth test with lots of nodes.
		{
			FUniformScalar* UniformScalar = new FUniformScalar(3);
			FConversionField<float, int32>* ConversionFieldFI = new FConversionField<float, int32>(UniformScalar);

			FBoxFalloff* BoxFalloff = new FBoxFalloff(1.f, 7.f, 9.f, 13.f, FTransform::Identity);
			FCullingField<int32>* CullingFieldI = new FCullingField<int32>(BoxFalloff, ConversionFieldFI);

			FUniformInteger* UniformInteger = new FUniformInteger(3);
			FConversionField<int32, float>*  ConversionFieldIF = new FConversionField<int32, float>(UniformInteger);

			FPlaneFalloff* PlaneFalloff = new FPlaneFalloff(1.f, 3.f, 5.f, 7.f, 100.f, FVector(9, 11, 13), FVector(17, 19, 23));
			FCullingField<float>* CullingFieldF = new FCullingField<float>(PlaneFalloff, ConversionFieldIF);


			FNoiseField* NoiseField2 = new FNoiseField(1.f, 3.f);
			FRandomVector* RandomVector = new FRandomVector(1.f);
			FCullingField<FVector>* CullingFieldV = new FCullingField<FVector>(NoiseField2, RandomVector);

			FNoiseField* NoiseField = new FNoiseField(1.f, 3.f);
			FSumScalar* SumScalar = new FSumScalar(1.f, CullingFieldF, NoiseField, EFieldOperationType::Field_Substract);

			FUniformVector* UniformVector = new FUniformVector(1.f, FVector(3, 5, 7));
			FSumVector* SumVector = new FSumVector(1.f, SumScalar, CullingFieldV, UniformVector, EFieldOperationType::Field_Divide);

			FReturnResultsTerminal<int32>*  ReturnResultsTerminalI = new FReturnResultsTerminal<int32>();
			FConversionField<int32, float>*  ConversionFieldIF2 = new FConversionField<int32, float>(ReturnResultsTerminalI);
			FReturnResultsTerminal<float>* ReturnResultsTerminalF = new FReturnResultsTerminal<float>();
			FSumScalar* SumScalar2 = new FSumScalar(1.f, ReturnResultsTerminalF, ConversionFieldIF2, EFieldOperationType::Field_Substract);


			FReturnResultsTerminal<FVector>* ReturnResultsTerminalV =new FReturnResultsTerminal<FVector>();
			FSumVector* SumVector2 = new FSumVector(1.f, SumScalar2, ReturnResultsTerminalV, SumVector, EFieldOperationType::Field_Divide);

			FReturnResultsTerminal<FVector>* ReturnResultsTerminalV2 = new FReturnResultsTerminal<FVector>();
			FConversionField<int32, float>*  ConversionFieldIF3 = new FConversionField<int32, float>(CullingFieldI);
			FSumVector* SumVector3 = new FSumVector(1.f, ConversionFieldIF3, SumVector2, ReturnResultsTerminalV2, EFieldOperationType::Field_Divide);

			FFieldSystemCommand CommandOut("DeepTreeOfEverything", SumVector3);
			FFieldSystemCommand CommandIn = SaveAndLoad(CommandOut);
			EXPECT_TRUE(TestFFieldSystemEquality(CommandIn, CommandOut));
		}

	}
}
