// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestFramework.h"
#include "GeometryCollection/GeometryCollectionTestUtility.h"

#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"

#include "Generators/SphereGenerator.h"
#include "Generators/GridBoxMeshGenerator.h"

#include "HeadlessChaosTestUtility.h"
#include "ChaosSolversModule.h"
#include "Chaos/ErrorReporter.h"

#include "../Resource/SphereGeometry.h"

namespace GeometryCollectionTest
{
	TSharedPtr<FGeometryCollection> MakeSphereElement(
		FTransform RootTranform, FTransform GeomTransform,
		const int NumberOfMaterials = 2)
	{
		UE::Geometry::FSphereGenerator SphereGen;
		SphereGen.Radius = 1.f;
		SphereGen.NumPhi = 16;		// Vertical divisions
		SphereGen.NumTheta = 16;	// Horizontal divisions
		SphereGen.Generate();

		// FSphereGenerator's points drift off the surface just a bit, so we correct for that.
		Chaos::FSphere Sphere(Chaos::FVec3(0), SphereGen.Radius);
		Chaos::FVec3 Normal;
		for (int32 Idx = 0; Idx < SphereGen.Vertices.Num(); Idx++)
		{
			auto& SrcPt = SphereGen.Vertices[Idx]; // double precision
			Chaos::FVec3 Pt(SrcPt[0], SrcPt[1], SrcPt[2]); // single precision
			const Chaos::FReal Phi = Sphere.PhiWithNormal(Pt, Normal);
			SrcPt[0] -= Phi * Normal[0];
			SrcPt[1] -= Phi * Normal[1];
			SrcPt[2] -= Phi * Normal[2];

			// Ensure all the normals are pointing the right direction
			check(FVector::DotProduct(FVector(SphereGen.Normals[Idx][0], SphereGen.Normals[Idx][1], SphereGen.Normals[Idx][2]), Normal) > 0.0f);
		}

		// Ensure each edge has 2 associated faces (no holes)
		TMap<TPair<int32, int32>, uint8> EdgeCount;
		for (auto& Tri : SphereGen.Triangles)
		{
			// Reverse triangle orientations
			{
				int32 Tmp = Tri[0];
				Tri[0] = Tri[1];
				Tri[1] = Tmp;
			}
			for (int32 i = 0; i < 3; i++)
			{
				int32 A = Tri[i];
				int32 B = Tri[(i + 1) % 3];
				TPair<int32, int32> Edge(
					A < B ? A : B,
					A > B ? A : B);
				if (uint8* Count = EdgeCount.Find(Edge))
				{
					(*Count)++;
				}
				else
				{
					EdgeCount.Emplace(Edge) = 1;
				}
			}
		}
		for (auto& KV : EdgeCount)
		{
			check(KV.Value == 2);
		}

		return GeometryCollection::MakeMeshElement(
			SphereGen.Vertices,
			SphereGen.Normals,
			SphereGen.Triangles,
			SphereGen.UVs,
			RootTranform, GeomTransform, NumberOfMaterials);
	}

	TSharedPtr<FGeometryCollection> MakeCubeElement(FTransform RootTranform, FTransform GeomTransform)
	{
		using FVector3i = UE::Geometry::FVector3i;

		const TArray<FVector3f> PointsIn = { FVector3f(-1,1,-1), FVector3f(1,1,-1), FVector3f(1,-1,-1), FVector3f(-1,-1,-1), FVector3f(-1,1,1), FVector3f(1,1,1), FVector3f(1,-1,1), FVector3f(-1,-1,1) };
		const TArray<FVector3f> NormalsIn = { FVector3f(-1,1,-1).GetSafeNormal(), FVector3f(1,1,-1).GetSafeNormal(), FVector3f(1,-1,-1).GetSafeNormal(), FVector3f(-1,-1,-1).GetSafeNormal(), FVector3f(-1,1,1).GetSafeNormal(), FVector3f(1,1,1).GetSafeNormal(), FVector3f(1,-1,1).GetSafeNormal(), FVector3f(-1,-1,1).GetSafeNormal() };
		const TArray<FVector3i> TrianglesIn = { FVector3i(0,1,2),FVector3i(0,2,3), FVector3i(2,1,6),FVector3i(1,5,6),FVector3i(2,6,7),FVector3i(3,2,7),FVector3i(4,7,3),FVector3i(4,0,3),FVector3i(4,1,0),FVector3i(4,5,1),FVector3i(5,4,7),FVector3i(5,7,6) };
		const TArray<FVector2f> UVsIn = { FVector2f(0.f,0.f), FVector2f(0.f,0.f), FVector2f(0.f,0.f), FVector2f(0.f,0.f),FVector2f(0.f,0.f), FVector2f(0.f,0.f), FVector2f(0.f,0.f), FVector2f(0.f,0.f) };
		return GeometryCollection::MakeMeshElement(PointsIn, NormalsIn, TrianglesIn, UVsIn, RootTranform, GeomTransform);
	}

	TSharedPtr<FGeometryCollection> MakeTetrahedronElement(FTransform RootTranform, FTransform GeomTransform)
	{
		using FVector3i = UE::Geometry::FVector3i;

		const TArray<FVector3f> PointsIn = { FVector3f(-1,1,-1), FVector3f(-1,-1,1), FVector3f(1,-1,-1), FVector3f(1,1,1) };
		const TArray<FVector3f> NormalsIn = { FVector3f(-1,1,-1).GetSafeNormal(), FVector3f(-1,-1,1).GetSafeNormal(), FVector3f(1,-1,-1).GetSafeNormal(), FVector3f(1,1,1).GetSafeNormal() };
		const TArray<FVector3i> TrianglesIn = { FVector3i(1,0,2), FVector3i(2,1,3), FVector3i(3,2,0), FVector3i(3,0,1) };
		const TArray<FVector2f> UVsIn = { FVector2f(0.f,0.f), FVector2f(0.f,0.f), FVector2f(0.f,0.f), FVector2f(0.f,0.f) };
		return GeometryCollection::MakeMeshElement(PointsIn, NormalsIn, TrianglesIn, UVsIn, RootTranform, GeomTransform);
	}

	TSharedPtr<FGeometryCollection> MakeImportedSphereElement(FTransform RootTranform, FTransform GeomTransform)
	{
		FGeometryCollection* Collection = FGeometryCollection::NewGeometryCollection(SphereGeometry::RawVertexArray, SphereGeometry::RawIndicesArray);
		TManagedArray<FVector3f>& Vertices = Collection->Vertex;
		for (int i = 0; i < Vertices.Num(); i++) Vertices[i] = FVector3f(GeomTransform.TransformPosition(FVector(Vertices[i])));
		return TSharedPtr<FGeometryCollection>(Collection);
	}


	TSharedPtr<FGeometryCollection> MakeGriddedBoxElement(
		FTransform RootTranform, FTransform GeomTransform,
		const FVector& Extents = FVector(1, 1, 1),
		const UE::Geometry::FIndex3i& EdgeVertices = UE::Geometry::FIndex3i(4, 4, 4),
		const int NumberOfMaterials = 2)
	{
		using GeomVector = UE::Math::TVector<double>;
		UE::Geometry::FGridBoxMeshGenerator BoxGen;
		BoxGen.Box = UE::Geometry::FOrientedBox3d(GeomVector::Zero(), GeomVector(Extents)); // box center, box dimensions
		BoxGen.EdgeVertices = EdgeVertices;
		BoxGen.Generate();

		return GeometryCollection::MakeMeshElement(
			BoxGen.Vertices,
			BoxGen.Normals,
			BoxGen.Triangles,
			BoxGen.UVs,
			FTransform::Identity, GeomTransform, NumberOfMaterials);
	}

	FGeometryCollectionWrapper::FGeometryCollectionWrapper() 
		: WrapperBase(WrapperType::GeometryCollection)
	{
		FChaosUserData::Set<FCustomChaosPayload>(&UserData, reinterpret_cast<FCustomChaosPayload*>(this));
	}

	WrapperBase* CommonInit(const TSharedPtr<FGeometryCollection>& RestCollection, const CreationParameters& Params)
	{
		FTransformCollection SingleTransform = FTransformCollection::SingleTransform();
		for (int i = 0; i < Params.NestedTransforms.Num(); i++)
		{
			int ChildIndex = RestCollection->NumElements(FTransformCollection::TransformGroup) - 1;
			int32 ParentIndex = RestCollection->AppendTransform(SingleTransform, Params.NestedTransforms[0]);
			RestCollection->ParentTransforms(ParentIndex, ChildIndex);
		}

		Chaos::FMaterialHandle NewHandle = Chaos::FPhysicalMaterialManager::Get().Create();
		InitMaterialToZero(NewHandle.Get());
		Chaos::FPhysicalMaterialManager::Get().UpdateMaterial(NewHandle);

		TSharedPtr<FGeometryDynamicCollection> DynamicCollection = GeometryCollectionToGeometryDynamicCollection(RestCollection, Params.DynamicState);

		FGeometryCollectionWrapper* Wrapper = new FGeometryCollectionWrapper();

		FSimulationParameters SimulationParams;
		{
			SimulationParams.RestCollectionShared = RestCollection;
			SimulationParams.PhysicalMaterialHandle = NewHandle;
			SimulationParams.Shared.Mass = Params.Mass;
			SimulationParams.Shared.bMassAsDensity = Params.bMassAsDensity;
			SimulationParams.Shared.SizeSpecificData[0].CollisionShapesData[0].CollisionType = Params.CollisionType;
			SimulationParams.Shared.SizeSpecificData[0].CollisionShapesData[0].ImplicitType = Params.ImplicitType;
			SimulationParams.Shared.SizeSpecificData[0].CollisionShapesData[0].LevelSetData.MinLevelSetResolution = Params.MinLevelSetResolution;
			SimulationParams.Shared.SizeSpecificData[0].CollisionShapesData[0].LevelSetData.MaxLevelSetResolution = Params.MaxLevelSetResolution;
			SimulationParams.Shared.SizeSpecificData[0].CollisionShapesData[0].ImplicitType = Params.ImplicitType;
			SimulationParams.Shared.SizeSpecificData[0].CollisionShapesData[0].CollisionParticleData.CollisionParticlesFraction = Params.CollisionParticleFraction;
			SimulationParams.CollisionSampleFraction = Params.CollisionParticleFraction; // this should not be here, must remove/replace SimulationParams.CollisionSampleFraction in other files.
			SimulationParams.Simulating = Params.Simulating;
			SimulationParams.EnableClustering = Params.EnableClustering;
			SimulationParams.InitialLinearVelocity = Params.InitialLinearVelocity;
			SimulationParams.InitialVelocityType = Params.InitialVelocityType;
			SimulationParams.DamageThreshold = Params.DamageThreshold;
			SimulationParams.MaxClusterLevel = Params.MaxClusterLevel;
			SimulationParams.MaxSimulatedLevel = Params.MaxSimulatedLevel;
			SimulationParams.ClusterConnectionMethod = Params.ClusterConnectionMethod;
			SimulationParams.CollisionGroup = Params.CollisionGroup;
			SimulationParams.ClusterGroupIndex = Params.ClusterGroupIndex;
			SimulationParams.LinearDamping = 0;
			SimulationParams.AngularDamping = 0;
			SimulationParams.UseCCD = false;
			SimulationParams.UseMACD = false;
			SimulationParams.AllowPartialIslandSleep = true;
			SimulationParams.PositionSolverIterations = 8;
			SimulationParams.VelocitySolverIterations = 2;
			SimulationParams.ProjectionSolverIterations = 1;
			SimulationParams.UserData = reinterpret_cast<void*>(&Wrapper->UserData);

			Chaos::FErrorReporter ErrorReporter;
			BuildSimulationData(ErrorReporter, *RestCollection.Get(), SimulationParams.Shared);

			SimulationParams.bUseSimplicialsWhenAvailable =
				SimulationParams.RestCollectionShared
				&& SimulationParams.RestCollectionShared->HasAttribute(FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup)
				&& SimulationParams.Shared.SizeSpecificData[0].CollisionShapesData.Num()
				&& (SimulationParams.Shared.SizeSpecificData[0].CollisionShapesData[0].CollisionType == ECollisionTypeEnum::Chaos_Surface_Volumetric);

			FGeometryCollectionPhysicsProxy::InitializeDynamicCollection(*DynamicCollection, *RestCollection, SimulationParams);
		}

		PRAGMA_DISABLE_INTERNAL_WARNINGS
		const Chaos::Filter::FShapeFilterData ShapeFilterData = Chaos::Filter::FShapeFilterBuilder::BuildBlockAll(EFilterFlags::All);
		PRAGMA_ENABLE_INTERNAL_WARNINGS
		Chaos::Filter::FInstanceData ShapeInstanceData;

		FGeometryCollectionPhysicsProxy* PhysObject =
			new FGeometryCollectionPhysicsProxy(
				nullptr,			// UObject owner
				DynamicCollection.ToSharedRef(), // Game thread collection
				SimulationParams,
				ShapeFilterData,
				ShapeInstanceData);

		Wrapper->RestCollection = RestCollection;
		Wrapper->DynamicCollection = DynamicCollection;
		Wrapper->PhysObject = PhysObject;
		return Wrapper;
	}

	template <>
	WrapperBase* TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(const CreationParameters Params)
	{
		TSharedPtr<FGeometryCollection> RestCollection;
		switch (Params.SimplicialType)
		{
		case ESimplicialType::Chaos_Simplicial_Box:
			RestCollection = MakeCubeElement(Params.RootTransform, Params.GeomTransform);
			break;
		case ESimplicialType::Chaos_Simplicial_Sphere:
			RestCollection = MakeSphereElement(Params.RootTransform, Params.GeomTransform);
			break;
		case ESimplicialType::Chaos_Simplicial_GriddleBox:
			RestCollection = MakeGriddedBoxElement(Params.RootTransform, Params.GeomTransform);
			break;
		case ESimplicialType::Chaos_Simplicial_Tetrahedron:
			RestCollection = MakeTetrahedronElement(Params.RootTransform, Params.GeomTransform);
			break;
		case ESimplicialType::Chaos_Simplicial_Imported_Sphere:
			RestCollection = MakeImportedSphereElement(Params.RootTransform, Params.GeomTransform);
			break;
		case ESimplicialType::Chaos_Simplicial_None:
			RestCollection = TSharedPtr<FGeometryCollection>(new FGeometryCollection());
			RestCollection->AddElements(1, FGeometryCollection::GeometryGroup);
			RestCollection->TransformIndex[0] = 0;
			RestCollection->InnerRadius[0] = 1.f;	// Assume sphere w/radius 1
			RestCollection->OuterRadius[0] = 1.f;	// Assume sphere w/radius 1
			RestCollection->AddElements(1, FGeometryCollection::TransformGroup);
			RestCollection->Transform[0] = FTransform3f(Params.RootTransform);
			RestCollection->Transform[0].NormalizeRotation();
			break;
		default:
			check(false); // unimplemented!
			break;
		}

		return CommonInit(RestCollection, Params);
	}

	template <>
	WrapperBase* TNewSimulationObject<GeometryType::RigidFloor>::Init(const CreationParameters Params)
	{
		TSharedPtr<Chaos::FChaosPhysicsMaterial> PhysicalMaterial = MakeShared<Chaos::FChaosPhysicsMaterial>(); InitMaterialToZero(PhysicalMaterial.Get());
		auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& Particle = Proxy->GetGameThreadAPI();
		Particle.SetGeometry(MakeImplicitObjectPtr<Chaos::TPlane<FReal, 3>>(FVector(0), FVector(0, 0, 1)));

		PRAGMA_DISABLE_INTERNAL_WARNINGS
		const Chaos::Filter::FShapeFilterData ShapeFilterData = Chaos::Filter::FShapeFilterBuilder::BuildBlockAll(EFilterFlags::All);
		Particle.SetShapeFilterData(0, ShapeFilterData);
		PRAGMA_ENABLE_INTERNAL_WARNINGS

		return new RigidBodyWrapper(PhysicalMaterial, Proxy);
	}

	template <>
	WrapperBase* TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(const CreationParameters Params)
	{
		check(Params.RestCollection.IsValid());

		return CommonInit(Params.RestCollection, Params);
	}

	FFramework::FFramework(FrameworkParameters Parameters)
	: Dt(Parameters.Dt)
	, Module(FChaosSolversModule::GetModule())
	, Solver(nullptr)
	{
		Solver = Module->CreateSolver(nullptr, Parameters.AsyncDt, Parameters.ThreadingMode);	//until refactor is done, solver must be created after thread change
		ChaosTest::InitSolverSettings(Solver);
	}

	FFramework::~FFramework()
	{
		for (WrapperBase* Object : PhysicsObjects)
		{
			if (FGeometryCollectionWrapper* GCW = Object->As<FGeometryCollectionWrapper>())
			{
				Solver->UnregisterObject(GCW->PhysObject);
			}
			else if (RigidBodyWrapper* BCW = Object->As<RigidBodyWrapper>())
			{
				Solver->UnregisterObject(BCW->Particle);
			}
		}

		FChaosSolversModule::GetModule()->DestroySolver(Solver);

		//don't delete wrapper objects until solver is gone.
		//can have callbacks that rely on wrapper objects
		for (WrapperBase* Object : PhysicsObjects)
		{
			delete Object;
		}
	}

	void FFramework::AddSimulationObject(WrapperBase* Object)
	{
		PhysicsObjects.Add(Object);
	}

	void FFramework::RemoveSimulationObject(WrapperBase* Object)
	{
		if (PhysicsObjects.Remove(Object) > 0)
		{
			if (FGeometryCollectionWrapper* GCW = Object->As<FGeometryCollectionWrapper>())
			{
				Solver->UnregisterObject(GCW->PhysObject);
			}
			else if (RigidBodyWrapper* BCW = Object->As<RigidBodyWrapper>())
			{
				Solver->UnregisterObject(BCW->Particle);
			}

			delete Object;
		}
	}

	void FFramework::Initialize()
	{
		for (WrapperBase* Object : PhysicsObjects)
		{
			if (FGeometryCollectionWrapper* GCW = Object->As<FGeometryCollectionWrapper>())
			{
				Solver->RegisterObject(GCW->PhysObject);
				Solver->AddDirtyProxy(GCW->PhysObject);
			}
			else if (RigidBodyWrapper* RBW = Object->As<RigidBodyWrapper>())
			{
				Solver->RegisterObject(RBW->Particle);
			}
		}
	}

	void FFramework::Advance()
	{
		Solver->AdvanceAndDispatch_External(Dt);
		Solver->UpdateGameThreadStructures();
		Solver->SyncEvents_GameThread();
	}

} // end namespace GeometryCollectionTest

