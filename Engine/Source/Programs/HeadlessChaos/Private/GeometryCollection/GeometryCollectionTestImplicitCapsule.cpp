// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestImplicitCapsule.h"

#include "Chaos/Capsule.h"

namespace GeometryCollectionTest
{
	using namespace Chaos;

	void RunTestComputeSamplePoints(const Chaos::FCapsule &Capsule)
	{
		const Chaos::FCapsule OACapsule = Chaos::FCapsule::NewFromOriginAndAxis(Capsule.GetOriginf(), Capsule.GetAxisf(), Capsule.GetHeightf(), Capsule.GetRadiusf());
		EXPECT_NEAR((Capsule.GetOriginf() - OACapsule.GetOriginf()).Size(), 0, KINDA_SMALL_NUMBER) << *FString("Capsule != OACapsule, origin.");
		EXPECT_NEAR((Capsule.GetInsertion() - OACapsule.GetInsertion()).Size(), 0, KINDA_SMALL_NUMBER) << *FString("Capsule != OACapsule, insertion.");
		EXPECT_NEAR((Capsule.GetAxisf() - OACapsule.GetAxisf()).Size(), 0, KINDA_SMALL_NUMBER) << *FString("Capsule != OACapsule, axis.");
		EXPECT_NEAR(Capsule.GetHeightf() - OACapsule.GetHeightf(), 0, KINDA_SMALL_NUMBER) << *FString("Capsule != OACapsule, height.");
		EXPECT_NEAR(Capsule.GetRadiusf() - OACapsule.GetRadiusf(), 0, KINDA_SMALL_NUMBER) << *FString("Capsule != OACapsule, radius.");

		FVec3 Point;
		FReal Phi;
		EXPECT_EQ(Capsule.GetType(), Chaos::ImplicitObjectType::Capsule) << *FString("Implicit object type is not 'capsule'.");

		Point = Capsule.GetAxisf();
		EXPECT_LT(FMath::Abs(Point.Size() - 1.0), KINDA_SMALL_NUMBER) << *FString("Capsule axis is not unit length.");
		
		Point = Capsule.GetOriginf() + Capsule.GetAxisf() * (Capsule.GetHeightf() + Capsule.GetRadiusf() * 2);
		EXPECT_LT((Point - Capsule.GetInsertion()).Size(), KINDA_SMALL_NUMBER) << *FString("Capsule is broken.");

		Point = Capsule.GetInsertion();// Capsule.GetOriginf() + Capsule.GetAxis() * Capsule.GetHeight();
		Phi = Capsule.SignedDistance(Point);
		EXPECT_NEAR(Phi, 0, KINDA_SMALL_NUMBER) << *FString("Capsule failed phi surface (insertion) sanity test.");

		Point = Capsule.GetOriginf() + Capsule.GetAxisf() * (Capsule.GetHeightf() + Capsule.GetRadiusf() * 2) * 0.25;
		Phi = Capsule.SignedDistance(Point);
		EXPECT_LE(Phi, (FReal)0.0) << *FString("Capsule failed phi depth (1/4 origin) sanity test.");

		Point = Capsule.GetOriginf() + Capsule.GetAxisf() * (Capsule.GetHeightf() + Capsule.GetRadiusf() * 2) * 0.75;
		Phi = Capsule.SignedDistance(Point);
		EXPECT_LE(Phi, (FReal)0.0) << *FString("Capsule failed phi depth (3/4 origin) sanity test.");

		EXPECT_NEAR((Capsule.GetCenterf() - FVec3(Capsule.GetOriginf() + Capsule.GetAxisf() * (Capsule.GetHeightf() + Capsule.GetRadiusf() * 2) * 0.5)).Size(), 0, KINDA_SMALL_NUMBER) << *FString("Capsule center is off mid axis.");

		Point = Capsule.GetCenterf();
		Phi = Capsule.SignedDistance(Point);
		EXPECT_LT(Phi, (FReal)0.0) << *FString("Capsule failed phi depth sanity test.");
		
		Point = Capsule.GetOriginf();
		Phi = Capsule.SignedDistance(Point);
		EXPECT_NEAR(FMath::Abs(Phi), 0, KINDA_SMALL_NUMBER) << *FString("Capsule failed phi surface (origin) sanity test.");

		Point = Capsule.GetInsertion();
		Phi = Capsule.SignedDistance(Point);
		EXPECT_NEAR(FMath::Abs(Phi), 0, KINDA_SMALL_NUMBER) << *FString("Capsule failed phi surface (origin+axis*height) sanity test.");

		Point = Capsule.GetOriginf() + Capsule.GetAxisf() * Capsule.GetRadiusf() + Capsule.GetAxisf().GetOrthogonalVector().GetSafeNormal() * Capsule.GetRadiusf();
		Phi = Capsule.SignedDistance(Point);
		EXPECT_NEAR(FMath::Abs(Phi), 0, KINDA_SMALL_NUMBER) << *FString("Capsule failed phi surface (origin+orthogonalAxis*radius) sanity test.");

		Point = Capsule.GetCenterf() + Capsule.GetAxisf().GetOrthogonalVector().GetSafeNormal() * Capsule.GetRadiusf();
		Phi = Capsule.SignedDistance(Point);
		EXPECT_NEAR(FMath::Abs(Phi), 0, KINDA_SMALL_NUMBER) << *FString("Capsule failed phi surface (center+orthogonalAxis*radius) sanity test.");

		TArray<FVec3> Points = Capsule.ComputeSamplePoints(100);
		check(Points.Num() == 100);
		Point[0] = TNumericLimits<FReal>::Max();
		FReal MinPhi = TNumericLimits<FReal>::Max();
		FReal MaxPhi = -TNumericLimits<FReal>::Max();
		for (int32 i=0; i < Points.Num(); i++)
		{
			const FVec3 &Pt = Points[i];

			Phi = Capsule.SignedDistance(Pt);
			MinPhi = FMath::Min(Phi, MinPhi);
			MaxPhi = FMath::Max(Phi, MaxPhi);
			
			const bool Differs = Pt != Point;
			check(Differs);
			EXPECT_TRUE(Differs) << *FString("Produced a redundant value.");
			Point = Pt;
		}

		const bool OnSurface = FMath::Abs(MinPhi) <= KINDA_SMALL_NUMBER && FMath::Abs(MaxPhi) <= KINDA_SMALL_NUMBER;
		check(OnSurface);
		EXPECT_TRUE(OnSurface) << *FString("Produced a point not on the surface of the capsule.");
	}

	void TestComputeSamplePoints_Capsule()
	{
		//
		// Height == 1
		//

		// At the origin with radius 1
		{
			FCapsule Capsule(FVec3(0,0,0), FVec3(0,0,1), (FReal)1.0);
			RunTestComputeSamplePoints(Capsule);
		}
		// At the origin with radius > 1
		{
			FCapsule Capsule(FVec3(0,0,0), FVec3(0,0,1), (FReal)10.0);
			RunTestComputeSamplePoints(Capsule);
		}
		// At the origin with radius < 1
		{
			FCapsule Capsule(FVec3(0,0,0), FVec3(0,0,1), (FReal)0.1);
			RunTestComputeSamplePoints(Capsule);
		}
		// Off the origin with radius 1
		{
			FCapsule Capsule(FVec3(10,10,10), FVec3(10,10,11), (FReal)1.0);
			RunTestComputeSamplePoints(Capsule);
		}
		// Off the origin with radius > 1
		{
			FCapsule Capsule(FVec3(10,10,10), FVec3(10,10,11), (FReal)10.0);
			RunTestComputeSamplePoints(Capsule);
		}
		// Off the origin with radius < 1
		{
			FCapsule Capsule(FVec3(10,10,10), FVec3(10,10,11), (FReal)0.1);
			RunTestComputeSamplePoints(Capsule);
		}

		//
		// Height > 1
		//

		// At the origin with radius 1
		{
			FCapsule Capsule(FVec3(0,0,0), FVec3(0,0,10), (FReal)1.0);
			RunTestComputeSamplePoints(Capsule);
		}
		// At the origin with radius > 1
		{
			FCapsule Capsule(FVec3(0,0,0), FVec3(0,0,10), (FReal)10.0);
			RunTestComputeSamplePoints(Capsule);
		}
		// At the origin with radius < 1
		{
			FCapsule Capsule(FVec3(0,0,0), FVec3(0,0,10), (FReal)0.1);
			RunTestComputeSamplePoints(Capsule);
		}
		// Off the origin with radius 1
		{
			FCapsule Capsule(FVec3(10,10,10), FVec3(10,10,21), (FReal)1.0);
			RunTestComputeSamplePoints(Capsule);
		}
		// Off the origin with radius > 1
		{
			FCapsule Capsule(FVec3(10,10,10), FVec3(10,10,21), (FReal)10.0);
			RunTestComputeSamplePoints(Capsule);
		}
		// Off the origin with radius < 1
		{
			FCapsule Capsule(FVec3(10,10,10), FVec3(10,10,21), (FReal)0.1);
			RunTestComputeSamplePoints(Capsule);
		}

		// 
		// Off axis
		//

		// At the origin with radius 1
		{
			FCapsule Capsule(FVec3(0,0,0), FVec3(1,1,1), (FReal)1.0);
			RunTestComputeSamplePoints(Capsule);
		}
		// At the origin with radius > 1
		{
			FCapsule Capsule(FVec3(0,0,0), FVec3(1,1,1), (FReal)10.0);
			RunTestComputeSamplePoints(Capsule);
		}
		// At the origin with radius < 1
		{
			FCapsule Capsule(FVec3(0,0,0), FVec3(1,1,1), (FReal)0.1);
			RunTestComputeSamplePoints(Capsule);
		}
		// Off the origin with radius 1
		{
			FCapsule Capsule(FVec3(10,10,10), FVec3(11,11,11), (FReal)1.0);
			RunTestComputeSamplePoints(Capsule);
		}
		// Off the origin with radius > 1
		{
			FCapsule Capsule(FVec3(10,10,10), FVec3(11,11,11), (FReal)10.0);
			RunTestComputeSamplePoints(Capsule);
		}
		// Off the origin with radius < 1
		{
			FCapsule Capsule(FVec3(10,10,10), FVec3(11,11,11), (FReal)0.1);
			RunTestComputeSamplePoints(Capsule);
		}
	}

	void TestImplicitCapsule()
	{
		TestComputeSamplePoints_Capsule();
	}
} // namespace GeometryCollectionTest
