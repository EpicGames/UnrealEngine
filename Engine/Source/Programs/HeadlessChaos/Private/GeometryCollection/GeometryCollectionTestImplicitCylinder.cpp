// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestImplicitCylinder.h"

#include "Chaos/Cylinder.h"

namespace GeometryCollectionTest
{
	using namespace Chaos;

	void RunTestComputeSamplePoints(const FCylinder &Cylinder)
	{
		FVec3 Point;
		FReal Phi;
		EXPECT_EQ(Cylinder.GetType(), Chaos::ImplicitObjectType::Cylinder) << *FString("Implicit object type is not 'cylinder'.");

		Point = Cylinder.GetAxis();
		EXPECT_LT(FMath::Abs(Point.Size() - 1.0), KINDA_SMALL_NUMBER) << *FString("Cylinder axis is not unit length.");
		
		Point = Cylinder.GetOrigin() + Cylinder.GetAxis() * Cylinder.GetHeight();
		EXPECT_LT((Point - Cylinder.GetInsertion()).Size(), KINDA_SMALL_NUMBER) << *FString("Cylinder is broken.");

		Point = Cylinder.GetInsertion();// Cylinder.GetOrigin() + Cylinder.GetAxis() * Cylinder.GetHeight();
		Phi = Cylinder.SignedDistance(Point);
		EXPECT_LE(Phi, KINDA_SMALL_NUMBER) << *FString("Cylinder failed phi surface (insertion) sanity test.");

		Point = Cylinder.GetOrigin() + Cylinder.GetAxis() * Cylinder.GetHeight() * 0.25;
		Phi = Cylinder.SignedDistance(Point);
		EXPECT_LE(Phi, (FReal)0.0) << *FString("Cylinder failed phi depth (1/4 origin) sanity test.");

		Point = Cylinder.GetOrigin() + Cylinder.GetAxis() * Cylinder.GetHeight() * 0.75;
		Phi = Cylinder.SignedDistance(Point);
		EXPECT_LE(Phi, (FReal)0.0) << *FString("Cylinder failed phi depth (3/4 origin) sanity test.");

		EXPECT_LE((Cylinder.GetCenter() - FVec3(Cylinder.GetOrigin() + Cylinder.GetAxis() * Cylinder.GetHeight() * 0.5)).Size(), KINDA_SMALL_NUMBER) << *FString("Cylinder center is off mid axis.");

		Point = Cylinder.GetCenter();
		Phi = Cylinder.SignedDistance(Point);
		EXPECT_LT(Phi, (FReal)0.0) << *FString("Cylinder failed phi depth sanity test.");
		
		Point = Cylinder.GetOrigin();
		Phi = Cylinder.SignedDistance(Point);
		EXPECT_LE(FMath::Abs(Phi), KINDA_SMALL_NUMBER) << *FString("Cylinder failed phi surface (origin) sanity test.");

		Point = Cylinder.GetOrigin() + Cylinder.GetAxis() * Cylinder.GetHeight();
		Phi = Cylinder.SignedDistance(Point);
		EXPECT_LE(FMath::Abs(Phi), KINDA_SMALL_NUMBER) << *FString("Cylinder failed phi surface (origin+axis*height) sanity test.");

		Point = Cylinder.GetOrigin() + Cylinder.GetAxis().GetOrthogonalVector().GetSafeNormal() * Cylinder.GetRadius();
		Phi = Cylinder.SignedDistance(Point);
		EXPECT_LE(FMath::Abs(Phi), KINDA_SMALL_NUMBER) << *FString("Cylinder failed phi surface (origin+orthogonalAxis*radius) sanity test.");

		Point = Cylinder.GetCenter() + Cylinder.GetAxis().GetOrthogonalVector().GetSafeNormal() * Cylinder.GetRadius();
		Phi = Cylinder.SignedDistance(Point);
		EXPECT_LE(FMath::Abs(Phi), KINDA_SMALL_NUMBER) << *FString("Cylinder failed phi surface (center+orthogonalAxis*radius) sanity test.");

		TArray<FVec3> Points = Cylinder.ComputeSamplePoints(100);
		check(Points.Num() == 100);
		Point[0] = TNumericLimits<FReal>::Max();
		FReal MinPhi = TNumericLimits<FReal>::Max();
		FReal MaxPhi = -TNumericLimits<FReal>::Max();
		for (int32 i=0; i < Points.Num(); i++)
		{
			const FVec3 &Pt = Points[i];

			Phi = Cylinder.SignedDistance(Pt);
			MinPhi = FMath::Min(Phi, MinPhi);
			MaxPhi = FMath::Max(Phi, MaxPhi);
			
			const bool Differs = Pt != Point;
			check(Differs);
			EXPECT_TRUE(Differs) << *FString("Produced a redundant value.");
			Point = Pt;
		}

		const bool OnSurface = FMath::Abs(MinPhi) <= KINDA_SMALL_NUMBER && FMath::Abs(MaxPhi) <= KINDA_SMALL_NUMBER;
		check(OnSurface);
		EXPECT_TRUE(OnSurface) << *FString("Produced a point not on the surface of the cylinder.");
	}

	void TestComputeSamplePoints_Cylinder()
	{
		//
		// Height == 1
		//

		// At the origin with radius 1
		{
			FCylinder Cylinder(FVec3(0,0,0), FVec3(0,0,1), (FReal)1.0);
			RunTestComputeSamplePoints(Cylinder);
		}
		// At the origin with radius > 1
		{
			FCylinder Cylinder(FVec3(0,0,0), FVec3(0,0,1), (FReal)10.0);
			RunTestComputeSamplePoints(Cylinder);
		}
		// At the origin with radius < 1
		{
			FCylinder Cylinder(FVec3(0,0,0), FVec3(0,0,1), (FReal)0.1);
			RunTestComputeSamplePoints(Cylinder);
		}
		// Off the origin with radius 1
		{
			FCylinder Cylinder(FVec3(10,10,10), FVec3(10,10,11), (FReal)1.0);
			RunTestComputeSamplePoints(Cylinder);
		}
		// Off the origin with radius > 1
		{
			FCylinder Cylinder(FVec3(10,10,10), FVec3(10,10,11), (FReal)10.0);
			RunTestComputeSamplePoints(Cylinder);
		}
		// Off the origin with radius < 1
		{
			FCylinder Cylinder(FVec3(10,10,10), FVec3(10,10,11), (FReal)0.1);
			RunTestComputeSamplePoints(Cylinder);
		}

		//
		// Height > 1
		//

		// At the origin with radius 1
		{
			FCylinder Cylinder(FVec3(0,0,0), FVec3(0,0,10), (FReal)1.0);
			RunTestComputeSamplePoints(Cylinder);
		}
		// At the origin with radius > 1
		{
			FCylinder Cylinder(FVec3(0,0,0), FVec3(0,0,10), (FReal)10.0);
			RunTestComputeSamplePoints(Cylinder);
		}
		// At the origin with radius < 1
		{
			FCylinder Cylinder(FVec3(0,0,0), FVec3(0,0,10), (FReal)0.1);
			RunTestComputeSamplePoints(Cylinder);
		}
		// Off the origin with radius 1
		{
			FCylinder Cylinder(FVec3(10,10,10), FVec3(10,10,21), (FReal)1.0);
			RunTestComputeSamplePoints(Cylinder);
		}
		// Off the origin with radius > 1
		{
			FCylinder Cylinder(FVec3(10,10,10), FVec3(10,10,21), (FReal)10.0);
			RunTestComputeSamplePoints(Cylinder);
		}
		// Off the origin with radius < 1
		{
			FCylinder Cylinder(FVec3(10,10,10), FVec3(10,10,21), (FReal)0.1);
			RunTestComputeSamplePoints(Cylinder);
		}

		// 
		// Off axis
		//

		// At the origin with radius 1
		{
			FCylinder Cylinder(FVec3(0,0,0), FVec3(1,1,1), (FReal)1.0);
			RunTestComputeSamplePoints(Cylinder);
		}
		// At the origin with radius > 1
		{
			FCylinder Cylinder(FVec3(0,0,0), FVec3(1,1,1), (FReal)10.0);
			RunTestComputeSamplePoints(Cylinder);
		}
		// At the origin with radius < 1
		{
			FCylinder Cylinder(FVec3(0,0,0), FVec3(1,1,1), (FReal)0.1);
			RunTestComputeSamplePoints(Cylinder);
		}
		// Off the origin with radius 1
		{
			FCylinder Cylinder(FVec3(10,10,10), FVec3(11,11,11), (FReal)1.0);
			RunTestComputeSamplePoints(Cylinder);
		}
		// Off the origin with radius > 1
		{
			FCylinder Cylinder(FVec3(10,10,10), FVec3(11,11,11), (FReal)10.0);
			RunTestComputeSamplePoints(Cylinder);
		}
		// Off the origin with radius < 1
		{
			FCylinder Cylinder(FVec3(10,10,10), FVec3(11,11,11), (FReal)0.1);
			RunTestComputeSamplePoints(Cylinder);
		}
	}

	void TestImplicitCylinder()
	{
		TestComputeSamplePoints_Cylinder();
	}
} // namespace GeometryCollectionTest
