// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestImplicitSphere.h"

#include "Chaos/Pair.h"
#include "Chaos/Sphere.h"

#include <string>

namespace GeometryCollectionTest
{
	//==========================================================================
	// FindClosestIntersection() tests
	//==========================================================================

	using namespace Chaos;

	typedef Chaos::Pair<FVec3, bool> IntersectionResult;

	void RunTestFindClosestIntersectionHelper(
		const FString TestName,
		const FVec3& Center, 
		const FReal Radius,
		const FReal Thickness,
		const FVec3& StartPt,
		const FVec3& EndPt,
		const IntersectionResult ExpectedRes)
	{
		Chaos::FSphere Sphere(Center, Radius);
		IntersectionResult Res = Sphere.FindClosestIntersection(StartPt, EndPt, Thickness);
		EXPECT_TRUE(
			(!Res.Second && !ExpectedRes.Second) ||
			(Res.Second && ExpectedRes.Second && (Res.First - ExpectedRes.First).Size() < KINDA_SMALL_NUMBER)) <<
			*FString::Printf(
				TEXT(
					"%s - Sphere((%g, %g, %g), %g).FindClosestIntersection("
						"StartPt(%g, %g, %g), "
						"EndPt(%g, %g, %g), "
						"Thickness:%g) = "
					"RESULT: %d, Point: (%g, %g, %g), "
					"EXPECTED: %d (%g, %g, %g)."),
				*TestName,
				Center[0], Center[1], Center[2],
				Radius,

				StartPt[0], StartPt[1], StartPt[2],
				EndPt[0], EndPt[1], EndPt[2],
				Thickness,
				
				Res.Second ? 1 : 0,
				Res.First[0], Res.First[1], Res.First[2],
				
				ExpectedRes.Second ? 1 : 0,
				ExpectedRes.First[0], ExpectedRes.First[1], ExpectedRes.First[2]
			);
	}

	void RunTestFindClosestIntersection_Tangent(
		const FVec3& Center,
		const FReal Radius,
		const FReal Thickness)
	{
		FVec3 StartPt;
		FVec3 EndPt;

		// Start & End points coincident, lying on the surface of the sphere w/zero thickness
		// Fails with no collision:
		// "RunTestFindClosestIntersection_Tangent1 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 1), EndPt(0, 0, 1), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (0, 0, 1.1)."	FString
		StartPt = EndPt = Center + FVec3(0, 0, Radius);
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Tangent1", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));

		StartPt = Center + FVec3(Radius, 0, Radius+Thickness);
		EndPt = Center + FVec3(-Radius, 0, Radius+Thickness);
		RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Tangent2", Center, Radius, Thickness,
			StartPt, EndPt, 
			IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));
		
		// Start & End points coincident, lying on the surface of the sphere w/thickness
		// Fails with incorrect collision point:
		// "RunTestFindClosestIntersection_Tangent3 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 1.1), EndPt(0, 0, 1.1), Thickness:0.1) = RESULT: 1, Point: (0, 0, 1), EXPECTED: 1 (0, 0, 1.1)."	FString
		// "RunTestFindClosestIntersection_Tangent3 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 1.5), EndPt(0, 0, 1.5), Thickness:0.5) = RESULT: 1, Point: (0, 0, 1), EXPECTED: 1 (0, 0, 1.5)."	FString
		// "RunTestFindClosestIntersection_Tangent3 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 2), EndPt(0, 0, 2), Thickness:1) = RESULT: 1, Point: (0, 0, 1), EXPECTED: 1 (0, 0, 2)."	FString
		// "RunTestFindClosestIntersection_Tangent3 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 3), EndPt(0, 0, 3), Thickness:2) = RESULT: 1, Point: (0, 0, 1), EXPECTED: 1 (0, 0, 3)."	FString
		StartPt = EndPt = Center + FVec3(0, 0, Radius+Thickness);
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Tangent3", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));

		StartPt = Center + FVec3(Radius, 0, Radius+Thickness);
		EndPt = Center + FVec3(-Radius, 0, Radius+Thickness);
		RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Tangent4", Center, Radius, Thickness,
			StartPt, EndPt, 
			IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));

		// Radius + Thickness + 1 (miss)
		StartPt = EndPt = Center + FVec3(0, 0, Radius+Thickness+1.0);
		RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Tangent5", Center, Radius, Thickness,
			StartPt, EndPt, 
			IntersectionResult(Center+FVec3(Radius+Thickness,0,0), false));

		StartPt = Center + FVec3(Radius, 0, Radius+Thickness+1.0);
		EndPt = Center + FVec3(-Radius, 0, Radius+Thickness+1.0);
		RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Tangent6", Center, Radius, Thickness,
			StartPt, EndPt, 
			IntersectionResult(Center+FVec3(0,0,Radius+Thickness), false));
	}

	void RunTestFindClosestIntersection_Shallow(
		const FVec3& Center,
		const FReal Radius,
		const FReal Thickness)
	{
		FVec3 StartPt(0, 0, Radius * .999999);
		FVec3 EndPt = StartPt;

		// Fails w/no collision:
		// "RunTestFindClosestIntersection_Shallow1 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 0.999999), EndPt(0, 0, 0.999999), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (1.1, 0...	FString
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Shallow1", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(Radius+Thickness,0,0), true));

		// Fails w/no collision:
		// "RunTestFindClosestIntersection_Shallow2 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 0.999999), EndPt(0, 0, 1), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (0, 0, 1.1)."	FString
		EndPt += FVec3(0, 0, Radius * .000001);
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Shallow2", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));

		// Fails w/no collision:
		// "RunTestFindClosestIntersection_Shallow3 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 0.999999), EndPt(0, 0, 1), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (0, 0, 1.1)."	FString
		EndPt += FVec3(0, 0, Radius * .000001);
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Shallow3", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));

		EndPt += FVec3(0, 0, Thickness);
		RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Shallow3.1", Center, Radius, Thickness,
			StartPt, EndPt, 
			IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));

		// Fails w/no collision:
		// "RunTestFindClosestIntersection_Shallow4 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 1), EndPt(0, 0, 0.999999), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (0, 0, 1.1)."	FString
		EndPt = StartPt;
		StartPt = EndPt + FVec3(0, 0, Radius * .000001);
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Shallow4", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));

		// Fails w/no collision:
		// "RunTestFindClosestIntersection_Shallow5 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 1), EndPt(0, 0, 0.999999), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (0, 0, 1.1)."	FString
		StartPt += FVec3(0, 0, Radius * .000001);
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Shallow5", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));

		StartPt += FVec3(0, 0, Thickness);
		RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Shallow5.1", Center, Radius, Thickness,
			StartPt, EndPt, 
			IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));
	}

	void RunTestFindClosestIntersection_Mid(
		const FVec3& Center, 
		const FReal Radius,
		const FReal Thickness)
	{
		FVec3 StartPt(0,0,Radius * .5);
		FVec3 EndPt = StartPt;

		// Fails w/no collision:
		// "RunTestFindClosestIntersection_Mid1 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 0.5), EndPt(0, 0, 0.5), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (1.1, 0, 0)."	FString
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Mid1", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(Radius+Thickness,0,0), true));

		// Fails w/no collision:
		// "RunTestFindClosestIntersection_Mid2 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 0.5), EndPt(0, 0, 1), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (0, 0, 1.1)."	FString
		//EndPt += FVec3(0, 0, Radius * .5);
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Mid2", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));
		
		// Fails w/no collision:
		// "RunTestFindClosestIntersection_Mid3 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 0.5), EndPt(0, 0, 1), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (0, 0, 1.1)."	FString
		EndPt += FVec3(0, 0, Radius * .5);
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Mid3", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));

		// Fails w/no collision:
		// "RunTestFindClosestIntersection_Mid4 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 1), EndPt(0, 0, 0.5), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (0, 0, 1.1)."	FString
		EndPt = StartPt;
		StartPt = EndPt + FVec3(0, 0, Radius * .5);
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Mid4", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));

		StartPt += FVec3(0, 0, Radius * .5);
		RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Mid5", Center, Radius, Thickness,
			StartPt, EndPt, 
			IntersectionResult(Center+FVec3(0,0,Radius+Thickness), true));
	}

	void RunTestFindClosestIntersection_Deep(
		const FVec3& Center, 
		const FReal Radius,
		const FReal Thickness)
	{
		FVec3 StartPt = Center;
		FVec3 EndPt = Center;

		// Fails w/no collision
		// "RunTestFindClosestIntersection_Deep1 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 0), EndPt(0, 0, 0), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (1.1, 0, 0)."	FString
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Deep1", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(Radius+Thickness,0,0), true));

		// Fails w/no collision
		// "RunTestFindClosestIntersection_Deep2 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 0), EndPt(0, 0, 0.5), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (0, 0, 1.1)."	FString
		EndPt += FVec3(0, 0, Radius*.5);
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Deep2", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(0, 0, Radius+Thickness), true));

		EndPt += FVec3(0, 0, Radius);
		RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Deep3", Center, Radius, Thickness,
			StartPt, EndPt, 
			IntersectionResult(Center+FVec3(0, 0, Radius+Thickness), true));

		// Fails w/no collision:
		// "RunTestFindClosestIntersection_Deep4 - Sphere((0, 0, 0), 1).FindClosestIntersection(StartPt(0, 0, 0.5), EndPt(0, 0, 0), Thickness:0.1) = RESULT: 0, Point: (0, 0, 0), EXPECTED: 1 (0, 0, 1.1)."	FString
		EndPt = Center;
		StartPt += FVec3(0, 0, Radius*.5);
		//RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Deep4", Center, Radius, Thickness,
		//	StartPt, EndPt, 
		//	IntersectionResult(Center+FVec3(0, 0, Radius+Thickness), true));

		StartPt += FVec3(0, 0, Radius);
		RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Deep5", Center, Radius, Thickness,
			StartPt, EndPt, 
			IntersectionResult(Center+FVec3(0, 0, Radius+Thickness), true));

		// Miss
		StartPt = Center + FVec3(Radius + Thickness + 1);
		EndPt = StartPt + FVec3(Radius + Thickness + 1);
		RunTestFindClosestIntersectionHelper("RunTestFindClosestIntersection_Deep6", Center, Radius, Thickness,
			StartPt, EndPt, 
			IntersectionResult(StartPt, false));
	}

	void RunTestFindClosestIntersection(
		const FVec3& Center, const FReal Radius)
	{
		//
		// Run different test configurations with varying thicknesses.
		//

		// Deep intersection
		RunTestFindClosestIntersection_Deep(Center, Radius, 0.0);
		RunTestFindClosestIntersection_Deep(Center, Radius, Radius*.1);
//		RunTestFindClosestIntersection_Deep(Center, Radius, Radius*.5); // fails with incorrect collision points
//		RunTestFindClosestIntersection_Deep(Center, Radius, Radius);
//		RunTestFindClosestIntersection_Deep(Center, Radius, Radius*2);

		// Mid intersection
		RunTestFindClosestIntersection_Mid(Center, Radius, 0.0);
		RunTestFindClosestIntersection_Mid(Center, Radius, Radius*.1);
//		RunTestFindClosestIntersection_Mid(Center, Radius, Radius*.5); // fails with incorrect collision points
//		RunTestFindClosestIntersection_Mid(Center, Radius, Radius);
//		RunTestFindClosestIntersection_Mid(Center, Radius, Radius*2);

		// Grazing intersection - all fail
		RunTestFindClosestIntersection_Shallow(Center, Radius, 0.0);
//		RunTestFindClosestIntersection_Shallow(Center, Radius, Radius*.1); // fails with incorrect collision points
//		RunTestFindClosestIntersection_Shallow(Center, Radius, Radius*.5);
//		RunTestFindClosestIntersection_Shallow(Center, Radius, Radius);
//		RunTestFindClosestIntersection_Shallow(Center, Radius, Radius*2);

		// Tangent intersection
		RunTestFindClosestIntersection_Tangent(Center, Radius, 0.0);
		RunTestFindClosestIntersection_Tangent(Center, Radius, Radius*.1);
		RunTestFindClosestIntersection_Tangent(Center, Radius, Radius*.5);
		RunTestFindClosestIntersection_Tangent(Center, Radius, Radius);
		RunTestFindClosestIntersection_Tangent(Center, Radius, Radius*2);
	}
	
	void TestIntersections()
	{
		// At the origin
		RunTestFindClosestIntersection(FVec3((FReal)0.0), (FReal)1.0);
//		RunTestFindClosestIntersection(FVec3((FReal)0.0), (FReal)10.0);
//		RunTestFindClosestIntersection(FVec3((FReal)0.0), (FReal)100.0);

		// Off origin
//		RunTestFindClosestIntersection(FVec3((FReal)1.0), (FReal)1.0);
//		RunTestFindClosestIntersection(FVec3((FReal)10.0), (FReal)1.0);
//		RunTestFindClosestIntersection(FVec3((FReal)100.0), (FReal)1.0);

//		RunTestFindClosestIntersection(FVec3((FReal)1.0), (FReal)10.0);
//		RunTestFindClosestIntersection(FVec3((FReal)10.0), (FReal)10.0);
//		RunTestFindClosestIntersection(FVec3((FReal)100.0), (FReal)10.0);

//		RunTestFindClosestIntersection(FVec3((FReal)1.0), (FReal)100.0);
//		RunTestFindClosestIntersection(FVec3((FReal)10.0), (FReal)100.0);
//		RunTestFindClosestIntersection(FVec3((FReal)100.0), (FReal)100.0);
	}

	//==========================================================================
	// Sample points tests
	//==========================================================================

	void RunTestComputeSamplePoints(const Chaos::FSphere &Sphere)
	{
		EXPECT_EQ(Sphere.GetType(), Chaos::ImplicitObjectType::Sphere) << *FString("Implicit object type is not 'sphere'.");

		FVec3 Point = Sphere.GetCenterf();
		FReal Phi = Sphere.SignedDistance(Point);
		EXPECT_LE(FMath::Abs(Phi + Sphere.GetRadiusf()), SMALL_NUMBER) << *FString("Sphere failed phi depth sanity test.");
		
		Point[0] += Sphere.GetRadiusf();
		Phi = Sphere.SignedDistance(Point);
		EXPECT_LE(FMath::Abs(Phi), KINDA_SMALL_NUMBER) << *FString("Sphere failed phi surface sanity test.");

		TArray<FVec3> Points = Sphere.ComputeSamplePoints(100);
		check(Points.Num() == 100);
		Point[0] = TNumericLimits<FReal>::Max();
		for (const FVec3 &Pt : Points)
		{
			Phi = Sphere.SignedDistance(Pt);
			const bool OnSurface = FMath::Abs(Phi) <= KINDA_SMALL_NUMBER;
			check(OnSurface);
			EXPECT_TRUE(OnSurface) << *FString("Produced a point not on the surface of the sphere.");
			const bool Differs = Pt != Point;
			check(Differs);
			EXPECT_TRUE(Differs) << *FString("Produced a redundant value.");
			Point = Pt;
		}
	}

	void RunTestComputeSemispherePoints(const Chaos::FSphere &Sphere)
	{
		TArray<FVec3> Points;
		Chaos::TSphereSpecializeSamplingHelper<FReal, 3>::ComputeBottomHalfSemiSphere(Points, Sphere, 100);
		for (FVec3& Pt : Points)
		{
			FReal Phi = Sphere.SignedDistance(Pt);
			const bool OnSurface = FMath::Abs(Phi) <= KINDA_SMALL_NUMBER;
			check(OnSurface);
			EXPECT_TRUE(OnSurface) << *FString("Produced a point not on the surface of the sphere.");

			const bool BelowCenter = Pt[2] < Sphere.GetCenterf()[2] + KINDA_SMALL_NUMBER;
			check(BelowCenter);
			EXPECT_TRUE(BelowCenter) << *FString("Bottom semisphere produced a point above midline.");
		}

		Points.Reset();
		Chaos::TSphereSpecializeSamplingHelper<FReal, 3>::ComputeTopHalfSemiSphere(Points, Sphere, 100);
		for (FVec3& Pt : Points)
		{
			FReal Phi = Sphere.SignedDistance(Pt);
			const bool OnSurface = FMath::Abs(Phi) <= KINDA_SMALL_NUMBER;
			check(OnSurface);
			EXPECT_TRUE(OnSurface) << *FString("Produced a point not on the surface of the sphere.");

			const bool BelowCenter = Pt[2] > Sphere.GetCenterf()[2] - KINDA_SMALL_NUMBER;
			check(BelowCenter);
			EXPECT_TRUE(BelowCenter) << *FString("Top semisphere produced a point above midline.");
		}
	}

	void TestComputeSamplePoints_SemiSphere()
	{
		// At the origin with radius 1
		{
			Chaos::FSphere Sphere(FVec3((FReal)0.0), (FReal)1.0);
			RunTestComputeSemispherePoints(Sphere);
		}	
	}

		void TestComputeSamplePoints_Sphere()
	{
		// At the origin with radius 1
		{
			Chaos::FSphere Sphere(FVec3(0.0), (FReal)1.0);
			RunTestComputeSamplePoints(Sphere);
		}
		// At the origin with radius > 1
		{
			Chaos::FSphere Sphere(FVec3(0.0), (FReal)10.0);
			RunTestComputeSamplePoints(Sphere);
		}
		// At the origin with radius < 1
		{
			Chaos::FSphere Sphere(FVec3(0.0), (FReal).1);
			RunTestComputeSamplePoints(Sphere);
		}
		// Off the origin with radius 1
		{
			Chaos::FSphere Sphere(FVec3(10.0), (FReal)1.0);
			RunTestComputeSamplePoints(Sphere);
		}
		// Off the origin with radius > 1
		{
			Chaos::FSphere Sphere(FVec3(10.0), (FReal)10.0);
			RunTestComputeSamplePoints(Sphere);
		}
		// Off the origin with radius < 1
		{
			Chaos::FSphere Sphere(FVec3(10.0), (FReal).1);
			RunTestComputeSamplePoints(Sphere);
		}
	}

	void TestImplicitSphere()
	{
		TestComputeSamplePoints_Sphere();
		TestComputeSamplePoints_SemiSphere();
		TestIntersections();
	}
	
} // namespace GeometryCollectionTest
