// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestInitilization.h"
#include "GeometryCollection/GeometryCollectionTestFramework.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "HeadlessChaosTestUtility.h"

namespace GeometryCollectionTest
{
using namespace ChaosTest;


GTEST_TEST(AllTraits,GeometryCollection_Initilization_TransformedGeometryCollectionRoot)
{
	FVector GlobalTranslation(10); FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(0));
	CreationParameters Params; Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic; Params.EnableClustering = false;
	Params.RootTransform = FTransform(GlobalRotation,GlobalTranslation); Params.NestedTransforms ={FTransform(FVector(10)),FTransform::Identity,FTransform::Identity};
	FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
	EXPECT_EQ(Collection->DynamicCollection->GetParent(0),1); // is a child of index one

	FFramework UnitTest;
	UnitTest.AddSimulationObject(Collection);
	UnitTest.Initialize();
	UnitTest.Advance();

	{ // test results
		EXPECT_EQ(UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().Size(),1);
		FVector X = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().GetX(0);
		FQuat R = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().GetR(0);
		EXPECT_TRUE((R * GlobalRotation.Inverse()).IsIdentity(KINDA_SMALL_NUMBER));
		EXPECT_NEAR(X.X - GlobalTranslation[0],0.0f,KINDA_SMALL_NUMBER);
		EXPECT_NEAR(X.Y - GlobalTranslation[1],0.0f,KINDA_SMALL_NUMBER);
		EXPECT_LT(X.Z,GlobalTranslation[2]);

		TArray<FTransform> Transform;
		GeometryCollectionAlgo::Private::GlobalMatrices(*Collection->DynamicCollection,Transform);
		EXPECT_EQ(Collection->DynamicCollection->GetParent(0),FGeometryCollection::Invalid); // is not a child
		EXPECT_NEAR(Transform[0].GetTranslation().X - GlobalTranslation[0],0.0f,KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Transform[0].GetTranslation().Y - GlobalTranslation[1],0.0f,KINDA_SMALL_NUMBER);
		EXPECT_LT(Transform[0].GetTranslation().Z,GlobalTranslation[2]);
	}
}


GTEST_TEST(AllTraits,GeometryCollection_Initilization_TransformedGeometryCollectionParentNode)
{
	FVector GlobalTranslation(10); FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(0));
	CreationParameters Params; Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic; Params.EnableClustering = false;
	Params.RootTransform = FTransform(GlobalRotation,GlobalTranslation); Params.NestedTransforms ={FTransform::Identity,FTransform(FVector(10)),FTransform::Identity};
	FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

	FFramework UnitTest;
	UnitTest.AddSimulationObject(Collection);
	UnitTest.Initialize();
	UnitTest.Advance();

	{ // test results
		EXPECT_EQ(UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().Size(),1);
		FVector X = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().GetX(0);
		FQuat R = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().GetR(0);
		EXPECT_TRUE((R * GlobalRotation.Inverse()).IsIdentity(KINDA_SMALL_NUMBER));
		EXPECT_NEAR(X.X - GlobalTranslation[0],0.0f,KINDA_SMALL_NUMBER);
		EXPECT_NEAR(X.Y - GlobalTranslation[1],0.0f,KINDA_SMALL_NUMBER);
		EXPECT_LT(X.Z,GlobalTranslation[2]);

		TArray<FTransform> Transform;
		GeometryCollectionAlgo::Private::GlobalMatrices(*Collection->DynamicCollection,Transform);
		EXPECT_EQ(Collection->DynamicCollection->GetParent(0),FGeometryCollection::Invalid); // is not a child
		EXPECT_NEAR(Transform[0].GetTranslation().X - GlobalTranslation[0],0.0f,KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Transform[0].GetTranslation().Y - GlobalTranslation[1],0.0f,KINDA_SMALL_NUMBER);
		EXPECT_LT(Transform[0].GetTranslation().Z,GlobalTranslation[2]);
	}
}


GTEST_TEST(AllTraits,GeometryCollection_Initilization_TransformedGeometryCollectionGeometryNode)
{
	FVector GlobalTranslation(10); FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(0));
	CreationParameters Params; Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic; Params.EnableClustering = false;
	Params.RootTransform = FTransform(GlobalRotation,GlobalTranslation); Params.NestedTransforms ={FTransform::Identity,FTransform::Identity,FTransform(FVector(10))};
	FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

	FFramework UnitTest;
	UnitTest.AddSimulationObject(Collection);
	UnitTest.Initialize();
	UnitTest.Advance();

	{ // test results
		EXPECT_EQ(UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().Size(),1);
		FVector X = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().GetX(0);
		FQuat R = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().GetR(0);
		EXPECT_TRUE((R * GlobalRotation.Inverse()).IsIdentity(KINDA_SMALL_NUMBER));
		EXPECT_NEAR(X.X - GlobalTranslation[0],0.0f,KINDA_SMALL_NUMBER);
		EXPECT_NEAR(X.Y - GlobalTranslation[1],0.0f,KINDA_SMALL_NUMBER);
		EXPECT_LT(X.Z,GlobalTranslation[2]);

		TArray<FTransform> Transform;
		GeometryCollectionAlgo::Private::GlobalMatrices(*Collection->DynamicCollection,Transform);
		EXPECT_EQ(Collection->DynamicCollection->GetParent(0),FGeometryCollection::Invalid); // is not a child
		EXPECT_NEAR(Transform[0].GetTranslation().X - GlobalTranslation[0],0.0f,KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Transform[0].GetTranslation().Y - GlobalTranslation[1],0.0f,KINDA_SMALL_NUMBER);
		EXPECT_LT(Transform[0].GetTranslation().Z,GlobalTranslation[2]);
	}
}

	
	// We expect that if we actually apply the global matrices that it does not care about MassToLocal. To make this accurate, 
	// X and Y should still be at origin, and check that Z is < 0. IT shouldnt make a difference if we move the geometry...
GTEST_TEST(AllTraits,GeometryCollection_Initilization_TransformedGeometryCollectionGeometryVertices)
{
	FVector GlobalTranslation(10); FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(0));
	CreationParameters Params; Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic; Params.EnableClustering = false;
	Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere; Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
	Params.GeomTransform = FTransform(GlobalRotation,GlobalTranslation); Params.NestedTransforms ={FTransform::Identity,FTransform::Identity,FTransform::Identity};
	FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

	//
	// validate the vertices have been moved.  
	//
	TArray<FTransform> RestTransforms;
	int32 NumVertices = Collection->DynamicCollection->NumElements(FGeometryCollection::VerticesGroup);
	const TManagedArray<FVector3f>& Vertices = Collection->RestCollection->template GetAttribute<FVector3f>("Vertex",FGeometryCollection::VerticesGroup);
	const TManagedArray<int32>& BoneMap = Collection->RestCollection->template GetAttribute<int32>("BoneMap",FGeometryCollection::VerticesGroup);
	GeometryCollectionAlgo::GlobalMatrices(Collection->RestCollection->Transform,Collection->RestCollection->Parent,RestTransforms);
	FVector CenterOfMass(0);  
	for (int VertexIndex = 0; VertexIndex < NumVertices; VertexIndex++)
	{
		CenterOfMass += RestTransforms[BoneMap[VertexIndex]].TransformPosition(FVector(Vertices[VertexIndex]));
	}
	CenterOfMass /= NumVertices;
	EXPECT_NEAR(CenterOfMass.X - GlobalTranslation[0],0.0f,KINDA_SMALL_NUMBER);
	EXPECT_NEAR(CenterOfMass.Y - GlobalTranslation[1],0.0f,KINDA_SMALL_NUMBER);
	EXPECT_NEAR(CenterOfMass.Z - GlobalTranslation[2],0.0f,KINDA_SMALL_NUMBER);

	FFramework UnitTest;
	UnitTest.AddSimulationObject(Collection);
	UnitTest.Initialize();
	UnitTest.Advance();

	{ // test results
		const TManagedArray<FTransform>& MassToLocal = Collection->RestCollection->template GetAttribute<FTransform>("MassToLocal",FGeometryCollection::TransformGroup);


		EXPECT_EQ(UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().Size(), 1);
		FVector ParticlePos = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().GetX(0);
		FQuat ParticleR = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles().GetR(0);

		TArray<FTransform> Transform;
		GeometryCollectionAlgo::Private::GlobalMatrices(*Collection->DynamicCollection, Transform);
		EXPECT_EQ(Collection->DynamicCollection->GetParent(0), FGeometryCollection::Invalid); // is not a child
		EXPECT_NEAR(ParticlePos.X, GlobalTranslation[0], KINDA_SMALL_NUMBER); // Check particle position from solver 
		EXPECT_NEAR(ParticlePos.Y, GlobalTranslation[1], KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Transform[0].GetTranslation().X, 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Transform[0].GetTranslation().Y, 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_LT(Transform[0].GetTranslation().Z, 0.0f); // Check collection fell
		EXPECT_GT(Transform[0].GetTranslation().Z, -1.0f);
	}
}


} // namespace GeometryCollectionTest
