// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestMassProperties.h"

#include "Chaos/TriangleMesh.h"
#include "Chaos/MassProperties.h"
#include "Math/Box.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionTestFramework.h"
#include "GeometryCollection/GeometryCollectionTestUtility.h"
#include "GeometryCollection/TransformCollection.h"
#include "HeadlessChaosTestUtility.h"

namespace GeometryCollectionTest
{
	using namespace ChaosTest;

	GTEST_TEST(AllTraits, GeometryCollection_MassProperties_Compute)
	{
		using namespace Chaos;

		Chaos::FParticles Vertices;
		Vertices.AddParticles(8);
		Vertices.SetX(0, FVec3(-1, 1, -1));
		Vertices.SetX(1, FVec3(1, 1, -1));
		Vertices.SetX(2, FVec3(1, -1, -1));
		Vertices.SetX(3, FVec3(-1, -1, -1));
		Vertices.SetX(4, FVec3(-1, 1, 1));
		Vertices.SetX(5, FVec3(1, 1, 1));
		Vertices.SetX(6, FVec3(1, -1, 1));
		Vertices.SetX(7, FVec3(-1, -1, 1));

		// @todo(chaos):  breaking : this trips an ensure in the test, why?
		for (int i = 0; i < 8; i++) {
			Vertices.SetX(i, Vertices.GetX(i) * FVector(1, 2, 3));
			Vertices.SetX(i, Vertices.GetX(i) + FVector(1, 2, 3));
		}

		TArray<Chaos::TVec3<int32>> Faces;
		Faces.SetNum(12);
		Faces[0] = TVec3<int32>(0,1,2);
		Faces[1] = TVec3<int32>(0,2,3);
		Faces[2] = TVec3<int32>(2,1,6);
		Faces[3] = TVec3<int32>(1,5,6);
		Faces[4] = TVec3<int32>(2,6,7);
		Faces[5] = TVec3<int32>(3,2,7);
		Faces[6] = TVec3<int32>(4,7,3);
		Faces[7] = TVec3<int32>(4,0,3);
		Faces[8] = TVec3<int32>(4,1,0);
		Faces[9] = TVec3<int32>(4,5,1);
		Faces[10] = TVec3<int32>(5,4,7);
		Faces[11] = TVec3<int32>(5,7,6);
		Chaos::FTriangleMesh Surface(MoveTemp(Faces));

		FMassProperties MassProperties;
		MassProperties.Mass = 1.f;
		//Chaos::FMassProperties MassProperties = Chaos::CalculateMassProperties(Vertices, Surface.GetElements(), 1.f);
		{
			const auto& SurfaceElements = Surface.GetElements();
			CalculateVolumeAndCenterOfMass(Vertices, SurfaceElements, MassProperties.Volume, MassProperties.CenterOfMass);

			for (int32 Idx = 0; Idx < 8; ++Idx)
			{
				Vertices.SetX(Idx, Vertices.GetX(Idx) - MassProperties.CenterOfMass);
			}

			check(MassProperties.Mass > 0);
			check(MassProperties.Volume > SMALL_NUMBER);
			
			// @todo(chaos) : breaking
			CalculateInertiaAndRotationOfMass(Vertices, SurfaceElements, MassProperties.Mass / MassProperties.Volume, MassProperties.CenterOfMass, MassProperties.InertiaTensor, MassProperties.RotationOfMass);
		}
		EXPECT_EQ(MassProperties.Mass, 1.f);
		EXPECT_TRUE(MassProperties.CenterOfMass.Equals(FVector(1, 2, 3)));
		
		// This is just measured data to let us know when it changes. Ideally this would be derived. 
		FVector EulerAngle = MassProperties.RotationOfMass.Euler();
		EXPECT_TRUE(MassProperties.RotationOfMass.Euler().Equals(FVector(115.8153, -12.4347, 1.9705)));
		EXPECT_TRUE(FMath::IsNearlyEqual((FReal)MassProperties.InertiaTensor.M[0][0], static_cast<FReal>(14.9866095), (FReal)KINDA_SMALL_NUMBER));
		EXPECT_TRUE(FMath::IsNearlyEqual((FReal)MassProperties.InertiaTensor.M[1][1], static_cast<FReal>(1.40656376), (FReal)KINDA_SMALL_NUMBER));
		EXPECT_TRUE(FMath::IsNearlyEqual((FReal)MassProperties.InertiaTensor.M[2][2], static_cast<FReal>(13.7401619), (FReal)KINDA_SMALL_NUMBER));
	}

	GTEST_TEST(AllTraits, GeometryCollection_MassProperties_Cube)
	{
		using namespace Chaos;
		FVector GlobalTranslation(0); FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(0));
		CreationParameters Params; Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Box; Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.GeomTransform = FTransform(GlobalRotation, GlobalTranslation); Params.NestedTransforms = { FTransform::Identity, FTransform::Identity,  FTransform::Identity };
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		TArray<FTransform> Transform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->RestCollection->Transform, Collection->RestCollection->Parent, Transform);

		// group ?
		const TManagedArray<bool>& Visible = Collection->RestCollection->Visible;

		// VerticesGroup
		const TManagedArray<FVector3f>& Vertex = Collection->RestCollection->Vertex;

		// GeometryGroup
		const int32 NumGeometries = Collection->RestCollection->NumElements(FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& VertexCount = Collection->RestCollection->VertexCount;
		const TManagedArray<int32>& VertexStart = Collection->RestCollection->VertexStart;
		const TManagedArray<int32>& FaceCount = Collection->RestCollection->FaceCount;
		const TManagedArray<int32>& FaceStart = Collection->RestCollection->FaceStart;
		const TManagedArray<int32>& TransformIndex = Collection->RestCollection->TransformIndex;
		const TManagedArray<FIntVector>& Indices = Collection->RestCollection->Indices;
		const TManagedArray<int32>& BoneMap = Collection->RestCollection->BoneMap;
		int GeometryIndex = 0;

		TUniquePtr<FTriangleMesh> TriMesh(
			CreateTriangleMesh(
				FaceStart[GeometryIndex],
				FaceCount[GeometryIndex],
				Visible,
				Indices,
				false));

		//TArray<Chaos::TVec3<int32>> Faces;
		//Faces.SetNum(Indices.Num());
		//for (int i = 0; i < Indices.Num(); i++) { Faces[i] = TVec3<int32>(Indices[i][0], Indices[i][1], Indices[i][2]); }
		//Chaos::FTriangleMesh TriMesh(MoveTemp(Faces));

		TArray<FMassProperties> MassPropertiesArray;
		MassPropertiesArray.AddUninitialized(NumGeometries);
		FMassProperties& MassProperties = MassPropertiesArray[GeometryIndex];
		MassProperties.CenterOfMass = FVector(0);
		MassProperties.Mass = 1.0;

		FParticles MassSpaceParticles;
		MassSpaceParticles.AddParticles(Vertex.Num());
		for (int32 Idx = 0; Idx < Vertex.Num(); ++Idx)
		{
			MassSpaceParticles.SetX(Idx, Transform[BoneMap[Idx]].TransformPosition(FVector(Vertex[Idx])));
		}

		CalculateVolumeAndCenterOfMass(MassSpaceParticles, TriMesh->GetElements(), MassProperties.Volume, MassProperties.CenterOfMass);

		EXPECT_NEAR(MassProperties.Volume - 8.0, 0.0f, KINDA_SMALL_NUMBER); 
		EXPECT_NEAR(MassProperties.CenterOfMass.X - GlobalTranslation[0], 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(MassProperties.CenterOfMass.Y - GlobalTranslation[1], 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(MassProperties.CenterOfMass.Z - GlobalTranslation[2], 0.0f, KINDA_SMALL_NUMBER);

		for (int32 Idx = 0; Idx < Vertex.Num(); ++Idx)
		{
			MassSpaceParticles.SetX(Idx, MassSpaceParticles.GetX(Idx) - MassProperties.CenterOfMass);
		}

		FReal Density = 1.0;
		FVec3 ZeroVec(0);
		CalculateInertiaAndRotationOfMass(MassSpaceParticles, TriMesh->GetSurfaceElements(), Density, ZeroVec, MassProperties.InertiaTensor, MassProperties.RotationOfMass);

		EXPECT_EQ(MassProperties.Mass, 1.f);
		EXPECT_TRUE((MassProperties.CenterOfMass - GlobalTranslation).Size() < SMALL_NUMBER);

		// This is just measured data to let us know when it changes. Ideally this would be derived. 
		EXPECT_TRUE((MassProperties.RotationOfMass.Euler() - FVector(115.8153, -12.4347, 1.9705)).Size() > KINDA_SMALL_NUMBER);
		EXPECT_TRUE(FMath::IsNearlyEqual((FReal)MassProperties.InertiaTensor.M[0][0], static_cast<FReal>(4.99521351), (FReal)KINDA_SMALL_NUMBER));
		EXPECT_TRUE(FMath::IsNearlyEqual((FReal)MassProperties.InertiaTensor.M[1][1], static_cast<FReal>(4.07145357), (FReal)KINDA_SMALL_NUMBER));
		EXPECT_TRUE(FMath::IsNearlyEqual((FReal)MassProperties.InertiaTensor.M[2][2], static_cast<FReal>(4.26666689), (FReal)KINDA_SMALL_NUMBER));
	}

	GTEST_TEST(AllTraits, GeometryCollection_MassProperties_Sphere)
	{
		using namespace Chaos;
		FVector GlobalTranslation(10); FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(0)); FVector Scale(1);
		CreationParameters Params; Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere; Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
		Params.GeomTransform = FTransform(GlobalRotation, GlobalTranslation, Scale); Params.NestedTransforms = { FTransform::Identity, FTransform::Identity,  FTransform::Identity };
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		TArray<FTransform> Transform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->RestCollection->Transform, Collection->RestCollection->Parent, Transform);

		// group ?
		const TManagedArray<bool>& Visible = Collection->RestCollection->Visible;

		// VerticesGroup
		const TManagedArray<FVector3f>& Vertex = Collection->RestCollection->Vertex;

		// GeometryGroup
		const int32 NumGeometries = Collection->RestCollection->NumElements(FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& VertexCount = Collection->RestCollection->VertexCount;
		const TManagedArray<int32>& VertexStart = Collection->RestCollection->VertexStart;
		const TManagedArray<int32>& FaceCount = Collection->RestCollection->FaceCount;
		const TManagedArray<int32>& FaceStart = Collection->RestCollection->FaceStart;
		const TManagedArray<int32>& TransformIndex = Collection->RestCollection->TransformIndex;
		const TManagedArray<FIntVector>& Indices = Collection->RestCollection->Indices;
		const TManagedArray<int32>& BoneMap = Collection->RestCollection->BoneMap;
		int GeometryIndex = 0;

		TUniquePtr<FTriangleMesh> TriMesh(
			CreateTriangleMesh(
				FaceStart[GeometryIndex],
				FaceCount[GeometryIndex],
				Visible,
				Indices,
				false));

		TArray<FMassProperties> MassPropertiesArray;
		MassPropertiesArray.AddUninitialized(NumGeometries);
		FMassProperties& MassProperties = MassPropertiesArray[GeometryIndex];

		FParticles MassSpaceParticles;
		MassSpaceParticles.AddParticles(Vertex.Num());
		for (int32 Idx = 0; Idx < Vertex.Num(); ++Idx)
		{
			MassSpaceParticles.SetX(Idx, Transform[BoneMap[Idx]].TransformPosition(FVector(Vertex[Idx])));
		}

		CalculateVolumeAndCenterOfMass(MassSpaceParticles, TriMesh->GetElements(), MassProperties.Volume, MassProperties.CenterOfMass);

		// Since we're intersecting triangles with a sphere, where the vertices of the 
		// triangle vertices are on the sphere surface, we're missing some volume.  Thus,
		// we'd expect the volume of the triangulation to approach the analytic volume as
		// the number of polygons goes to infinity (MakeSphereElement() currently does 
		// 16x16 divisions in U and V).
		const FReal AnalyticVolume = (4.0/3) * (22.0/7) * Scale[0] * Scale[0] * Scale[0];
		EXPECT_NEAR(MassProperties.Volume - AnalyticVolume, 0.0f, 0.2); // this should be 4.19047642
		EXPECT_NEAR(MassProperties.CenterOfMass.X - GlobalTranslation[0], 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(MassProperties.CenterOfMass.Y - GlobalTranslation[1], 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(MassProperties.CenterOfMass.Z - GlobalTranslation[2], 0.0f, KINDA_SMALL_NUMBER);


		for (int32 Idx = 0; Idx < Vertex.Num(); ++Idx)
		{
			MassSpaceParticles.SetX(Idx, MassSpaceParticles.GetX(Idx) - MassProperties.CenterOfMass);
		}

		FReal Density = 0.01;
		FVec3 ZeroVec(0);
		CalculateInertiaAndRotationOfMass(MassSpaceParticles, TriMesh->GetSurfaceElements(), Density, ZeroVec, MassProperties.InertiaTensor, MassProperties.RotationOfMass);

		// todo(chaos) : Check this. 
		// This is just measured data to let us know when it changes. Ideally this would be derived. 
		FVector EulerAngle = MassProperties.RotationOfMass.Euler();
		//EXPECT_TRUE((MassProperties.RotationOfMass.Euler() - FVector(115.8153, -12.4347, 1.9705)).Size() > SMALL_NUMBER);
		//EXPECT_TRUE(MassProperties.InertiaTensor.M[0][0] - 4.99521351 < SMALL_NUMBER);
		//EXPECT_TRUE(MassProperties.InertiaTensor.M[1][1] - 4.07145357 < SMALL_NUMBER);
		//EXPECT_TRUE(MassProperties.InertiaTensor.M[2][2] - 4.26666689 < SMALL_NUMBER);
	}


	GTEST_TEST(AllTraits, GeometryCollection_MassProperties_Tetrahedron)
	{
		using namespace Chaos;
		FVector GlobalTranslation(0); FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(0));
		CreationParameters Params; Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Tetrahedron; Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
		Params.GeomTransform = FTransform(GlobalRotation, GlobalTranslation); Params.NestedTransforms = { FTransform::Identity, FTransform::Identity,  FTransform::Identity };
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		TArray<FTransform> Transform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->RestCollection->Transform, Collection->RestCollection->Parent, Transform);

		// group ?
		const TManagedArray<bool>& Visible = Collection->RestCollection->Visible;

		// VerticesGroup
		const TManagedArray<FVector3f>& Vertex = Collection->RestCollection->Vertex;

		// GeometryGroup
		const int32 NumGeometries = Collection->RestCollection->NumElements(FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& VertexCount = Collection->RestCollection->VertexCount;
		const TManagedArray<int32>& VertexStart = Collection->RestCollection->VertexStart;
		const TManagedArray<int32>& FaceCount = Collection->RestCollection->FaceCount;
		const TManagedArray<int32>& FaceStart = Collection->RestCollection->FaceStart;
		const TManagedArray<int32>& TransformIndex = Collection->RestCollection->TransformIndex;
		const TManagedArray<FIntVector>& Indices = Collection->RestCollection->Indices;
		const TManagedArray<int32>& BoneMap = Collection->RestCollection->BoneMap;
		int GeometryIndex = 0;

		TUniquePtr<FTriangleMesh> TriMesh(
			CreateTriangleMesh(
				FaceStart[GeometryIndex],
				FaceCount[GeometryIndex],
				Visible,
				Indices,
				false));

		TArray<FMassProperties> MassPropertiesArray;
		MassPropertiesArray.AddUninitialized(NumGeometries);
		FMassProperties& MassProperties = MassPropertiesArray[GeometryIndex];
		MassProperties.Mass = 1.0;
		MassProperties.CenterOfMass = FVector(0);

		FParticles MassSpaceParticles;
		MassSpaceParticles.AddParticles(Vertex.Num());
		for (int32 Idx = 0; Idx < Vertex.Num(); ++Idx)
		{
			MassSpaceParticles.SetX(Idx, Transform[BoneMap[Idx]].TransformPosition(FVector(Vertex[Idx])));
		}

		CalculateVolumeAndCenterOfMass(MassSpaceParticles, TriMesh->GetElements(), MassProperties.Volume, MassProperties.CenterOfMass);

		EXPECT_NEAR(MassProperties.Volume - 2.666666, 0.0f, KINDA_SMALL_NUMBER); // Tetrahedron with edge lengths 2.8284
		EXPECT_NEAR(MassProperties.CenterOfMass.X - GlobalTranslation[0], 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(MassProperties.CenterOfMass.Y - GlobalTranslation[1], 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(MassProperties.CenterOfMass.Z - GlobalTranslation[2], 0.0f, KINDA_SMALL_NUMBER);

		for (int32 Idx = 0; Idx < Vertex.Num(); ++Idx)
		{
			MassSpaceParticles.SetX(Idx, MassSpaceParticles.GetX(Idx) - MassProperties.CenterOfMass);
		}

		FReal Density = 0.01;
		FVec3 ZeroVec(0);
		CalculateInertiaAndRotationOfMass(MassSpaceParticles, TriMesh->GetSurfaceElements(), Density, ZeroVec, MassProperties.InertiaTensor, MassProperties.RotationOfMass);

		// todo(chaos) : Check this. 
		// This is just measured data to let us know when it changes. Ideally this would be derived. 
		FVector EulerAngle = MassProperties.RotationOfMass.Euler();
		//EXPECT_TRUE((MassProperties.RotationOfMass.Euler() - FVector(115.8153, -12.4347, 1.9705)).Size() > SMALL_NUMBER);
		//EXPECT_TRUE(MassProperties.InertiaTensor.M[0][0] - 4.99521351 < SMALL_NUMBER);
		//EXPECT_TRUE(MassProperties.InertiaTensor.M[1][1] - 4.07145357 < SMALL_NUMBER);
		//EXPECT_TRUE(MassProperties.InertiaTensor.M[2][2] - 4.26666689 < SMALL_NUMBER);
	}




	GTEST_TEST(AllTraits, GeometryCollection_MassProperties_ScaledSphere)
	{
		// This test has points that are scaled, rotated and translated within mass space. 
		// So the resulting surface is not about the center of mass and needs to be
		// moved for simulation. 

		using namespace Chaos;
		FVector GlobalTranslation(10); FQuat GlobalRotation = FQuat::MakeFromEuler(FVector(45,0,0));
		CreationParameters Params; Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere; Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
		Params.GeomTransform = FTransform(GlobalRotation,GlobalTranslation, FVector(1, 5, 11)); Params.NestedTransforms = { FTransform::Identity, FTransform::Identity,  FTransform::Identity };
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		TArray<FTransform> Transform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->RestCollection->Transform, Collection->RestCollection->Parent, Transform);

		// group ?
		const TManagedArray<bool>& Visible = Collection->RestCollection->Visible;

		// VerticesGroup
		TManagedArray<FVector3f>& Vertex = Collection->RestCollection->Vertex;

		// GeometryGroup
		const int32 NumGeometries = Collection->RestCollection->NumElements(FGeometryCollection::GeometryGroup);
		const TManagedArray<int32>& VertexCount = Collection->RestCollection->VertexCount;
		const TManagedArray<int32>& VertexStart = Collection->RestCollection->VertexStart;
		const TManagedArray<int32>& FaceCount = Collection->RestCollection->FaceCount;
		const TManagedArray<int32>& FaceStart = Collection->RestCollection->FaceStart;
		const TManagedArray<int32>& TransformIndex = Collection->RestCollection->TransformIndex;
		const TManagedArray<FIntVector>& Indices = Collection->RestCollection->Indices;
		const TManagedArray<int32>& BoneMap = Collection->RestCollection->BoneMap;
		int GeometryIndex = 0;

		TUniquePtr<FTriangleMesh> TriMesh(
			CreateTriangleMesh(
				FaceStart[GeometryIndex],
				FaceCount[GeometryIndex],
				Visible,
				Indices,
				false));


		TArray<FMassProperties> MassPropertiesArray;
		MassPropertiesArray.AddUninitialized(NumGeometries);
		FMassProperties& MassProperties = MassPropertiesArray[GeometryIndex];

		TArray<FVector> SomeVec;
		FParticles MassSpaceParticles;
		MassSpaceParticles.AddParticles(Vertex.Num());
		for (int32 Idx = 0; Idx < Vertex.Num(); ++Idx)
		{
			FVector VertexPoint = FVector(Vertex[Idx]);
			MassSpaceParticles.SetX(Idx, Transform[BoneMap[Idx]].TransformPosition(FVector(Vertex[Idx])));
			FVector MassSpacePoint = FVector(MassSpaceParticles.GetX(Idx)[0], MassSpaceParticles.GetX(Idx)[1], MassSpaceParticles.GetX(Idx)[2]);
			SomeVec.Add(MassSpacePoint);
		}

		FBox Bounds(SomeVec);
		CalculateVolumeAndCenterOfMass(MassSpaceParticles, TriMesh->GetElements(), MassProperties.Volume, MassProperties.CenterOfMass);
		
		EXPECT_NEAR(MassProperties.CenterOfMass.X - GlobalTranslation[0], 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(MassProperties.CenterOfMass.Y - GlobalTranslation[1], 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(MassProperties.CenterOfMass.Z - GlobalTranslation[2], 0.0f, KINDA_SMALL_NUMBER);

		for (int32 Idx = 0; Idx < Vertex.Num(); ++Idx)
		{
			MassSpaceParticles.SetX(Idx, MassSpaceParticles.GetX(Idx) - MassProperties.CenterOfMass);
		}

		FReal Density = 0.01;
		FVec3 ZeroVec(0);
		CalculateInertiaAndRotationOfMass(MassSpaceParticles, TriMesh->GetSurfaceElements(), Density, ZeroVec, MassProperties.InertiaTensor, MassProperties.RotationOfMass);

		FVector RotationEulerClamped = MassProperties.RotationOfMass.Euler();
		if (RotationEulerClamped[0] < 0.)
		{
			RotationEulerClamped[0] += 180.f;
		}
		
		// rotational alignment.
		EXPECT_NEAR(RotationEulerClamped[0], 135., KINDA_SMALL_NUMBER);
		EXPECT_NEAR(RotationEulerClamped[1], 0., KINDA_SMALL_NUMBER);
		EXPECT_NEAR(RotationEulerClamped[2], 0., KINDA_SMALL_NUMBER);
		// X dominate inertia tensor
		EXPECT_GT(MassProperties.InertiaTensor.M[0][0], MassProperties.InertiaTensor.M[2][2]);
		EXPECT_GT(MassProperties.InertiaTensor.M[0][0], MassProperties.InertiaTensor.M[1][1]);
	}

}