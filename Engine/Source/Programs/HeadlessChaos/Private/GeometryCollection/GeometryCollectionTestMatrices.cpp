// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestMatrices.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionTestUtility.h"
#include "GeometryCollection/TransformCollection.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

namespace GeometryCollectionTest
{
	DEFINE_LOG_CATEGORY_STATIC(GCTM_Log, Verbose, All);

	MATCHER_P2(VectorNear, V, Tolerance, "") { return arg.Equals(V, Tolerance); }
#define EXPECT_FVECTOR_NEAR(A,B,T) EXPECT_THAT(A, VectorNear(B, T)) << *FString("Expected: " + B.ToString() + "\nActual:   " + A.ToString());

	void BasicGlobalMatrices()
	{ 
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 90.)), FVector(0, 10, 0)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 90.)), FVector(0, 10, 0)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 90.)), FVector(0, 10, 0)), FVector(1.0)));

		//  0
		//  ...1
		//  ......2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);
		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);
		(Collection->Parent)[2] = 1;

		TArray<FTransform> GlobalTransform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransform);

		FVector Rot0 = GlobalTransform[0].GetRotation().Euler();
		EXPECT_FVECTOR_NEAR(Rot0, FVector(0, 0, 90), 0.0001);

		FVector Rot1 = GlobalTransform[1].GetRotation().Euler();
		EXPECT_TRUE(Rot1.Equals(FVector(0, 0, -180), 0.0001) || Rot1.Equals(FVector(0, 0, 180), 0.0001));

		FVector Rot2 = GlobalTransform[2].GetRotation().Euler();
		EXPECT_FVECTOR_NEAR(Rot2, FVector(0, 0, -90), 0.0001);

		FVector Pos0 = GlobalTransform[0].GetTranslation();
		EXPECT_FVECTOR_NEAR(Pos0, FVector(0, 10, 0), 0.0001);

		FVector Pos1 = GlobalTransform[1].GetTranslation();
		EXPECT_FVECTOR_NEAR(Pos1, FVector(-10, 10, 0), 0.0001);

		FVector Pos2 = GlobalTransform[2].GetTranslation();
		EXPECT_FVECTOR_NEAR(Pos2, FVector(-10, 0, 0), 0.0001);

		FTransform3f Frame = GeometryCollectionAlgo::GlobalMatrix3f(Collection->Transform, Collection->Parent, 2);
		EXPECT_FVECTOR_NEAR(Frame.GetRotation().Euler(), FVector3f(0, 0, -90), 0.0001);
		EXPECT_FVECTOR_NEAR(Frame.GetTranslation(), FVector3f(-10, 0, 0), 0.0001);

		Frame = GeometryCollectionAlgo::GlobalMatrix3f(Collection->Transform, Collection->Parent, 1);
		FVector3f FrameRot = Frame.GetRotation().Euler();
		EXPECT_TRUE(FrameRot.Equals(FVector3f(0, 0, -180), 0.0001f) || FrameRot.Equals(FVector3f(0, 0, 180), 0.0001f));
		EXPECT_FVECTOR_NEAR(Frame.GetTranslation(), FVector3f(-10, 10, 0), 0.0001f);
	}

	void ReparentingMatrices()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 90.)), FVector(0, 10, 0)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 90.)), FVector(0, 10, 0)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, -90.)), FVector(-10, 0, 0)), FVector(1.0)));

		//  0
		//  ...1
		//  2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);
		(Collection->Parent)[1] = 0;
		(Collection->Parent)[2] = -1;

		//  0
		//  ...1
		//  ......2
		TArray<int32> Bones = { 2 };
		GeometryCollectionAlgo::ParentTransforms(Collection.Get(), 1, Bones);
		EXPECT_FVECTOR_NEAR((Collection->Transform)[2].GetTranslation(), FVector3f(0, 10, 0), 0.0001);
		EXPECT_FVECTOR_NEAR((Collection->Transform)[2].GetRotation().Euler(), FVector3f(0, 0, 90.), 0.0001);

		TArray<FTransform> GlobalTransform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransform);

		FVector Rot0 = GlobalTransform[0].GetRotation().Euler();
		EXPECT_FVECTOR_NEAR(Rot0, FVector(0, 0, 90), 0.0001);

		FVector Rot1 = GlobalTransform[1].GetRotation().Euler();
		EXPECT_TRUE(Rot1.Equals(FVector(0, 0, -180), 0.0001) || Rot1.Equals(FVector(0, 0, 180), 0.0001));

		FVector Rot2 = GlobalTransform[2].GetRotation().Euler();
		EXPECT_FVECTOR_NEAR(Rot2, FVector(0, 0, -90), 0.0001);

		FVector Pos0 = GlobalTransform[0].GetTranslation();
		EXPECT_FVECTOR_NEAR(Pos0, FVector(0, 10, 0), 0.0001);

		FVector Pos1 = GlobalTransform[1].GetTranslation();
		EXPECT_FVECTOR_NEAR(Pos1, FVector(-10, 10, 0), 0.0001);

		FVector Pos2 = GlobalTransform[2].GetTranslation();
		EXPECT_FVECTOR_NEAR(Pos2, FVector(-10, 0, 0), 0.0001);
	}

	void TransformMatrixElement()
	{
		FTransformCollection Collection;

		int index=0;
		for (int i=0; i < 4; i++)
		{
			index = Collection.AddElements(1, FGeometryCollection::TransformGroup);
			(Collection.Parent)[index] = index - 1;
			(Collection.Children)[index].Add(index+1);
		}
		(Collection.Children)[index].Empty();

		(Collection.Transform)[0] = FTransform3f(FQuat4f::MakeFromEuler(FVector3f(0, 0, 0)), FVector3f(0, 0, 0));
		(Collection.Transform)[1] = FTransform3f(FQuat4f::MakeFromEuler(FVector3f(0, 0, 90.)), FVector3f(1, 0, 0));
		(Collection.Transform)[2] = FTransform3f(FQuat4f::MakeFromEuler(FVector3f(0, 90.,0)), FVector3f(1, 0, 0));
		(Collection.Transform)[3] = FTransform3f(FQuat4f::MakeFromEuler(FVector3f(0, 0, 0)), FVector3f(1, 0, 0));

		TArray<FTransform> GlobalMatrices0;
		GeometryCollectionAlgo::GlobalMatrices(Collection.Transform, Collection.Parent, GlobalMatrices0);

		Collection.RelativeTransformation(1, FTransform(FQuat::MakeFromEuler(FVector(22, 90, 55)), FVector(17,11,13)));
		Collection.RelativeTransformation(2, FTransform(FQuat::MakeFromEuler(FVector(22, 90, 55)), FVector(17, 11, 13)));

		TArray<FTransform> GlobalMatrices1;
		GeometryCollectionAlgo::GlobalMatrices(Collection.Transform, Collection.Parent, GlobalMatrices1);

		
		//for (int rdx = 0; rdx < GlobalMatrices0.Num(); rdx++)
		//{
		//	UE_LOGF(GCTM_Log, Verbose, "... Position[%d] : (%3.5f,%3.5f,%3.5f)", rdx, (Collection.Transform)[rdx].GetTranslation().X, (Collection.Transform)[rdx].GetTranslation().Y, (Collection.Transform)[rdx].GetTranslation().Z);
		//	UE_LOGF(GCTM_Log, Verbose, "... GlobalM0[%d] : (%3.5f,%3.5f,%3.5f)", rdx, GlobalMatrices0[rdx].GetTranslation().X, GlobalMatrices0[rdx].GetTranslation().Y, GlobalMatrices0[rdx].GetTranslation().Z);
		//	UE_LOGF(GCTM_Log, Verbose, "... GlobalM1[%d] : (%3.5f,%3.5f,%3.5f)", rdx, GlobalMatrices1[rdx].GetTranslation().X, GlobalMatrices1[rdx].GetTranslation().Y, GlobalMatrices1[rdx].GetTranslation().Z);
		//}
		EXPECT_LT(FMath::Abs((GlobalMatrices1[0].GetTranslation() - GlobalMatrices0[0].GetTranslation()).Size()), 1e-3);
		EXPECT_GT(FMath::Abs((GlobalMatrices1[1].GetTranslation() - GlobalMatrices0[1].GetTranslation()).Size()), 1.0);
		EXPECT_GT(FMath::Abs((GlobalMatrices1[2].GetTranslation() - GlobalMatrices0[2].GetTranslation()).Size()), 1.0);
		EXPECT_LT(FMath::Abs((GlobalMatrices1[3].GetTranslation() - GlobalMatrices0[3].GetTranslation()).Size()), 1e-3);
	}

}

