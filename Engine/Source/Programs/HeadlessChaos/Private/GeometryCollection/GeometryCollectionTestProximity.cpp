// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestProximity.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/TransformCollection.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "../Resource/FracturedGeometry.h"

namespace GeometryCollectionTest
{
	void BuildProximity()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.f, 0.f, 0.f)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.5f, 0.f, 1.f)),FVector(1.0)));

		//  0
		//  ...1
		//  ......2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);
		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);
		(Collection->Parent)[2] = 1;
//		GeometryCollectionAlgo::ParentTransform(Collection, 0, 1);

		FGeometryCollection* Coll = Collection.Get();

		TArray<FTransform> GlobalTransform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransform);

		FGeometryCollectionProximityUtility ProximityUtility(Coll);
		ProximityUtility.UpdateProximity();

		// Breaking Data
		const TManagedArray<TSet<int32>>& Proximity = Collection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);

		EXPECT_TRUE((Proximity)[0].Contains(1));
		EXPECT_TRUE((Proximity)[1].Contains(0));
		EXPECT_TRUE((Proximity)[1].Contains(2));
		EXPECT_TRUE((Proximity)[2].Contains(1));

		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(0));
	}

	void GeometryDeleteFromStart()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.f, 0.f, 0.f)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(2.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(-0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.5f, 0.f, 1.f)),FVector(1.0)));

		//  0
		//  ...1
		//  ......2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);

		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);

		(Collection->Parent)[2] = 0;
		(Collection->Children)[2].Add(3);

		(Collection->Parent)[3] = 0;
		(Collection->Children)[3].Add(4);

		(Collection->Parent)[4] = 0;
		(Collection->Children)[4].Add(5);

		(Collection->Parent)[5] = 0;
		//		GeometryCollectionAlgo::ParentTransform(Collection, 0, 1);

		FGeometryCollection* Coll = Collection.Get();

		TArray<FTransform> GlobalTransform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransform);

		FGeometryCollectionProximityUtility ProximityUtility(Coll);
		ProximityUtility.UpdateProximity();

		// Breaking Data
		const TManagedArray<TSet<int32>>& Proximity = Collection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);

		// Proximity = [(3,4,1), (0,4,5,2), (1,5), (0,4), (0,1,3,5), (1,2,4)]

		EXPECT_TRUE((Proximity)[0].Contains(3));
		EXPECT_TRUE((Proximity)[0].Contains(4));
		EXPECT_TRUE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(5));

		EXPECT_TRUE((Proximity)[1].Contains(0));
		EXPECT_TRUE((Proximity)[1].Contains(4));
		EXPECT_TRUE((Proximity)[1].Contains(5));
		EXPECT_TRUE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(3));

		EXPECT_TRUE((Proximity)[2].Contains(1));
		EXPECT_TRUE((Proximity)[2].Contains(5));
		EXPECT_FALSE((Proximity)[2].Contains(0));
		EXPECT_FALSE((Proximity)[2].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(3));
		EXPECT_FALSE((Proximity)[2].Contains(4));

		EXPECT_TRUE((Proximity)[3].Contains(0));
		EXPECT_TRUE((Proximity)[3].Contains(4));
		EXPECT_FALSE((Proximity)[3].Contains(1));
		EXPECT_FALSE((Proximity)[3].Contains(2));
		EXPECT_FALSE((Proximity)[3].Contains(3));
		EXPECT_FALSE((Proximity)[3].Contains(5));

		EXPECT_TRUE((Proximity)[4].Contains(0));
		EXPECT_TRUE((Proximity)[4].Contains(1));
		EXPECT_TRUE((Proximity)[4].Contains(3));
		EXPECT_TRUE((Proximity)[4].Contains(5));
		EXPECT_FALSE((Proximity)[4].Contains(2));
		EXPECT_FALSE((Proximity)[4].Contains(4));

		EXPECT_TRUE((Proximity)[5].Contains(1));
		EXPECT_TRUE((Proximity)[5].Contains(2));
		EXPECT_TRUE((Proximity)[5].Contains(4));
		EXPECT_FALSE((Proximity)[5].Contains(0));
		EXPECT_FALSE((Proximity)[5].Contains(3));
		EXPECT_FALSE((Proximity)[5].Contains(5));

		TArray<int32> DelList = { 0 };
		Coll->RemoveElements(FGeometryCollection::GeometryGroup, DelList);

		// Proximity = [(3,4,1), (0,4), (3), (0,2,4), (0,1,3)]

		EXPECT_TRUE((Proximity)[0].Contains(3));
		EXPECT_TRUE((Proximity)[0].Contains(4));
		EXPECT_TRUE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(5));

		EXPECT_TRUE((Proximity)[1].Contains(0));
		EXPECT_TRUE((Proximity)[1].Contains(4));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(3));
		EXPECT_FALSE((Proximity)[1].Contains(5));

		EXPECT_TRUE((Proximity)[2].Contains(3));
		EXPECT_FALSE((Proximity)[2].Contains(5));
		EXPECT_FALSE((Proximity)[2].Contains(0));
		EXPECT_FALSE((Proximity)[2].Contains(1));
		EXPECT_FALSE((Proximity)[2].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(4));

		EXPECT_TRUE((Proximity)[3].Contains(0));
		EXPECT_TRUE((Proximity)[3].Contains(2));
		EXPECT_TRUE((Proximity)[3].Contains(4));
		EXPECT_FALSE((Proximity)[3].Contains(1));
		EXPECT_FALSE((Proximity)[3].Contains(3));
		EXPECT_FALSE((Proximity)[3].Contains(5));

		EXPECT_TRUE((Proximity)[4].Contains(0));
		EXPECT_TRUE((Proximity)[4].Contains(1));
		EXPECT_TRUE((Proximity)[4].Contains(3));
		EXPECT_FALSE((Proximity)[4].Contains(2));
		EXPECT_FALSE((Proximity)[4].Contains(4));
		EXPECT_FALSE((Proximity)[4].Contains(5));

		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 5);
	}

	void GeometryDeleteFromEnd()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.f, 0.f, 0.f)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(2.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(-0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.5f, 0.f, 1.f)),FVector(1.0)));

		//  0
		//  ...1
		//  ......2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);

		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);

		(Collection->Parent)[2] = 0;
		(Collection->Children)[2].Add(3);

		(Collection->Parent)[3] = 0;
		(Collection->Children)[3].Add(4);

		(Collection->Parent)[4] = 0;
		(Collection->Children)[4].Add(5);

		(Collection->Parent)[5] = 0;
		//		GeometryCollectionAlgo::ParentTransform(Collection, 0, 1);

		FGeometryCollection* Coll = Collection.Get();

		TArray<FTransform> GlobalTransform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransform);

		FGeometryCollectionProximityUtility ProximityUtility(Coll);
		ProximityUtility.UpdateProximity();

		// Breaking Data
		const TManagedArray<TSet<int32>>& Proximity = Collection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);

		// Proximity = [(3,4,1), (0,4,5,2), (1,5), (0,4), (0,1,3,5), (1,2,4)]

		EXPECT_TRUE((Proximity)[0].Contains(3));
		EXPECT_TRUE((Proximity)[0].Contains(4));
		EXPECT_TRUE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(5));

		EXPECT_TRUE((Proximity)[1].Contains(0));
		EXPECT_TRUE((Proximity)[1].Contains(4));
		EXPECT_TRUE((Proximity)[1].Contains(5));
		EXPECT_TRUE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(3));

		EXPECT_TRUE((Proximity)[2].Contains(1));
		EXPECT_TRUE((Proximity)[2].Contains(5));
		EXPECT_FALSE((Proximity)[2].Contains(0));
		EXPECT_FALSE((Proximity)[2].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(3));
		EXPECT_FALSE((Proximity)[2].Contains(4));

		EXPECT_TRUE((Proximity)[3].Contains(0));
		EXPECT_TRUE((Proximity)[3].Contains(4));
		EXPECT_FALSE((Proximity)[3].Contains(1));
		EXPECT_FALSE((Proximity)[3].Contains(2));
		EXPECT_FALSE((Proximity)[3].Contains(3));
		EXPECT_FALSE((Proximity)[3].Contains(5));

		EXPECT_TRUE((Proximity)[4].Contains(0));
		EXPECT_TRUE((Proximity)[4].Contains(1));
		EXPECT_TRUE((Proximity)[4].Contains(3));
		EXPECT_TRUE((Proximity)[4].Contains(5));
		EXPECT_FALSE((Proximity)[4].Contains(2));
		EXPECT_FALSE((Proximity)[4].Contains(4));

		EXPECT_TRUE((Proximity)[5].Contains(1));
		EXPECT_TRUE((Proximity)[5].Contains(2));
		EXPECT_TRUE((Proximity)[5].Contains(4));
		EXPECT_FALSE((Proximity)[5].Contains(0));
		EXPECT_FALSE((Proximity)[5].Contains(3));
		EXPECT_FALSE((Proximity)[5].Contains(5));

		TArray<int32> DelList = { 5 };
		Coll->RemoveElements(FGeometryCollection::GeometryGroup, DelList);

		// Proximity = [(3,4,1), (0,4,2), (1), (0,4), (0,1,3)]

		EXPECT_TRUE((Proximity)[0].Contains(3));
		EXPECT_TRUE((Proximity)[0].Contains(4));
		EXPECT_TRUE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(5));

		EXPECT_TRUE((Proximity)[1].Contains(0));
		EXPECT_TRUE((Proximity)[1].Contains(4));
		EXPECT_TRUE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(3));
		EXPECT_FALSE((Proximity)[1].Contains(5));

		EXPECT_TRUE((Proximity)[2].Contains(1));
		EXPECT_FALSE((Proximity)[2].Contains(5));
		EXPECT_FALSE((Proximity)[2].Contains(0));
		EXPECT_FALSE((Proximity)[2].Contains(3));
		EXPECT_FALSE((Proximity)[2].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(4));

		EXPECT_TRUE((Proximity)[3].Contains(0));
		EXPECT_TRUE((Proximity)[3].Contains(4));
		EXPECT_FALSE((Proximity)[3].Contains(2));
		EXPECT_FALSE((Proximity)[3].Contains(1));
		EXPECT_FALSE((Proximity)[3].Contains(3));
		EXPECT_FALSE((Proximity)[3].Contains(5));

		EXPECT_TRUE((Proximity)[4].Contains(0));
		EXPECT_TRUE((Proximity)[4].Contains(1));
		EXPECT_TRUE((Proximity)[4].Contains(3));
		EXPECT_FALSE((Proximity)[4].Contains(2));
		EXPECT_FALSE((Proximity)[4].Contains(4));
		EXPECT_FALSE((Proximity)[4].Contains(5));

		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 5);
	}

	void GeometryDeleteFromMiddle()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.f, 0.f, 0.f)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(2.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(-0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.5f, 0.f, 1.f)),FVector(1.0)));

		//  0
		//  ...1
		//  ......2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);

		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);

		(Collection->Parent)[2] = 0;
		(Collection->Children)[2].Add(3);

		(Collection->Parent)[3] = 0;
		(Collection->Children)[3].Add(4);

		(Collection->Parent)[4] = 0;
		(Collection->Children)[4].Add(5);

		(Collection->Parent)[5] = 0;
		//		GeometryCollectionAlgo::ParentTransform(Collection, 0, 1);

		FGeometryCollection* Coll = Collection.Get();

		TArray<FTransform> GlobalTransform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransform);

		FGeometryCollectionProximityUtility ProximityUtility(Coll);
		ProximityUtility.UpdateProximity();

		// Breaking Data
		const TManagedArray<TSet<int32>>& Proximity = Collection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);

		// Proximity = [(3,4,1), (0,4,5,2), (1,5), (0,4), (0,1,3,5), (1,2,4)]

		EXPECT_TRUE((Proximity)[0].Contains(3));
		EXPECT_TRUE((Proximity)[0].Contains(4));
		EXPECT_TRUE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(5));

		EXPECT_TRUE((Proximity)[1].Contains(0));
		EXPECT_TRUE((Proximity)[1].Contains(4));
		EXPECT_TRUE((Proximity)[1].Contains(5));
		EXPECT_TRUE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(3));

		EXPECT_TRUE((Proximity)[2].Contains(1));
		EXPECT_TRUE((Proximity)[2].Contains(5));
		EXPECT_FALSE((Proximity)[2].Contains(0));
		EXPECT_FALSE((Proximity)[2].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(3));
		EXPECT_FALSE((Proximity)[2].Contains(4));

		EXPECT_TRUE((Proximity)[3].Contains(0));
		EXPECT_TRUE((Proximity)[3].Contains(4));
		EXPECT_FALSE((Proximity)[3].Contains(1));
		EXPECT_FALSE((Proximity)[3].Contains(2));
		EXPECT_FALSE((Proximity)[3].Contains(3));
		EXPECT_FALSE((Proximity)[3].Contains(5));

		EXPECT_TRUE((Proximity)[4].Contains(0));
		EXPECT_TRUE((Proximity)[4].Contains(1));
		EXPECT_TRUE((Proximity)[4].Contains(3));
		EXPECT_TRUE((Proximity)[4].Contains(5));
		EXPECT_FALSE((Proximity)[4].Contains(2));
		EXPECT_FALSE((Proximity)[4].Contains(4));

		EXPECT_TRUE((Proximity)[5].Contains(1));
		EXPECT_TRUE((Proximity)[5].Contains(2));
		EXPECT_TRUE((Proximity)[5].Contains(4));
		EXPECT_FALSE((Proximity)[5].Contains(0));
		EXPECT_FALSE((Proximity)[5].Contains(3));
		EXPECT_FALSE((Proximity)[5].Contains(5));

		TArray<int32> DelList = { 3 };
		Coll->RemoveElements(FGeometryCollection::GeometryGroup, DelList);

		// Proximity = [(3,1), (0,3,4,2), (1,4), (0,1,4), (1,2,3)]

		EXPECT_TRUE((Proximity)[0].Contains(3));
		EXPECT_TRUE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(4));
		EXPECT_FALSE((Proximity)[0].Contains(5));

	
		EXPECT_TRUE((Proximity)[1].Contains(0));
		EXPECT_TRUE((Proximity)[1].Contains(3));
		EXPECT_TRUE((Proximity)[1].Contains(4));
		EXPECT_TRUE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(5));

		EXPECT_TRUE((Proximity)[2].Contains(1));
		EXPECT_TRUE((Proximity)[2].Contains(4));
		EXPECT_FALSE((Proximity)[2].Contains(0));
		EXPECT_FALSE((Proximity)[2].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(3));
		EXPECT_FALSE((Proximity)[2].Contains(5));

		EXPECT_TRUE((Proximity)[3].Contains(0));
		EXPECT_TRUE((Proximity)[3].Contains(1));
		EXPECT_TRUE((Proximity)[3].Contains(4));
		EXPECT_FALSE((Proximity)[3].Contains(2));
		EXPECT_FALSE((Proximity)[3].Contains(3));
		EXPECT_FALSE((Proximity)[3].Contains(5));

		EXPECT_TRUE((Proximity)[4].Contains(1));
		EXPECT_TRUE((Proximity)[4].Contains(2));
		EXPECT_TRUE((Proximity)[4].Contains(3));
		EXPECT_FALSE((Proximity)[4].Contains(0));
		EXPECT_FALSE((Proximity)[4].Contains(4));
		EXPECT_FALSE((Proximity)[4].Contains(5));

		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 5);
	}

	void GeometryDeleteMultipleFromMiddle()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.f, 0.f, 0.f)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(2.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(-0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.5f, 0.f, 1.f)),FVector(1.0)));

		//  0
		//  ...1
		//  ......2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);

		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);

		(Collection->Parent)[2] = 0;
		(Collection->Children)[2].Add(3);

		(Collection->Parent)[3] = 0;
		(Collection->Children)[3].Add(4);

		(Collection->Parent)[4] = 0;
		(Collection->Children)[4].Add(5);

		(Collection->Parent)[5] = 0;
		//		GeometryCollectionAlgo::ParentTransform(Collection, 0, 1);

		FGeometryCollection* Coll = Collection.Get();

		TArray<FTransform> GlobalTransform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransform);

		FGeometryCollectionProximityUtility ProximityUtility(Coll);
		ProximityUtility.UpdateProximity();

		// Breaking Data
		const TManagedArray<TSet<int32>>& Proximity = Collection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);

		// Proximity = [(3,4,1), (0,4,5,2), (1,5), (0,4), (0,1,3,5), (1,2,4)]

		EXPECT_TRUE((Proximity)[0].Contains(3));
		EXPECT_TRUE((Proximity)[0].Contains(4));
		EXPECT_TRUE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(5));

		EXPECT_TRUE((Proximity)[1].Contains(0));
		EXPECT_TRUE((Proximity)[1].Contains(4));
		EXPECT_TRUE((Proximity)[1].Contains(5));
		EXPECT_TRUE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(3));

		EXPECT_TRUE((Proximity)[2].Contains(1));
		EXPECT_TRUE((Proximity)[2].Contains(5));
		EXPECT_FALSE((Proximity)[2].Contains(0));
		EXPECT_FALSE((Proximity)[2].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(3));
		EXPECT_FALSE((Proximity)[2].Contains(4));

		EXPECT_TRUE((Proximity)[3].Contains(0));
		EXPECT_TRUE((Proximity)[3].Contains(4));
		EXPECT_FALSE((Proximity)[3].Contains(1));
		EXPECT_FALSE((Proximity)[3].Contains(2));
		EXPECT_FALSE((Proximity)[3].Contains(3));
		EXPECT_FALSE((Proximity)[3].Contains(5));

		EXPECT_TRUE((Proximity)[4].Contains(0));
		EXPECT_TRUE((Proximity)[4].Contains(1));
		EXPECT_TRUE((Proximity)[4].Contains(3));
		EXPECT_TRUE((Proximity)[4].Contains(5));
		EXPECT_FALSE((Proximity)[4].Contains(2));
		EXPECT_FALSE((Proximity)[4].Contains(4));

		EXPECT_TRUE((Proximity)[5].Contains(1));
		EXPECT_TRUE((Proximity)[5].Contains(2));
		EXPECT_TRUE((Proximity)[5].Contains(4));
		EXPECT_FALSE((Proximity)[5].Contains(0));
		EXPECT_FALSE((Proximity)[5].Contains(3));
		EXPECT_FALSE((Proximity)[5].Contains(5));

		TArray<int32> DelList = { 2,3,4 };
		Coll->RemoveElements(FGeometryCollection::GeometryGroup, DelList);

		// Proximity = [(1), (0,2), (1)]

		EXPECT_TRUE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(3));
		EXPECT_FALSE((Proximity)[0].Contains(4));
		EXPECT_FALSE((Proximity)[0].Contains(5));

		EXPECT_TRUE((Proximity)[1].Contains(0));
		EXPECT_TRUE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(3));
		EXPECT_FALSE((Proximity)[1].Contains(4));
		EXPECT_FALSE((Proximity)[1].Contains(5));

		EXPECT_TRUE((Proximity)[2].Contains(1));
		EXPECT_FALSE((Proximity)[2].Contains(4));
		EXPECT_FALSE((Proximity)[2].Contains(0));
		EXPECT_FALSE((Proximity)[2].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(3));
		EXPECT_FALSE((Proximity)[2].Contains(5));

		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 3);
	}

	void GeometryDeleteRandom()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.f, 0.f, 0.f)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(2.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(-0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.5f, 0.f, 1.f)),FVector(1.0)));

		//  0
		//  ...1
		//  ......2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);

		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);

		(Collection->Parent)[2] = 0;
		(Collection->Children)[2].Add(3);

		(Collection->Parent)[3] = 0;
		(Collection->Children)[3].Add(4);

		(Collection->Parent)[4] = 0;
		(Collection->Children)[4].Add(5);

		(Collection->Parent)[5] = 0;
		//		GeometryCollectionAlgo::ParentTransform(Collection, 0, 1);

		FGeometryCollection* Coll = Collection.Get();

		TArray<FTransform> GlobalTransform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransform);

		FGeometryCollectionProximityUtility ProximityUtility(Coll);
		ProximityUtility.UpdateProximity();

		// Breaking Data
		const TManagedArray<TSet<int32>>& Proximity = Collection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);

		// Proximity = [(3,4,1), (0,4,5,2), (1,5), (0,4), (0,1,3,5), (1,2,4)]

		EXPECT_TRUE((Proximity)[0].Contains(3));
		EXPECT_TRUE((Proximity)[0].Contains(4));
		EXPECT_TRUE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(5));

		EXPECT_TRUE((Proximity)[1].Contains(0));
		EXPECT_TRUE((Proximity)[1].Contains(4));
		EXPECT_TRUE((Proximity)[1].Contains(5));
		EXPECT_TRUE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(3));

		EXPECT_TRUE((Proximity)[2].Contains(1));
		EXPECT_TRUE((Proximity)[2].Contains(5));
		EXPECT_FALSE((Proximity)[2].Contains(0));
		EXPECT_FALSE((Proximity)[2].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(3));
		EXPECT_FALSE((Proximity)[2].Contains(4));

		EXPECT_TRUE((Proximity)[3].Contains(0));
		EXPECT_TRUE((Proximity)[3].Contains(4));
		EXPECT_FALSE((Proximity)[3].Contains(1));
		EXPECT_FALSE((Proximity)[3].Contains(2));
		EXPECT_FALSE((Proximity)[3].Contains(3));
		EXPECT_FALSE((Proximity)[3].Contains(5));

		EXPECT_TRUE((Proximity)[4].Contains(0));
		EXPECT_TRUE((Proximity)[4].Contains(1));
		EXPECT_TRUE((Proximity)[4].Contains(3));
		EXPECT_TRUE((Proximity)[4].Contains(5));
		EXPECT_FALSE((Proximity)[4].Contains(2));
		EXPECT_FALSE((Proximity)[4].Contains(4));

		EXPECT_TRUE((Proximity)[5].Contains(1));
		EXPECT_TRUE((Proximity)[5].Contains(2));
		EXPECT_TRUE((Proximity)[5].Contains(4));
		EXPECT_FALSE((Proximity)[5].Contains(0));
		EXPECT_FALSE((Proximity)[5].Contains(3));
		EXPECT_FALSE((Proximity)[5].Contains(5));

		TArray<int32> DelList = { 1,3,5 };
		Coll->RemoveElements(FGeometryCollection::GeometryGroup, DelList);

		// Proximity = [(2), (), (0)]

		EXPECT_TRUE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(3));
		EXPECT_FALSE((Proximity)[0].Contains(4));
		EXPECT_FALSE((Proximity)[0].Contains(5));

		EXPECT_FALSE((Proximity)[1].Contains(0));
		EXPECT_FALSE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(3));
		EXPECT_FALSE((Proximity)[1].Contains(4));
		EXPECT_FALSE((Proximity)[1].Contains(5));

		EXPECT_TRUE((Proximity)[2].Contains(0));
		EXPECT_FALSE((Proximity)[2].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(1));
		EXPECT_FALSE((Proximity)[2].Contains(3));
		EXPECT_FALSE((Proximity)[2].Contains(4));
		EXPECT_FALSE((Proximity)[2].Contains(5));

		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 3);
	}

	void GeometryDeleteRandom2()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.f, 0.f, 0.f)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(2.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(-0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.5f, 0.f, 1.f)),FVector(1.0)));

		//  0
		//  ...1
		//  ......2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);

		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);

		(Collection->Parent)[2] = 0;
		(Collection->Children)[2].Add(3);

		(Collection->Parent)[3] = 0;
		(Collection->Children)[3].Add(4);

		(Collection->Parent)[4] = 0;
		(Collection->Children)[4].Add(5);

		(Collection->Parent)[5] = 0;
		//		GeometryCollectionAlgo::ParentTransform(Collection, 0, 1);

		FGeometryCollection* Coll = Collection.Get();

		TArray<FTransform> GlobalTransform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransform);

		FGeometryCollectionProximityUtility ProximityUtility(Coll);
		ProximityUtility.UpdateProximity();

		// Breaking Data
		const TManagedArray<TSet<int32>>& Proximity = Collection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);

		// Proximity = [(3,4,1), (0,4,5,2), (1,5), (0,4), (0,1,3,5), (1,2,4)]

		EXPECT_TRUE((Proximity)[0].Contains(3));
		EXPECT_TRUE((Proximity)[0].Contains(4));
		EXPECT_TRUE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(5));

		EXPECT_TRUE((Proximity)[1].Contains(0));
		EXPECT_TRUE((Proximity)[1].Contains(4));
		EXPECT_TRUE((Proximity)[1].Contains(5));
		EXPECT_TRUE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(3));

		EXPECT_TRUE((Proximity)[2].Contains(1));
		EXPECT_TRUE((Proximity)[2].Contains(5));
		EXPECT_FALSE((Proximity)[2].Contains(0));
		EXPECT_FALSE((Proximity)[2].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(3));
		EXPECT_FALSE((Proximity)[2].Contains(4));

		EXPECT_TRUE((Proximity)[3].Contains(0));
		EXPECT_TRUE((Proximity)[3].Contains(4));
		EXPECT_FALSE((Proximity)[3].Contains(1));
		EXPECT_FALSE((Proximity)[3].Contains(2));
		EXPECT_FALSE((Proximity)[3].Contains(3));
		EXPECT_FALSE((Proximity)[3].Contains(5));

		EXPECT_TRUE((Proximity)[4].Contains(0));
		EXPECT_TRUE((Proximity)[4].Contains(1));
		EXPECT_TRUE((Proximity)[4].Contains(3));
		EXPECT_TRUE((Proximity)[4].Contains(5));
		EXPECT_FALSE((Proximity)[4].Contains(2));
		EXPECT_FALSE((Proximity)[4].Contains(4));

		EXPECT_TRUE((Proximity)[5].Contains(1));
		EXPECT_TRUE((Proximity)[5].Contains(2));
		EXPECT_TRUE((Proximity)[5].Contains(4));
		EXPECT_FALSE((Proximity)[5].Contains(0));
		EXPECT_FALSE((Proximity)[5].Contains(3));
		EXPECT_FALSE((Proximity)[5].Contains(5));

		TArray<int32> DelList = { 0,1,4,5 };
		Coll->RemoveElements(FGeometryCollection::GeometryGroup, DelList);

		// Proximity = [(), ()]

		EXPECT_FALSE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(3));
		EXPECT_FALSE((Proximity)[0].Contains(4));
		EXPECT_FALSE((Proximity)[0].Contains(5));

		EXPECT_FALSE((Proximity)[1].Contains(0));
		EXPECT_FALSE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(3));
		EXPECT_FALSE((Proximity)[1].Contains(4));
		EXPECT_FALSE((Proximity)[1].Contains(5));

		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 2);
	}

	void GeometryDeleteAll()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.f, 0.f, 0.f)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(2.f, 0.f, 0.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(-0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.5f, 0.f, 1.f)),FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.5f, 0.f, 1.f)),FVector(1.0)));

		//  0
		//  ...1
		//  ......2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);

		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);

		(Collection->Parent)[2] = 0;
		(Collection->Children)[2].Add(3);

		(Collection->Parent)[3] = 0;
		(Collection->Children)[3].Add(4);

		(Collection->Parent)[4] = 0;
		(Collection->Children)[4].Add(5);

		(Collection->Parent)[5] = 0;
		//		GeometryCollectionAlgo::ParentTransform(Collection, 0, 1);

		FGeometryCollection* Coll = Collection.Get();

		TArray<FTransform> GlobalTransform;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransform);

		FGeometryCollectionProximityUtility ProximityUtility(Coll);
		ProximityUtility.UpdateProximity();

		// Breaking Data
		const TManagedArray<TSet<int32>>& Proximity = Collection->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);

		// Proximity = [(3,4,1), (0,4,5,2), (1,5), (0,4), (0,1,3,5), (1,2,4)]

		EXPECT_TRUE((Proximity)[0].Contains(3));
		EXPECT_TRUE((Proximity)[0].Contains(4));
		EXPECT_TRUE((Proximity)[0].Contains(1));
		EXPECT_FALSE((Proximity)[0].Contains(0));
		EXPECT_FALSE((Proximity)[0].Contains(2));
		EXPECT_FALSE((Proximity)[0].Contains(5));

		EXPECT_TRUE((Proximity)[1].Contains(0));
		EXPECT_TRUE((Proximity)[1].Contains(4));
		EXPECT_TRUE((Proximity)[1].Contains(5));
		EXPECT_TRUE((Proximity)[1].Contains(2));
		EXPECT_FALSE((Proximity)[1].Contains(1));
		EXPECT_FALSE((Proximity)[1].Contains(3));

		EXPECT_TRUE((Proximity)[2].Contains(1));
		EXPECT_TRUE((Proximity)[2].Contains(5));
		EXPECT_FALSE((Proximity)[2].Contains(0));
		EXPECT_FALSE((Proximity)[2].Contains(2));
		EXPECT_FALSE((Proximity)[2].Contains(3));
		EXPECT_FALSE((Proximity)[2].Contains(4));

		EXPECT_TRUE((Proximity)[3].Contains(0));
		EXPECT_TRUE((Proximity)[3].Contains(4));
		EXPECT_FALSE((Proximity)[3].Contains(1));
		EXPECT_FALSE((Proximity)[3].Contains(2));
		EXPECT_FALSE((Proximity)[3].Contains(3));
		EXPECT_FALSE((Proximity)[3].Contains(5));

		EXPECT_TRUE((Proximity)[4].Contains(0));
		EXPECT_TRUE((Proximity)[4].Contains(1));
		EXPECT_TRUE((Proximity)[4].Contains(3));
		EXPECT_TRUE((Proximity)[4].Contains(5));
		EXPECT_FALSE((Proximity)[4].Contains(2));
		EXPECT_FALSE((Proximity)[4].Contains(4));

		EXPECT_TRUE((Proximity)[5].Contains(1));
		EXPECT_TRUE((Proximity)[5].Contains(2));
		EXPECT_TRUE((Proximity)[5].Contains(4));
		EXPECT_FALSE((Proximity)[5].Contains(0));
		EXPECT_FALSE((Proximity)[5].Contains(3));
		EXPECT_FALSE((Proximity)[5].Contains(5));

		TArray<int32> DelList = { 0,1,2,3,4,5 };
		Coll->RemoveElements(FGeometryCollection::GeometryGroup, DelList);

		// Proximity = []

		EXPECT_EQ(Collection->NumElements(FGeometryCollection::GeometryGroup), 0);
	}

	void GeometrySwapFlat()
	{
		FGeometryCollection Coll;
		Coll.AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0,0,0))));
		Coll.AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(1,0,0))));
		Coll.AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(2,0,0))));
		Coll.AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(3,0,0))));
		
		//verts are ordered with geometry
		for (int box = 0; box < 4; ++box)
		{
			for (int i = 0; i < 8; ++i)
			{
				EXPECT_EQ(Coll.BoneMap[i + box * 8], box);
			}
		}

		// Bottom: Y = -1
		int32 ExpectedIndices[] = {
			5,1,0,
			0,4,5,
			2,3,7,
			7,6,2,
			3,2,0,
			0,1,3,
			4,6,7,
			7,5,4,
			0,2,6,
			6,4,0,
			7,3,1,
			1,5,7 };

		//faces are ordered with geometry and point to the right vertices
		for (int box = 0; box < 4; ++box)
		{
			for (int i = 0; i < 12; ++i)
			{
				for (int idx = 0; idx < 3; ++idx)
				{
					EXPECT_EQ(Coll.Indices[i + box * 12][idx], (ExpectedIndices[i * 3 + idx] + box * 8));
				}
			}
		}

		TArray<int32> NewOrder = { 0,3,2,1 };
		Coll.ReorderElements(FGeometryCollection::TransformGroup, NewOrder);
		//transforms change
		EXPECT_EQ(Coll.Transform[0].GetLocation().X, 0.f);
		EXPECT_EQ(Coll.Transform[1].GetLocation().X, 3.f);
		EXPECT_EQ(Coll.Transform[2].GetLocation().X, 2.f);
		EXPECT_EQ(Coll.Transform[3].GetLocation().X, 1.f);

		//groups swap to be contiguous with transform array
		EXPECT_EQ(Coll.TransformIndex[0], 0);
		EXPECT_EQ(Coll.TransformIndex[1], 1);
		EXPECT_EQ(Coll.TransformIndex[2], 2);
		EXPECT_EQ(Coll.TransformIndex[3], 3);

		//verts are still contiguous
		for (int box = 0; box < 4; ++box)
		{
			for (int i = 0; i < 8; ++i)
			{
				EXPECT_EQ(Coll.BoneMap[i + box * 8], box);
			}
		}

		//faces are reordered with geometry and point to the right vertices
		for (int box = 0; box < 4; ++box)
		{
			for (int i = 0; i < 12; ++i)
			{
				for (int idx = 0; idx < 3; ++idx)
				{
					//expect verts to reorder with faces so there's no indirection needed. The whole point is that we are contiguous
					EXPECT_EQ(Coll.Indices[i + box * 12][idx], (ExpectedIndices[i * 3 + idx] + box * 8));
				}
			}
		}
	}
	
	void TestFracturedGeometry()
	{
		FGeometryCollection* TestCollection = FGeometryCollection::NewGeometryCollection(FracturedGeometry::RawVertexArray,
																						 FracturedGeometry::RawIndicesArray,
																						 FracturedGeometry::RawBoneMapArray,
																						 FracturedGeometry::RawTransformArray,
																						 FracturedGeometry::RawLevelArray,
																						 FracturedGeometry::RawParentArray,
																						 FracturedGeometry::RawChildrenArray,
																						 FracturedGeometry::RawSimulationTypeArray,
																						 FracturedGeometry::RawStatusFlagsArray);

		EXPECT_EQ(TestCollection->NumElements(FGeometryCollection::GeometryGroup), 11);
	}

}

