// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestResponse.h"

namespace GeometryCollectionTest
{

	void ExampleResponse::ExpectTrue(bool Condition, FString Reason) 
	{
		ErrorFlag |= !Condition;
		if (!Condition)
		{
			Reasons.Add(Reason);
		}
	}
}