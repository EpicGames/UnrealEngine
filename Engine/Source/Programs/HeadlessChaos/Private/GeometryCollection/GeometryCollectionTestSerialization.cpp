// Copyright Epic Games, Inc. All Rights Reserved.
#include "GeometryCollection/GeometryCollectionTestSerialization.h"

#include "Chaos/Real.h"
#include "Chaos/SerializationTestUtility.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "Misc/Paths.h"

#include "gtest/gtest.h"

namespace GeometryCollectionTests
{
	FString GetSerializedBinaryPath()
	{
		return FPaths::EngineDir() / TEXT("Restricted/NotForLicensees/Source/Programs/GeometryCollectionTest/Resource/SerializedBinaries");
	}

	void GeometryCollectionSerialization()
	{
		FGeometryCollection GeometryCollection;
		GeometryCollection.AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::Identity, FVector(0.0f, 0.0f, 0.0f), FVector::OneVector)));
		GeometryCollection.AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::Identity, FVector(1.0f, 0.0f, 5.0f), FVector::OneVector)));
		GeometryCollection.AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::Identity, FVector(3.0f, 0.0f, 6.0f), FVector::OneVector)));
		GeometryCollection.AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::Identity, FVector(-1.0f, 0.0f, 6.0f), FVector::OneVector)));

		//  0
		//  ...1
		//  ......2,3
		(GeometryCollection.Parent)[0] = -1;
		(GeometryCollection.Children)[0].Add(1);
		(GeometryCollection.Parent)[1] = 0;
		(GeometryCollection.Children)[1].Add(2);
		(GeometryCollection.Children)[1].Add(3);
		(GeometryCollection.Parent)[2] = 1;
		(GeometryCollection.Parent)[3] = 1;


		TCHAR const *BinaryFolderName = TEXT("GeometryCollection");
		bool bSaveBinaryToDisk = false; // Flip to true and run to save current binary to disk for future tests.
		TArray<FGeometryCollection> ObjectsToTest;
		bool bResult = Chaos::SaveLoadUtility<Chaos::FReal, FGeometryCollection>(GeometryCollection, *GetSerializedBinaryPath(), BinaryFolderName, bSaveBinaryToDisk, ObjectsToTest);
		EXPECT_TRUE(bResult);

		for (FGeometryCollection const &TestCollection : ObjectsToTest)
		{
			EXPECT_EQ(TestCollection.Parent[0], -1);
			EXPECT_EQ(TestCollection.Parent[1], 0);
			EXPECT_EQ(TestCollection.Parent[2], 1);
			EXPECT_EQ(TestCollection.Parent[3], 1);
			EXPECT_EQ(TestCollection.Children[0].Num(), 1);
			EXPECT_EQ(TestCollection.Children[1].Num(), 2);
			EXPECT_EQ(TestCollection.Children[2].Num(), 0);
			EXPECT_EQ(TestCollection.Children[3].Num(), 0);
			EXPECT_EQ(TestCollection.Transform[0].GetTranslation(), FVector3f(0.0f, 0.0f, 0.0f));
			EXPECT_EQ(TestCollection.Transform[1].GetTranslation(), FVector3f(1.0f, 0.0f, 5.0f));
			EXPECT_EQ(TestCollection.Transform[2].GetTranslation(), FVector3f(3.0f, 0.0f, 6.0f));
			EXPECT_EQ(TestCollection.Transform[3].GetTranslation(), FVector3f(-1.0f, 0.0f, 6.0f));
		}
	}
}
