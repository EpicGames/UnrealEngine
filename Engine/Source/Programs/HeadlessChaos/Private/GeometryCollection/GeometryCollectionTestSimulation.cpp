// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestSimulation.h"
#include "GeometryCollection/GeometryCollectionTestUtility.h"
#include "GeometryCollection/GeometryCollectionTestFramework.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/TransformCollection.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "GeometryCollectionProxyData.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "PhysicsProxy/PhysicsProxies.h"
#include "Chaos/ErrorReporter.h"
#include "ChaosSolversModule.h"
#include "PBDRigidsSolver.h"
#include "HeadlessChaosTestUtility.h"

#define SMALL_THRESHOLD 1e-4
#define MEDIUM_THRESHOLD 1e-1

// #TODO Lots of duplication in here, anyone making solver or object changes
// has to go and fix up so many callsites here and they're all pretty much
// Identical. The similar code should be pulled out

namespace GeometryCollectionTest
{
	using namespace ChaosTest;
	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_SingleFallingUnderGravity)
	{
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init()->template As<FGeometryCollectionWrapper>();

		FFramework UnitTest; 
		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();
		UnitTest.Advance();

		{ // test results
			EXPECT_LT(FMath::Abs(Collection->RestCollection->Transform[0].GetTranslation().Z), SMALL_THRESHOLD); // rest never touched
			EXPECT_EQ(Collection->DynamicCollection->GetNumTransforms(), 1); // simulated is falling
			EXPECT_LT(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, 0.f);
			EXPECT_NEAR(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, -980.f * UnitTest.Dt * UnitTest.Dt, 1e-2);// we seem to be twice gravity
		}
	}

	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_SingleBodyCollidingWithGroundPlane)
	{
		FReal  Scale = 100.0f;
		CreationParameters Params; 
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box; 
		Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Box;
		FVector BoxScale(Scale); 
		Params.GeomTransform.SetScale3D(BoxScale); // Box dimensions
		Params.GeomTransform.SetLocation(0.99f * Scale * FVector::UpVector);	// Don't start too deep in penetration or the pushout is too aggressive
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();

		FFramework UnitTest;
		UnitTest.AddSimulationObject(Collection);
		UnitTest.AddSimulationObject(Floor);
		UnitTest.Initialize();
		for (int i = 0; i < 10; i++)
		{
			UnitTest.Advance();
		}

		{
			EXPECT_LT(FMath::Abs(Collection->RestCollection->Transform[0].GetTranslation().Z), SMALL_THRESHOLD);
			EXPECT_EQ(Collection->DynamicCollection->GetNumTransforms(), 1);
			EXPECT_LT(FMath::Abs(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z - 0.1f * Scale), MEDIUM_THRESHOLD * Scale);
		}
	}

	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_SingleSphereCollidingWithSolverFloor)
	{
		FVector Scale(0.5f);
		CreationParameters Params; 
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere; 
		Params.GeomTransform.SetScale3D(Scale); // Sphere radius
		FGeometryCollectionWrapper* Collection =
			TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		RigidBodyWrapper* Floor = 
			TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();

		FFramework UnitTest;
		UnitTest.AddSimulationObject(Collection);
		UnitTest.AddSimulationObject(Floor);
		UnitTest.Initialize();
		for (int i = 0; i < 10; i++)
		{
			UnitTest.Advance();
		}

		{ // test results
			EXPECT_LT(FMath::Abs(Collection->RestCollection->Transform[0].GetTranslation().Z), SMALL_THRESHOLD);
			EXPECT_EQ(Collection->DynamicCollection->GetNumTransforms(), 1);
			EXPECT_LT(FMath::Abs(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z) - Scale[0], SMALL_THRESHOLD);
		}
	}


	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_SingleCubeIntersectingWithSolverFloor)
	{
		FVector Scale(100.0f);
		CreationParameters Params; Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;  Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Box;
		Params.GeomTransform.SetScale3D(Scale); // Box size
		Params.GeomTransform.SetLocation(0.99f * Scale * FVector::UpVector);	// Don't start too deep in penetration or the pushout is too aggressive

		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();

		FFramework UnitTest;
		UnitTest.AddSimulationObject(Collection);
		UnitTest.AddSimulationObject(Floor);
		UnitTest.Initialize();
		for (int i = 0; i < 10; i++)
		{
			UnitTest.Advance();
		}

		{
			EXPECT_LT(FMath::Abs(Collection->RestCollection->Transform[0].GetTranslation().Z), SMALL_THRESHOLD);
			EXPECT_EQ(Collection->DynamicCollection->GetNumTransforms(), 1);
			EXPECT_LT(FMath::Abs(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z - 0.1f * Scale[0]), MEDIUM_THRESHOLD * Scale[0]);
		}
	}


	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_SingleKinematicBody)
	{
		CreationParameters Params; Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Kinematic;
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		FFramework UnitTest;
		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();
		for (int i = 0; i < 3; i++)
			UnitTest.Advance();

		{
			EXPECT_EQ(Collection->DynamicCollection->GetNumTransforms(), 1);
			//UE_LOGF(LogTest, Verbose, "Position : (%3.5f,%3.5f,%3.5f)", Transform[0].GetTranslation().X, Transform[0].GetTranslation().Y, Transform[0].GetTranslation().Z);
			EXPECT_EQ(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, 0.f);
			EXPECT_EQ(Collection->DynamicCollection->DynamicState[0], (int32)EObjectStateTypeEnum::Chaos_Object_Kinematic);
		}
	}


	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_SleepingDontMove)
	{
		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Sleeping;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		FReal InitialStartHeight = 5.0;
		Params.RootTransform.SetLocation(FVector(0.f, 0.f, InitialStartHeight));
		FGeometryCollectionWrapper* SleepingCollection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		FFramework UnitTest;
		UnitTest.AddSimulationObject(SleepingCollection);
		UnitTest.Initialize();

		const auto& Transform0 = SleepingCollection->DynamicCollection->GetTransform(0);
		for (int i = 0; i < 3; i++)
		{
			UnitTest.Advance();
			//UE_LOGF(LogTest, Verbose, "Position[0] : (%3.5f,%3.5f,%3.5f)", Transform0.GetTranslation().X, Transform0.GetTranslation().Y, Transform0.GetTranslation().Z);
		}

		{
			// particle doesn't fall due to sleeping state
			EXPECT_EQ(SleepingCollection->DynamicCollection->DynamicState[0], (int32)EObjectStateTypeEnum::Chaos_Object_Sleeping);
			EXPECT_LT(FMath::Abs(SleepingCollection->DynamicCollection->GetTransform(0).GetTranslation().Z - InitialStartHeight), SMALL_THRESHOLD);
		}

	}


	
	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_SleepingActivation)
	{
		CreationParameters Params; 
		Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Box;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;

		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.RootTransform.SetLocation(FVector(0.f, 0.f, 15.f));
		FGeometryCollectionWrapper* MovingCollection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		FReal InitialStartHeight = 5.0;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Sleeping;
		Params.RootTransform.SetLocation(FVector(0.f, 0.f, InitialStartHeight));
		FGeometryCollectionWrapper* SleepingCollection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		FFramework UnitTest;
		UnitTest.AddSimulationObject(SleepingCollection);
		UnitTest.AddSimulationObject(MovingCollection);
		UnitTest.Initialize();

		for (int i = 0; i < 15; i++)
		{
			UnitTest.Advance();

			//const auto& Transform0 = MovingCollection->DynamicCollection->GetTransform(0);
			//const auto& Transform1 = SleepingCollection->DynamicCollection->GetTransform(0);
			//UE_LOGF(LogTest, Verbose, "Position[0] : (%3.5f,%3.5f,%3.5f)", Transform0.GetTranslation().X, Transform0.GetTranslation().Y, Transform0.GetTranslation().Z);
			//UE_LOGF(LogTest, Verbose, "Position[1] : (%3.5f,%3.5f,%3.5f)", Transform1.GetTranslation().X, Transform1.GetTranslation().Y, Transform1.GetTranslation().Z);
		}

		{
			// Is now dynamic and has moved from initial position
			EXPECT_EQ(SleepingCollection->DynamicCollection->DynamicState[0], (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic);
			EXPECT_LT(MovingCollection->DynamicCollection->GetTransform(0).GetTranslation().Z, InitialStartHeight - 2.0f);
		}

	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Enabling)
	{
		CreationParameters Params;
		Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Box;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;

		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.RootTransform.SetLocation(FVector(0.f, 0.f, 15.f));
		FGeometryCollectionWrapper* MovingCollection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		FFramework UnitTest;
		UnitTest.AddSimulationObject(MovingCollection);
		UnitTest.Initialize();

		for (int i = 0; i < 5; i++)
		{
			UnitTest.Advance();
			EXPECT_EQ(MovingCollection->DynamicCollection->DynamicState[0], (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic);
			EXPECT_LT(MovingCollection->DynamicCollection->GetTransform(0).GetTranslation().Z, 15.0f);
		}
		// Disabled particle
		MovingCollection->PhysObject->DisableParticles_External({ 0 });
		FReal CurrentPosition = MovingCollection->DynamicCollection->GetTransform(0).GetTranslation().Z;
		for (int i = 0; i < 5; i++)
		{
			UnitTest.Advance();
			EXPECT_EQ(MovingCollection->DynamicCollection->DynamicState[0], (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic);
			EXPECT_EQ(MovingCollection->DynamicCollection->GetTransform(0).GetTranslation().Z, CurrentPosition);
		}
	}


	// CollisionGroup == 0 : Collide With Everything Except CollisionGroup=-1
	// CollisionGroup == -1 : Collide With Nothing Including CollisionGroup=0
	// CollisionGroup_A == CollisionGroup_B : Collide With Each Other
	// CollisionGroup_A != CollisionGroup_B : Don't Collide With Each Other
	// @todo(chaos): this test does not work with levelsets because the do not support manifolds
	// and therefore do not stack.
	GTEST_TEST(AllTraits, DISABLED_GeometryCollection_RigidBodies_CollisionGroup)
	{

		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection;
		RestCollection = GeometryCollection::MakeCubeElement(FTransform(FVector(0.f, 0.f, 210.f)), FVector(100.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0.f, 0.f, 320.f)), FVector(100.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0.f, 0.f, 430.f)), FVector(100.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0.f, 0.f, 540.f)), FVector(100.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FVector(0.f, 0.f, 650.f)), FVector(100.0)));

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		// I think there is suppose to be one more input param, but not sure what it is for...

		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();
		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();
		UnitTest.AddSimulationObject(Collection);
		UnitTest.AddSimulationObject(Floor);
		UnitTest.Initialize();
		UnitTest.Advance();

		// testing...
		for (int Frame = 1; Frame < 200; Frame++)
		{
			if (Frame == 1)
			{
				// Object 0 collides with everything except Object 4
				// Objects 1,2 collide with each other and Object 0 plus the ground
				// Object 3 collides with Object 0 plus the ground
				// Object 4 collides with nothing
				// We should end up with 2 stacks on the ground (0,1,2), (0,3) and one free-falling object (4)
				Collection->PhysObject->GetSolverClusterHandle_Internal(0)->SetCollisionGroup(0);
				Collection->PhysObject->GetSolverClusterHandle_Internal(1)->SetCollisionGroup(1);
				Collection->PhysObject->GetSolverClusterHandle_Internal(2)->SetCollisionGroup(1);
				Collection->PhysObject->GetSolverClusterHandle_Internal(3)->SetCollisionGroup(3);
				Collection->PhysObject->GetSolverClusterHandle_Internal(4)->SetCollisionGroup(-1);

				EXPECT_TRUE(Collection->DynamicCollection->GetTransform(0).GetRotation() == FQuat4f::Identity); // Can use defaulted zero rotation to indicate that the
				EXPECT_TRUE(Collection->DynamicCollection->GetTransform(1).GetRotation() == FQuat4f::Identity); // rigid has not been affected. Should we though??
				EXPECT_TRUE(Collection->DynamicCollection->GetTransform(2).GetRotation() == FQuat4f::Identity);
				EXPECT_TRUE(Collection->DynamicCollection->GetTransform(3).GetRotation() == FQuat4f::Identity);
				EXPECT_TRUE(Collection->DynamicCollection->GetTransform(4).GetRotation() == FQuat4f::Identity);
			}

			if (Frame == 100)
			{
				EXPECT_NEAR(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, 50.0f, 1.0f);
				EXPECT_NEAR(Collection->DynamicCollection->GetTransform(1).GetTranslation().Z, 150.0f, 1.0f);
				EXPECT_NEAR(Collection->DynamicCollection->GetTransform(2).GetTranslation().Z, 250.0f, 1.0f);
				EXPECT_FALSE(Collection->DynamicCollection->GetTransform(0).GetRotation() == FQuat4f::Identity);
				EXPECT_FALSE(Collection->DynamicCollection->GetTransform(1).GetRotation() == FQuat4f::Identity);
				EXPECT_FALSE(Collection->DynamicCollection->GetTransform(2).GetRotation() == FQuat4f::Identity);
				EXPECT_FALSE(Collection->DynamicCollection->GetTransform(3).GetRotation() == FQuat4f::Identity);
				EXPECT_TRUE(Collection->DynamicCollection->GetTransform(4).GetRotation() == FQuat4f::Identity);
			}
			UnitTest.Advance();
		}

		EXPECT_NEAR(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, 50.0f, 1.0f);
		EXPECT_NEAR(Collection->DynamicCollection->GetTransform(1).GetTranslation().Z, 150.0f, 1.0f);
		EXPECT_NEAR(Collection->DynamicCollection->GetTransform(2).GetTranslation().Z, 250.0f, 1.0f);
		EXPECT_NEAR(Collection->DynamicCollection->GetTransform(3).GetTranslation().Z, 150.0f, 1.0f);
		EXPECT_FALSE(Collection->DynamicCollection->GetTransform(3).GetRotation() == FQuat4f::Identity);
		EXPECT_TRUE(Collection->DynamicCollection->GetTransform(4).GetRotation() == FQuat4f::Identity); // Phased through everything, good.
		EXPECT_LT(Collection->DynamicCollection->GetTransform(4).GetTranslation().Z, -100.0f);

		Collection->PhysObject->GetSolverClusterHandle_Internal(0)->SetCollisionGroup(-1);
		for (int i = 0; i < 50; i++) { UnitTest.Advance(); }
		EXPECT_LT(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, -100.0f);

	}



	
	GTEST_TEST(AllTraits, GeometryCollection_TestImplicitCollisionGeometry)
	{
		typedef Chaos::FVec3 Vec;

		CreationParameters Params; 
		Params.SimplicialType = ESimplicialType::Chaos_Simplicial_GriddleBox;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_LevelSet;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;

		FGeometryCollectionWrapper* Collection =
			TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(
				Params)->template As<FGeometryCollectionWrapper>();

		const TManagedArray<TUniquePtr<Chaos::FBVHParticles>>& Simplicials =
			Collection->RestCollection->template GetAttribute<TUniquePtr<Chaos::FBVHParticles>>(
				FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup);
		EXPECT_EQ(Simplicials.Num(), 1);
		const Chaos::FBVHParticles& Simplicial = *Simplicials[0];

		const TManagedArray<Chaos::FImplicitObjectPtr>& Implicits = 
			Collection->RestCollection->template GetAttribute<Chaos::FImplicitObjectPtr>(
				FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
		EXPECT_EQ(Implicits.Num(), 1);
		check(Implicits[0]);
		const Chaos::FImplicitObject& Implicit = *Implicits[0];

		// Ensure all simplicial particles are on the surface of the implicit shape.
		check(Implicit.GetType() == Chaos::ImplicitObjectType::LevelSet);
		const Chaos::FLevelSet* LevelSet = static_cast<const Chaos::FLevelSet*>(&Implicit);
		const FReal DxSize = LevelSet->GetGrid().Dx().Size();

		FReal MinX = TNumericLimits<FReal>::Max();
		FReal MinY = TNumericLimits<FReal>::Max();
		FReal MinZ = TNumericLimits<FReal>::Max();

		FReal MaxX = -TNumericLimits<FReal>::Max();
		FReal MaxY = -TNumericLimits<FReal>::Max();
		FReal MaxZ = -TNumericLimits<FReal>::Max();
		for (uint32 Idx = 0; Idx < Simplicial.Size(); ++Idx)
		{
			const FReal phi = Implicit.SignedDistance(Simplicial.GetX(Idx));
			EXPECT_LT(FMath::Abs(phi), DxSize);
			//EXPECT_LT(FMath::Abs(phi), 0.01f);

			const auto& Pos = Simplicial.GetX(Idx);
			MinX = MinX < Pos[0] ? MinX : Pos[0];
			MinY = MinY < Pos[1] ? MinY : Pos[1];
			MinZ = MinZ < Pos[2] ? MinZ : Pos[2];

			MaxX = MaxX > Pos[0] ? MaxX : Pos[0];
			MaxY = MaxY > Pos[1] ? MaxY : Pos[1];
			MaxZ = MaxZ > Pos[2] ? MaxZ : Pos[2];
		}

		// Make sure the geometry occupies a volume.
		check(MinX < MaxX);
		check(MinY < MaxY);
		check(MinZ < MaxZ);

		// Cast a ray through the level set, and make sure it's as we expect.
		for(FReal x = 2*MinX; x < 2*MaxX; x += (MaxX-MinX)/10)
		{
			Vec Normal;
			const FReal phi = Implicit.PhiWithNormal(Vec(x, 0, 0), Normal);
			if (x < MinX || MaxX < x)
			{
				check(phi >= -0.01f);
				EXPECT_GT(phi, -0.01f);
			}
			else
			{
				check(phi <= 0.01f);
				EXPECT_LT(phi, 0.01f);
			}

			if (x < MinX/4)
			{
				EXPECT_LT((Normal-Vec(-1,0,0)).Size(), KINDA_SMALL_NUMBER);
			}
			else if (x > MaxX/4)
			{
				EXPECT_LT((Normal - Vec(1, 0, 0)).Size(), KINDA_SMALL_NUMBER);
			}
		}
	}

}

