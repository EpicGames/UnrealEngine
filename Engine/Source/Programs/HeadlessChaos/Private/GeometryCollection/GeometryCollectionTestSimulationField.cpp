// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestSimulationField.h"
#include "GeometryCollection/GeometryCollectionTestUtility.h"
#include "GeometryCollection/GeometryCollectionTestFramework.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/TransformCollection.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "Field/FieldSystem.h"
#include "Field/FieldSystemNodes.h"
#include "GeometryCollectionProxyData.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "PhysicsProxy/PhysicsProxies.h"
#include "Chaos/ErrorReporter.h"
#include "Chaos/PBDRigidClustering.h"
#include "PBDRigidsSolver.h"
#include "ChaosSolversModule.h"
#include "HeadlessChaosTestUtility.h"

//#include "GeometryCollection/GeometryCollectionAlgo.h"

#define SMALL_THRESHOLD 1e-4

// #TODO Lots of duplication in here, anyone making solver or object changes
// has to go and fix up so many callsites here and they're all pretty much
// Identical. The similar code should be pulled out

namespace GeometryCollectionTest
{
	using namespace ChaosTest;

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_KinematicActivationOnProxyDuringInit)
	{
		const FVector Translation0(0, 0, 1);

		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Kinematic;
		Params.RootTransform.SetLocation(Translation0);
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		Params.RootTransform.SetLocation(FVector(100, 0, 0));
		FGeometryCollectionWrapper* CollectionOther = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		FFramework UnitTest;
		UnitTest.AddSimulationObject(CollectionOther);
		UnitTest.AddSimulationObject(Collection);

		FRadialIntMask* RadialMaskTmp = new FRadialIntMask();
		RadialMaskTmp->Position = FVector(0.0, 0.0, 0.0);
		RadialMaskTmp->Radius = 100.0;
		RadialMaskTmp->InteriorValue = (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic;
		RadialMaskTmp->ExteriorValue = (int32)EObjectStateTypeEnum::Chaos_Object_Kinematic;
		RadialMaskTmp->SetMaskCondition = ESetMaskConditionType::Field_Set_IFF_NOT_Interior;
		FName TargetNameTmp = GetFieldPhysicsName(EFieldPhysicsType::Field_DynamicState);
		Collection->PhysObject->BufferFieldCommand_Internal(UnitTest.Solver, { TargetNameTmp, RadialMaskTmp });

		UnitTest.Initialize();
		UnitTest.Advance();

		UnitTest.Solver->RegisterSimOneShotCallback([&]()
			{
				EXPECT_EQ(CollectionOther->DynamicCollection->DynamicState[0], (int32)EObjectStateTypeEnum::Chaos_Object_Kinematic);
				EXPECT_EQ(Collection->DynamicCollection->DynamicState[0], (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic);
			});
	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_KinematicActivationOnProxyDuringUpdate)
	{
		const FVector Translation0(0, 0, 1);

		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Kinematic;
		Params.RootTransform.SetLocation(Translation0);
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		FFramework UnitTest; UnitTest.Dt = 1 / 24.0;
		UnitTest.AddSimulationObject(Collection);

		UnitTest.Initialize();

		{
			UnitTest.Advance();
		}

		TManagedArray<uint8>& DynamicState = Collection->DynamicCollection->DynamicState;
		EXPECT_EQ(DynamicState[0], (int32)EObjectStateTypeEnum::Chaos_Object_Kinematic);

		// simulated
		EXPECT_EQ(Collection->DynamicCollection->GetNumTransforms(), 1);
		const FVector Translation1 = FVector(Collection->DynamicCollection->GetTransform(0).GetTranslation());
		EXPECT_NEAR((Translation0 - Translation1).Size(), 0.f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, 1.f, KINDA_SMALL_NUMBER);

		FRadialIntMask* RadialMask = new FRadialIntMask();
		RadialMask->Position = FVector(0.0, 0.0, 0.0);
		RadialMask->Radius = 100.0;
		RadialMask->InteriorValue = (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic;
		RadialMask->ExteriorValue = (int32)EObjectStateTypeEnum::Chaos_Object_Kinematic;
		RadialMask->SetMaskCondition = ESetMaskConditionType::Field_Set_IFF_NOT_Interior;
		FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_DynamicState);
		Collection->PhysObject->BufferFieldCommand_Internal(UnitTest.Solver, { TargetName, RadialMask });

		{
			UnitTest.Advance();
		}
		EXPECT_EQ(DynamicState[0], (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic);

		const FVector Translation2 = FVector(Collection->DynamicCollection->GetTransform(0).GetTranslation());
		EXPECT_NE(Translation1, Translation2);
		EXPECT_LE(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, 0.f);
	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_KinematicActivation)
	{
		const FVector Translation0(0, 0, 1);

		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Kinematic;
		Params.RootTransform.SetLocation(Translation0);
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		FFramework UnitTest;
		UnitTest.AddSimulationObject(Collection);

		UnitTest.Initialize();

		for (int i = 0; i < 100; i++)
		{
			UnitTest.Advance();
		}

		// simulated
		EXPECT_EQ(Collection->DynamicCollection->GetNumTransforms(), 1);
		const FVector Translation1 = FVector(Collection->DynamicCollection->GetTransform(0).GetTranslation());
		EXPECT_NEAR((Translation0 - Translation1).Size(), 0.f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, 1.f, KINDA_SMALL_NUMBER);

		FRadialIntMask* RadialMask = new FRadialIntMask();
		RadialMask->Position = FVector(0.0, 0.0, 0.0);
		RadialMask->Radius = 100.0;
		RadialMask->InteriorValue = (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic;
		RadialMask->ExteriorValue = (int32)EObjectStateTypeEnum::Chaos_Object_Kinematic;
		RadialMask->SetMaskCondition = ESetMaskConditionType::Field_Set_IFF_NOT_Interior;
		FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_DynamicState);
		UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, RadialMask });

		for (int i = 0; i < 100; i++)
		{
			UnitTest.Advance();
		}

		const FVector Translation2 = FVector(Collection->DynamicCollection->GetTransform(0).GetTranslation());
		EXPECT_NE(Translation1, Translation2);
		EXPECT_LE(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, 0.f);
	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_InitialLinearVelocity)
	{
		FFramework UnitTest;

		// Physics Object Setup
		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Kinematic;
		Params.RootTransform.SetLocation(FVector(0.0, 0.0, 0.0));

		Params.InitialVelocityType = EInitialVelocityTypeEnum::Chaos_Initial_Velocity_User_Defined;
		Params.InitialLinearVelocity = FVector3f(0.f, 100.f, 0.f);
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);

		// Field setup
		FRadialIntMask* RadialMask = new FRadialIntMask();
		RadialMask->Position = FVector(0.0, 0.0, 0.0);
		RadialMask->Radius = 5.0f;
		RadialMask->InteriorValue = (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic;
		RadialMask->ExteriorValue = (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic;
		RadialMask->SetMaskCondition = ESetMaskConditionType::Field_Set_Always;

		UnitTest.Initialize();

		TManagedArray<uint8>& DynamicState = Collection->DynamicCollection->DynamicState;

		FReal PreviousY = 0.f;
		EXPECT_EQ(Collection->DynamicCollection->GetTransform(0).GetTranslation().X, 0);
		EXPECT_EQ(Collection->DynamicCollection->GetTransform(0).GetTranslation().Y, 0);

		for (int Frame = 0; Frame < 10; Frame++)
		{
			UnitTest.Advance();

			if (Frame == 1)
			{
				FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_DynamicState);
				Collection->PhysObject->BufferFieldCommand_Internal(UnitTest.Solver, { TargetName, RadialMask });
			}

			if (Frame >= 2)
			{
				EXPECT_EQ(DynamicState[0], (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic);
				EXPECT_EQ(Collection->DynamicCollection->GetTransform(0).GetTranslation().X, 0);
				EXPECT_GT(Collection->DynamicCollection->GetTransform(0).GetTranslation().Y, PreviousY);
				EXPECT_LT(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, 0);
			}
			else
			{
				EXPECT_EQ(DynamicState[0], (int32)EObjectStateTypeEnum::Chaos_Object_Kinematic);
				EXPECT_EQ(Collection->DynamicCollection->GetTransform(0).GetTranslation().X, 0);
				EXPECT_EQ(Collection->DynamicCollection->GetTransform(0).GetTranslation().Y, 0);
				EXPECT_EQ(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, 0);
			}
			PreviousY = Collection->DynamicCollection->GetTransform(0).GetTranslation().Y;
		}
	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_StayDynamic)
	{
		FFramework UnitTest;
		FReal PreviousHeight = 5.0;

		// Physics Object Setup
		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Static;
		Params.RootTransform.SetLocation(FVector(0.f, 0.f, PreviousHeight));
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);

		// Field setup
		FRadialIntMask* RadialMask = new FRadialIntMask();
		RadialMask->Position = FVector(0.0, 0.0, PreviousHeight);
		RadialMask->Radius = 5.0;
		RadialMask->InteriorValue = (int32)EObjectStateTypeEnum::Chaos_Object_Dynamic;
		RadialMask->ExteriorValue = (int32)EObjectStateTypeEnum::Chaos_Object_Kinematic;
		RadialMask->SetMaskCondition = ESetMaskConditionType::Field_Set_IFF_NOT_Interior;

		UnitTest.Initialize();

		for (int Frame = 0; Frame < 10; Frame++)
		{
			// Set everything inside the r=5.0 sphere to dynamic
			if (Frame == 5)
			{
				FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_DynamicState);
				UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, RadialMask });
			}

			UnitTest.Advance();

			if (Frame < 5)
			{
				// Before frame 5 nothing should have moved
				EXPECT_LT(FMath::Abs(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z - 5.f), SMALL_THRESHOLD);
			}
			else
			{
				// Frame 5 and after should be falling
				EXPECT_LT(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, PreviousHeight);
			}

			// Track current height of the object
			PreviousHeight = Collection->DynamicCollection->GetTransform(0).GetTranslation().Z;
		}

	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_LinearForce)
	{
		FFramework UnitTest;

		// Physics Object Setup
		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.RootTransform.SetLocation(FVector(0.f, 0.f, 5.f));
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);

		// Field setup
		FUniformVector* UniformVector = new FUniformVector();
		UniformVector->Direction = FVector(0.0, 1.0, 0.0);
		UniformVector->Magnitude = 1000.0;

		UnitTest.Initialize();

		FReal PreviousY = 0.0;
		for (int Frame = 0; Frame < 10; Frame++)
		{
			if (Frame >= 5)
			{
				FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_LinearForce);
				UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, UniformVector->NewCopy() });
			}

			UnitTest.Advance();

			if (Frame < 5)
			{
				EXPECT_LT(FMath::Abs(Collection->DynamicCollection->GetTransform(0).GetTranslation().Y), SMALL_THRESHOLD);
			}
			else
			{
				EXPECT_GT(Collection->DynamicCollection->GetTransform(0).GetTranslation().Y, PreviousY);
			}

			PreviousY = Collection->DynamicCollection->GetTransform(0).GetTranslation().Y;

		}

		delete UniformVector;
	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_Torque)
	{
		FFramework UnitTest;

		// Physics Object Setup
		FVector Scale = FVector(10.0);
		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.RootTransform.SetLocation(FVector(0.f, 0.f, 5.f));
		Params.GeomTransform.SetScale3D(Scale);
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);

		// Field setup
		FUniformVector* UniformVector = new FUniformVector();
		UniformVector->Direction = FVector(0.0, 1.0, 0.0);
		UniformVector->Magnitude = 100.0;

		UnitTest.Initialize();

		FReal PreviousY = 0.0;
		for (int Frame = 0; Frame < 10; Frame++)
		{
			if (Frame >= 5)
			{
				FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_AngularTorque);
				UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, UniformVector->NewCopy() });
			}

			UnitTest.Advance();

			auto& Particles = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles();
			if (Frame < 5)
			{
				EXPECT_LT(FMath::Abs(Collection->DynamicCollection->GetTransform(0).GetRotation().Euler().Y), SMALL_THRESHOLD);
			}
			else
			{
				EXPECT_NE(FMath::Abs(Collection->DynamicCollection->GetTransform(0).GetRotation().Euler().Y), SMALL_THRESHOLD);
				EXPECT_GT(Particles.GetW(0).Y, PreviousY);
			}

			PreviousY = Particles.GetW(0).Y;
		}

	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_Kill)
	{
		FFramework UnitTest;

		// Physics Object Setup
		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.RootTransform.SetLocation(FVector(0.f, 0.f, 20.f));
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);

		// Field setup
		FPlaneFalloff* FalloffField = new FPlaneFalloff();
		FalloffField->Magnitude = 1.0;
		FalloffField->Distance = 10.0f;
		FalloffField->Position = FVector(0.0, 0.0, 5.0);
		FalloffField->Normal = FVector(0.0, 0.0, 1.0);
		FalloffField->Falloff = EFieldFalloffType::Field_Falloff_Linear;

		UnitTest.Initialize();

		
		TManagedArray<bool>& Active = Collection->DynamicCollection->Active;
		auto& Particles = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles();
		for (int Frame = 0; Frame < 20; Frame++)
		{
			FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_Kill);
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, FalloffField->NewCopy() });

			UnitTest.Advance();

			if (Particles.Disabled(0))
			{
				break;
			}
		}

		EXPECT_EQ(Particles.Disabled(0), true);

		// hasn't fallen any further than this due to being disabled
		EXPECT_LT(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, 5.f);
		EXPECT_GT(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, -5.0f);
	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_LinearVelocity)
	{
		FFramework UnitTest;

		// Physics Object Setup
		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.RootTransform.SetLocation(FVector(0.f, 0.f, 20.f));
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);

		// Field setup
		FUniformVector* VectorField = new FUniformVector();
		VectorField->Magnitude = 100.0;
		VectorField->Direction = FVector(1.0, 0.0, 0.0);

		UnitTest.Initialize();

		FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_LinearVelocity);
		UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, VectorField->NewCopy() });
		UnitTest.Advance();

		FReal PreviousX = 0.0;
		
		for (int Frame = 1; Frame < 10; Frame++)
		{
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ GetFieldPhysicsName(EFieldPhysicsType::Field_LinearVelocity), VectorField->NewCopy() });

			UnitTest.Advance();

			EXPECT_GT(Collection->DynamicCollection->GetTransform(0).GetTranslation().X, PreviousX);
			PreviousX = Collection->DynamicCollection->GetTransform(0).GetTranslation().X;
		}
	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_CollisionGroup)
	{
		/**
		 * Create a stack of boxes on the ground and verify that we we change their collision
		 * group, they drop through the ground.
		 */

		FFramework UnitTest; UnitTest.Dt = 1 / 24.0;

		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();
		UnitTest.AddSimulationObject(Floor);

		// Generate Geometry - a stack of boxes.
		// The bottom box is on the ground, and the others are dropped into it.
		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		FVector Scale = FVector(100);
		Params.GeomTransform.SetScale3D(Scale);

		FGeometryCollectionWrapper* Collection[4];
		for (int n = 0; n < 3; n++)
		{
			Params.RootTransform.SetLocation(FVector(0.f, 0.f, n * 200.0f + 100.0f));
			Params.CollisionType = ECollisionTypeEnum::Chaos_Volumetric;
			Collection[n + 1] = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
			UnitTest.AddSimulationObject(Collection[n + 1]);
		}

		// Field setup
		FRadialIntMask* RadialMask = new FRadialIntMask();
		RadialMask->Position = FVector(0.0, 0.0, 0.0);
		RadialMask->Radius = 0;
		RadialMask->InteriorValue = -1;
		RadialMask->ExteriorValue = -1;
		RadialMask->SetMaskCondition = ESetMaskConditionType::Field_Set_Always;

		UnitTest.Initialize();

		for (int Frame = 0; Frame < 60; Frame++)
		{
			UnitTest.Advance();

			auto& Particles = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles();
			if (Frame == 30)
			{
				// The boxes should have landed on each other and settled by now
				EXPECT_NEAR(Collection[1]->DynamicCollection->GetTransform(0).GetTranslation().Z, (FReal)100, (FReal)20);
				EXPECT_NEAR(Collection[2]->DynamicCollection->GetTransform(0).GetTranslation().Z, (FReal)300, (FReal)20);
				EXPECT_NEAR(Collection[3]->DynamicCollection->GetTransform(0).GetTranslation().Z, (FReal)500, (FReal)20);
			}
			if (Frame == 31)
			{
				EFieldPhysicsType PhysicsType = GetGeometryCollectionPhysicsType(EGeometryCollectionPhysicsTypeEnum::Chaos_CollisionGroup);
				Collection[1]->PhysObject->BufferFieldCommand_Internal(UnitTest.Solver, { PhysicsType, RadialMask });
			}
		}
		// The bottom boxes should have fallen below the ground level, box 2 now on the ground with box 3 on top
		auto& Particles = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles();
		EXPECT_LT(Collection[1]->DynamicCollection->GetTransform(0).GetTranslation().Z, 0);
		EXPECT_TRUE(FMath::IsNearlyEqual((FReal)Collection[2]->DynamicCollection->GetTransform(0).GetTranslation().Z, (FReal)100, (FReal)20));
		EXPECT_TRUE(FMath::IsNearlyEqual((FReal)Collection[3]->DynamicCollection->GetTransform(0).GetTranslation().Z, (FReal)300, (FReal)20));

	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_ClusterBreak_StrainModel_Test1)
	{
		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody_TwoByTwo_ThreeTransform(FVector(0));

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 1.0 };
		Params.MaxClusterLevel = 1000;
		Params.ClusterConnectionMethod = Chaos::FClusterCreationParameters::EConnectionMethod::DelaunayTriangulation;
		Params.ClusterGroupIndex = 0;

		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);

		FRadialFalloff* FalloffField = new FRadialFalloff();
		FalloffField->Magnitude = 1.5;
		FalloffField->Radius = 100.0;
		FalloffField->Position = FVector(0.0, 0.0, 0.0);
		FalloffField->Falloff = EFieldFalloffType::Field_FallOff_None;

		UnitTest.Initialize();

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();

		UnitTest.Advance();

		{
			FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_ExternalClusterStrain);
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, FalloffField->NewCopy() });

			EXPECT_EQ(ClusterMap.Num(), 3);
			EXPECT_EQ(ClusterMap[Collection->PhysObject->GetParticle_Internal(4)].Num(), 2);
			EXPECT_TRUE(ClusterMap[Collection->PhysObject->GetParticle_Internal(4)].Contains(Collection->PhysObject->GetParticle_Internal(0)));
			EXPECT_TRUE(ClusterMap[Collection->PhysObject->GetParticle_Internal(4)].Contains(Collection->PhysObject->GetParticle_Internal(1)));
			EXPECT_EQ(ClusterMap[Collection->PhysObject->GetParticle_Internal(5)].Num(), 2);
			EXPECT_TRUE(ClusterMap[Collection->PhysObject->GetParticle_Internal(5)].Contains(Collection->PhysObject->GetParticle_Internal(2)));
			EXPECT_TRUE(ClusterMap[Collection->PhysObject->GetParticle_Internal(5)].Contains(Collection->PhysObject->GetParticle_Internal(3)));
			EXPECT_EQ(ClusterMap[Collection->PhysObject->GetParticle_Internal(6)].Num(), 2);
			EXPECT_TRUE(ClusterMap[Collection->PhysObject->GetParticle_Internal(6)].Contains(Collection->PhysObject->GetParticle_Internal(5)));
			EXPECT_TRUE(ClusterMap[Collection->PhysObject->GetParticle_Internal(6)].Contains(Collection->PhysObject->GetParticle_Internal(4)));

			EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(0)->Disabled());
			EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(1)->Disabled());
			EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(2)->Disabled());
			EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(3)->Disabled());
			EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(4)->Disabled());
			EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(5)->Disabled());
			EXPECT_FALSE(Collection->PhysObject->GetParticle_Internal(6)->Disabled());

			UnitTest.Advance();

			// todo: indices here might seem odd, particles 4 & 5 are swapped
			EXPECT_EQ(ClusterMap.Num(), 2);
			EXPECT_EQ(ClusterMap[Collection->PhysObject->GetParticle_Internal(4)].Num(), 2);
			EXPECT_TRUE(ClusterMap[Collection->PhysObject->GetParticle_Internal(4)].Contains(Collection->PhysObject->GetParticle_Internal(0)));
			EXPECT_TRUE(ClusterMap[Collection->PhysObject->GetParticle_Internal(4)].Contains(Collection->PhysObject->GetParticle_Internal(1)));
			EXPECT_EQ(ClusterMap[Collection->PhysObject->GetParticle_Internal(5)].Num(), 2);
			EXPECT_TRUE(ClusterMap[Collection->PhysObject->GetParticle_Internal(5)].Contains(Collection->PhysObject->GetParticle_Internal(2)));
			EXPECT_TRUE(ClusterMap[Collection->PhysObject->GetParticle_Internal(5)].Contains(Collection->PhysObject->GetParticle_Internal(3)));

			EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(0)->Disabled());
			EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(1)->Disabled());
			EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(2)->Disabled());
			EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(3)->Disabled());
			EXPECT_FALSE(Collection->PhysObject->GetParticle_Internal(4)->Disabled());
			EXPECT_FALSE(Collection->PhysObject->GetParticle_Internal(5)->Disabled());
			EXPECT_TRUE(Collection->PhysObject->GetParticle_Internal(6)->Disabled());
		}

		delete FalloffField;
	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_ClusterBreak_StrainModel_Test2)
	{
		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody_ThreeByTwo_ThreeTransform(FVector(0));

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 1.0 };
		Params.MaxClusterLevel = 1000;
		Params.ClusterGroupIndex = 0;

		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);

		FRadialFalloff* FalloffField = new FRadialFalloff();
		FalloffField->Magnitude = 1.5f;
		FalloffField->Radius = 200.0f;
		FalloffField->Position = FVector(0.0, 0.0, 0.0);
		FalloffField->Falloff = EFieldFalloffType::Field_FallOff_None;

		UnitTest.Initialize();
		UnitTest.Advance();

		auto ParticleHandles = [&Collection](int32 Index) -> Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*
		{
			return Collection->PhysObject->GetParticle_Internal(Index);
		};

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();

		{
			FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_ExternalClusterStrain);
			FFieldSystemCommand Command(TargetName, FalloffField->NewCopy());
			FFieldSystemMetaDataProcessingResolution* ResolutionData = new FFieldSystemMetaDataProcessingResolution(EFieldResolutionType::Field_Resolution_Maximum);
			Command.MetaData.Add(FFieldSystemMetaData::EMetaType::ECommandData_ProcessingResolution, TUniquePtr< FFieldSystemMetaDataProcessingResolution >(ResolutionData));
			UnitTest.Solver->GetPerSolverField().AddTransientCommand(Command);

			EXPECT_EQ(ClusterMap.Num(), 3);
			EXPECT_EQ(ClusterMap[ParticleHandles(6)].Num(), 3);
			EXPECT_TRUE(ClusterMap[ParticleHandles(6)].Contains(ParticleHandles(0)));
			EXPECT_TRUE(ClusterMap[ParticleHandles(6)].Contains(ParticleHandles(1)));
			EXPECT_TRUE(ClusterMap[ParticleHandles(6)].Contains(ParticleHandles(2)));
			EXPECT_EQ(ClusterMap[ParticleHandles(7)].Num(), 3);
			EXPECT_TRUE(ClusterMap[ParticleHandles(7)].Contains(ParticleHandles(3)));
			EXPECT_TRUE(ClusterMap[ParticleHandles(7)].Contains(ParticleHandles(4)));
			EXPECT_TRUE(ClusterMap[ParticleHandles(7)].Contains(ParticleHandles(5)));
			EXPECT_EQ(ClusterMap[ParticleHandles(8)].Num(), 2);
			EXPECT_TRUE(ClusterMap[ParticleHandles(8)].Contains(ParticleHandles(7)));
			EXPECT_TRUE(ClusterMap[ParticleHandles(8)].Contains(ParticleHandles(6)));

			EXPECT_TRUE(ParticleHandles(0)->Disabled());
			EXPECT_TRUE(ParticleHandles(1)->Disabled());
			EXPECT_TRUE(ParticleHandles(2)->Disabled());
			EXPECT_TRUE(ParticleHandles(3)->Disabled());
			EXPECT_TRUE(ParticleHandles(4)->Disabled());
			EXPECT_TRUE(ParticleHandles(5)->Disabled());
			EXPECT_TRUE(ParticleHandles(6)->Disabled());
			EXPECT_TRUE(ParticleHandles(7)->Disabled());
			EXPECT_FALSE(ParticleHandles(8)->Disabled());

			UnitTest.Advance();
			UnitTest.Solver->GetPerSolverField().AddTransientCommand(Command);
			UnitTest.Advance();

			EXPECT_EQ(ClusterMap.Num(), 1);
			EXPECT_EQ(ClusterMap[ParticleHandles(7)].Num(), 3);
			EXPECT_TRUE(ClusterMap[ParticleHandles(7)].Contains(ParticleHandles(3)));
			EXPECT_TRUE(ClusterMap[ParticleHandles(7)].Contains(ParticleHandles(4)));
			EXPECT_TRUE(ClusterMap[ParticleHandles(7)].Contains(ParticleHandles(5)));

			EXPECT_FALSE(ParticleHandles(0)->Disabled());
			EXPECT_FALSE(ParticleHandles(1)->Disabled());
			EXPECT_FALSE(ParticleHandles(2)->Disabled());
			EXPECT_TRUE(ParticleHandles(3)->Disabled());
			EXPECT_TRUE(ParticleHandles(4)->Disabled());
			EXPECT_TRUE(ParticleHandles(5)->Disabled());
			EXPECT_TRUE(ParticleHandles(6)->Disabled());
			EXPECT_FALSE(ParticleHandles(7)->Disabled());
			EXPECT_TRUE(ParticleHandles(8)->Disabled());
		}

		delete FalloffField;
	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_ClusterBreak_StrainModel_Test3)
	{
		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody_ThreeByTwo_ThreeTransform(FVector(0));

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 1.0 };
		Params.MaxClusterLevel = 1000;
		Params.ClusterGroupIndex = 0;

		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);

		FRadialFalloff* FalloffField = new FRadialFalloff();
		FalloffField->Magnitude = 1.1f;
		FalloffField->Radius = 200.0f;
		FalloffField->Position = FVector(350.0, 0.0, 0.0);
		FalloffField->Falloff = EFieldFalloffType::Field_FallOff_None;

		UnitTest.Initialize();
		UnitTest.Advance();

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();

		auto ParticleHandles = [&Collection](int32 Index) -> Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*
		{
			return Collection->PhysObject->GetParticle_Internal(Index);
		};

		UnitTest.Advance();

		{
			FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_ExternalClusterStrain);
			FFieldSystemCommand Command(TargetName, FalloffField->NewCopy());
			FFieldSystemMetaDataProcessingResolution* ResolutionData = new FFieldSystemMetaDataProcessingResolution(EFieldResolutionType::Field_Resolution_Maximum);
			Command.MetaData.Add(FFieldSystemMetaData::EMetaType::ECommandData_ProcessingResolution, TUniquePtr< FFieldSystemMetaDataProcessingResolution >(ResolutionData));
			UnitTest.Solver->GetPerSolverField().AddTransientCommand(Command);

			EXPECT_TRUE(ParticleHandles(0)->Disabled());
			EXPECT_TRUE(ParticleHandles(1)->Disabled());
			EXPECT_TRUE(ParticleHandles(2)->Disabled());
			EXPECT_TRUE(ParticleHandles(3)->Disabled());
			EXPECT_TRUE(ParticleHandles(4)->Disabled());
			EXPECT_TRUE(ParticleHandles(5)->Disabled());
			EXPECT_TRUE(ParticleHandles(6)->Disabled());
			EXPECT_TRUE(ParticleHandles(7)->Disabled());
			EXPECT_FALSE(ParticleHandles(8)->Disabled());

			UnitTest.Advance();

			// todo: indices here might be off but the test crashes before this so we can't validate yet
			EXPECT_EQ(ClusterMap.Num(), 2);
			EXPECT_EQ(ClusterMap[ParticleHandles(7)].Num(), 3);
			EXPECT_TRUE(ClusterMap[ParticleHandles(7)].Contains(ParticleHandles(3)));
			EXPECT_TRUE(ClusterMap[ParticleHandles(7)].Contains(ParticleHandles(4)));
			EXPECT_TRUE(ClusterMap[ParticleHandles(7)].Contains(ParticleHandles(5)));
			EXPECT_EQ(ClusterMap[ParticleHandles(6)].Num(), 3);
			EXPECT_TRUE(ClusterMap[ParticleHandles(6)].Contains(ParticleHandles(0)));
			EXPECT_TRUE(ClusterMap[ParticleHandles(6)].Contains(ParticleHandles(1)));
			EXPECT_TRUE(ClusterMap[ParticleHandles(6)].Contains(ParticleHandles(2)));

			EXPECT_TRUE(ParticleHandles(0)->Disabled());
			EXPECT_TRUE(ParticleHandles(1)->Disabled());
			EXPECT_TRUE(ParticleHandles(2)->Disabled());
			EXPECT_TRUE(ParticleHandles(3)->Disabled());
			EXPECT_TRUE(ParticleHandles(4)->Disabled());
			EXPECT_TRUE(ParticleHandles(5)->Disabled());
			EXPECT_FALSE(ParticleHandles(6)->Disabled());
			EXPECT_FALSE(ParticleHandles(7)->Disabled());
			EXPECT_TRUE(ParticleHandles(8)->Disabled());
		}

		delete FalloffField;

	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_ClusterBreak_StrainModel_Test4)
	{
		FFramework UnitTest;

		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody_TwoByTwo_ThreeTransform(FVector(0));
		

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 1.0 };
		Params.MaxClusterLevel = 1000;
		Params.ClusterGroupIndex = 0;

		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);

		FRadialFalloff* FalloffField = new FRadialFalloff();
		FalloffField->Magnitude = 1.5;
		FalloffField->Radius = 100.0;
		FalloffField->Position = FVector(0.0, 0.0, 0.0);
		FalloffField->Falloff = EFieldFalloffType::Field_FallOff_None;

		UnitTest.Initialize();
		UnitTest.Advance();

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();

		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			Collection->PhysObject->GetParticle_Internal(4),
			Collection->PhysObject->GetParticle_Internal(5),
			Collection->PhysObject->GetParticle_Internal(6),
		};
		{
			FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_ExternalClusterStrain);
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, FalloffField->NewCopy() });

			EXPECT_TRUE(ParticleHandles[0]->Disabled());
			EXPECT_TRUE(ParticleHandles[1]->Disabled());
			EXPECT_TRUE(ParticleHandles[2]->Disabled());
			EXPECT_TRUE(ParticleHandles[3]->Disabled());
			EXPECT_TRUE(ParticleHandles[4]->Disabled());
			EXPECT_TRUE(ParticleHandles[5]->Disabled());
			EXPECT_FALSE(ParticleHandles[6]->Disabled());

			UnitTest.Advance();

			EXPECT_EQ(ClusterMap.Num(), 2);
			EXPECT_EQ(ClusterMap[ParticleHandles[4]].Num(), 2);
			EXPECT_TRUE(ClusterMap[ParticleHandles[4]].Contains(ParticleHandles[0]));
			EXPECT_TRUE(ClusterMap[ParticleHandles[4]].Contains(ParticleHandles[1]));
			EXPECT_EQ(ClusterMap[ParticleHandles[5]].Num(), 2);
			EXPECT_TRUE(ClusterMap[ParticleHandles[5]].Contains(ParticleHandles[2]));
			EXPECT_TRUE(ClusterMap[ParticleHandles[5]].Contains(ParticleHandles[3]));

			EXPECT_TRUE(ParticleHandles[0]->Disabled());
			EXPECT_TRUE(ParticleHandles[1]->Disabled());
			EXPECT_TRUE(ParticleHandles[2]->Disabled());
			EXPECT_TRUE(ParticleHandles[3]->Disabled());
			EXPECT_FALSE(ParticleHandles[4]->Disabled());
			EXPECT_FALSE(ParticleHandles[5]->Disabled());
			EXPECT_TRUE(ParticleHandles[6]->Disabled());
		}

		delete FalloffField;
	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_ClusterBreak_StrainModel_TestResolutionMinimal)
	{
		FFramework UnitTest;
		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody_ThreeByTwo_ThreeTransform(FVector(0));

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 101.0, 103.0};
		Params.MaxClusterLevel = 10;

		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);

		FRadialFalloff* FalloffField = new FRadialFalloff(); // field resolution is minimal by default
		FalloffField->Magnitude = -100.0;
		FalloffField->Radius = 10000.0;
		FalloffField->Position = FVector(0.0, 0.0, 0.0);
		FalloffField->Falloff = EFieldFalloffType::Field_FallOff_None;

		UnitTest.Initialize();

		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			Collection->PhysObject->GetParticle_Internal(4),
			Collection->PhysObject->GetParticle_Internal(5),
			Collection->PhysObject->GetParticle_Internal(6),
			Collection->PhysObject->GetParticle_Internal(7),
			Collection->PhysObject->GetParticle_Internal(8),
		};
		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ClusteredParticleHandles = {
			Collection->PhysObject->GetSolverClusterHandle_Internal(0),
			Collection->PhysObject->GetSolverClusterHandle_Internal(1),
			Collection->PhysObject->GetSolverClusterHandle_Internal(2),
			Collection->PhysObject->GetSolverClusterHandle_Internal(3),
			Collection->PhysObject->GetSolverClusterHandle_Internal(4),
			Collection->PhysObject->GetSolverClusterHandle_Internal(5),
			Collection->PhysObject->GetSolverClusterHandle_Internal(6),
			Collection->PhysObject->GetSolverClusterHandle_Internal(7),
			Collection->PhysObject->GetSolverClusterHandle_Internal(8),
		};

		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();
		UnitTest.Solver->RegisterSimOneShotCallback([&]()
		{
			ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			Collection->PhysObject->GetParticle_Internal(4),
			Collection->PhysObject->GetParticle_Internal(5),
			Collection->PhysObject->GetParticle_Internal(6),
			Collection->PhysObject->GetParticle_Internal(7),
			Collection->PhysObject->GetParticle_Internal(8),
			};
			ClusteredParticleHandles = {
				Collection->PhysObject->GetSolverClusterHandle_Internal(0),
				Collection->PhysObject->GetSolverClusterHandle_Internal(1),
				Collection->PhysObject->GetSolverClusterHandle_Internal(2),
				Collection->PhysObject->GetSolverClusterHandle_Internal(3),
				Collection->PhysObject->GetSolverClusterHandle_Internal(4),
				Collection->PhysObject->GetSolverClusterHandle_Internal(5),
				Collection->PhysObject->GetSolverClusterHandle_Internal(6),
				Collection->PhysObject->GetSolverClusterHandle_Internal(7),
				Collection->PhysObject->GetSolverClusterHandle_Internal(8),
			};
			EXPECT_EQ(ClusterMap.Num(), 3);
			EXPECT_EQ(ClusteredParticleHandles[1]->GetInternalStrains(), 101);
		});
		FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_InternalClusterStrain);
		FalloffField->Magnitude = 0.0;
		UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, FalloffField->NewCopy() });
		UnitTest.Advance();


		{
			// { 101.0, 103.0 }
			EXPECT_TRUE(ParticleHandles[0]->Disabled());
			EXPECT_TRUE(ParticleHandles[1]->Disabled());
			EXPECT_TRUE(ParticleHandles[2]->Disabled());
			EXPECT_TRUE(ParticleHandles[3]->Disabled());
			EXPECT_TRUE(ParticleHandles[4]->Disabled());
			EXPECT_TRUE(ParticleHandles[5]->Disabled());
			EXPECT_TRUE(ParticleHandles[6]->Disabled());
			EXPECT_TRUE(ParticleHandles[7]->Disabled());
			EXPECT_FALSE(ParticleHandles[8]->Disabled());

			FalloffField->Magnitude = -100;
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, FalloffField->NewCopy() });
			UnitTest.Advance();

			// { 1.0, 3.0 }
			EXPECT_TRUE(ParticleHandles[0]->Disabled());
			EXPECT_TRUE(ParticleHandles[1]->Disabled());
			EXPECT_TRUE(ParticleHandles[2]->Disabled());
			EXPECT_TRUE(ParticleHandles[3]->Disabled());
			EXPECT_TRUE(ParticleHandles[4]->Disabled());
			EXPECT_TRUE(ParticleHandles[5]->Disabled());
			EXPECT_TRUE(ParticleHandles[6]->Disabled());
			EXPECT_TRUE(ParticleHandles[7]->Disabled());
			EXPECT_FALSE(ParticleHandles[8]->Disabled()); // Parent still clustered w/ threshold > 0

			EXPECT_EQ(ClusterMap.Num(), 3);
			EXPECT_EQ(ClusterMap[ParticleHandles[6]].Num(), 3);
			EXPECT_TRUE(ClusterMap[ParticleHandles[6]].Contains(ParticleHandles[0]));
			EXPECT_TRUE(ClusterMap[ParticleHandles[6]].Contains(ParticleHandles[1]));
			EXPECT_TRUE(ClusterMap[ParticleHandles[6]].Contains(ParticleHandles[2]));
			EXPECT_EQ(ClusterMap[ParticleHandles[7]].Num(), 3);
			EXPECT_TRUE(ClusterMap[ParticleHandles[7]].Contains(ParticleHandles[3]));
			EXPECT_TRUE(ClusterMap[ParticleHandles[7]].Contains(ParticleHandles[4]));
			EXPECT_TRUE(ClusterMap[ParticleHandles[7]].Contains(ParticleHandles[5]));
			EXPECT_EQ(ClusterMap[ParticleHandles[8]].Num(), 2);
			EXPECT_TRUE(ClusterMap[ParticleHandles[8]].Contains(ParticleHandles[6]));
			EXPECT_TRUE(ClusterMap[ParticleHandles[8]].Contains(ParticleHandles[7]));

			FalloffField->Magnitude = -1.0;
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, FalloffField->NewCopy() });
			UnitTest.Advance();

			// { 0.0, 2.0 } broken
			EXPECT_FALSE(ParticleHandles[6]->Disabled());
			EXPECT_FALSE(ParticleHandles[7]->Disabled());
			EXPECT_TRUE(ParticleHandles[8]->Disabled()); 

			FalloffField->Magnitude = -103;
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, FalloffField->NewCopy() });
			UnitTest.Advance();

			// { 0.0, 0.0 }
			EXPECT_FALSE(ParticleHandles[0]->Disabled());
			EXPECT_FALSE(ParticleHandles[1]->Disabled());
			EXPECT_FALSE(ParticleHandles[2]->Disabled());
			EXPECT_FALSE(ParticleHandles[3]->Disabled());
			EXPECT_FALSE(ParticleHandles[4]->Disabled());
			EXPECT_FALSE(ParticleHandles[5]->Disabled());
			EXPECT_TRUE(ParticleHandles[6]->Disabled());
			EXPECT_TRUE(ParticleHandles[7]->Disabled());
			EXPECT_TRUE(ParticleHandles[8]->Disabled());
		}
	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_Algebra1)
	{
		FFramework UnitTest;
		FVector ExpectedLocation = FVector(0.0f, 0.0f, 0.0f);
		FReal LastZ = ExpectedLocation.Z;

		// Physics Object Setup
		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.RootTransform.SetLocation(ExpectedLocation);
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);

		// Opposing Fields setup (velocity)
		FUniformVector* VectorFieldRight = new FUniformVector();
		VectorFieldRight->Magnitude = 100.0;
		VectorFieldRight->Direction = FVector(1.0, 0.0, 0.0);
		FUniformVector* VectorFieldLeft = new FUniformVector();
		VectorFieldLeft->Magnitude = 100.0;
		VectorFieldLeft->Direction = FVector(-1.0f, 0.0, 0.0);

		// Opposing Fields setup (force)
		FUniformVector* VectorFieldForward = new FUniformVector();
		VectorFieldRight->Magnitude = 100.0;
		VectorFieldRight->Direction = FVector(0.0, 100.0, 0.0);
		FUniformVector* VectorFieldBackward = new FUniformVector();
		VectorFieldLeft->Magnitude = 100.0;
		VectorFieldLeft->Direction = FVector(0.0f, -100.0, 0.0);

		UnitTest.Initialize();

		FName TargetNameVel = GetFieldPhysicsName(EFieldPhysicsType::Field_LinearVelocity);
		UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetNameVel, VectorFieldRight->NewCopy() });
		UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetNameVel, VectorFieldLeft->NewCopy() });
		FName TargetNameForce = GetFieldPhysicsName(EFieldPhysicsType::Field_LinearForce);
		UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetNameForce, VectorFieldForward->NewCopy() });
		UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetNameForce, VectorFieldBackward->NewCopy() });
		UnitTest.Advance();

		
		for (int Frame = 1; Frame < 10; Frame++)
		{
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ GetFieldPhysicsName(EFieldPhysicsType::Field_LinearVelocity), VectorFieldRight->NewCopy() });
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ GetFieldPhysicsName(EFieldPhysicsType::Field_LinearVelocity), VectorFieldLeft->NewCopy() });
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ GetFieldPhysicsName(EFieldPhysicsType::Field_LinearForce), VectorFieldForward->NewCopy() });
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ GetFieldPhysicsName(EFieldPhysicsType::Field_LinearForce), VectorFieldBackward->NewCopy() });

			UnitTest.Advance();

			EXPECT_NEAR(Collection->DynamicCollection->GetTransform(0).GetTranslation().X, ExpectedLocation.X, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(Collection->DynamicCollection->GetTransform(0).GetTranslation().Y, ExpectedLocation.Y, KINDA_SMALL_NUMBER);
			EXPECT_LT(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, ExpectedLocation.Z);
			EXPECT_LT(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, LastZ);
			LastZ = Collection->DynamicCollection->GetTransform(0).GetTranslation().Z;
		}
	}


	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_Algebra2)
	{
		FFramework UnitTest;
		FVector ExpectedLocation = FVector(0.0f, 0.0f, 0.0f);
		FVector LastLocation = ExpectedLocation;

		// Physics Object Setup
		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.InitialVelocityType = EInitialVelocityTypeEnum::Chaos_Initial_Velocity_User_Defined;
		Params.InitialLinearVelocity = FVector3f(100.0, 0.0, 0.0);
		Params.RootTransform.SetLocation(ExpectedLocation);
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);

		// Opposing Fields setup (force)
		FUniformVector* VectorFieldRightForce = new FUniformVector();
		VectorFieldRightForce->Magnitude = 100.0;
		VectorFieldRightForce->Direction = FVector(100.0, 0.0, 0.0);
		FUniformVector* VectorFieldLeftForce = new FUniformVector();
		VectorFieldLeftForce->Magnitude = 100.0;
		VectorFieldLeftForce->Direction = FVector(-100.0f, 0.0, 0.0);

		UnitTest.Initialize();
		UnitTest.Advance();

		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
		};
		Chaos::TVector<float, 3> CurrV = ParticleHandles[0]->GetV();
		

		for (int Frame = 1; Frame < 10; Frame++)
		{
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ GetFieldPhysicsName(EFieldPhysicsType::Field_LinearForce), VectorFieldRightForce->NewCopy() });
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ GetFieldPhysicsName(EFieldPhysicsType::Field_LinearForce), VectorFieldLeftForce->NewCopy() });

			UnitTest.Advance();

			CurrV = ParticleHandles[0]->GetV();
			EXPECT_NEAR(CurrV.X, Params.InitialLinearVelocity.X, KINDA_SMALL_NUMBER); // Velocity in +x
			EXPECT_GT(Collection->DynamicCollection->GetTransform(0).GetTranslation().X, LastLocation.X); // Pos in +x

			// Still falling?
			EXPECT_LT(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z, LastLocation.Z);

			LastLocation = FVector(Collection->DynamicCollection->GetTransform(0).GetTranslation());

		}
	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_Algebra3)
	{
		FFramework UnitTest;

		// Physics Object Setup
		FVector Scale = FVector(10.0);
		CreationParameters Params;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.RootTransform.SetLocation(FVector(0.f, 0.f, 5.f));
		Params.GeomTransform.SetScale3D(Scale);
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);

		// Fields setup... when both our applied, we expect Y rotation to cancel, while X rotation > 0.
		FUniformVector* UniformVector1 = new FUniformVector();
		UniformVector1->Direction = FVector(2.0, 1.0, 0.0);
		UniformVector1->Magnitude = 100.0;
		FUniformVector* UniformVector2 = new FUniformVector();
		UniformVector2->Direction = FVector(-1.0, -1.0, 0.0);
		UniformVector2->Magnitude = 100.0;

		UnitTest.Initialize();

		
		FReal PreviousHeight = Collection->DynamicCollection->GetTransform(0).GetTranslation().Z;
		FReal PreviousX = Collection->DynamicCollection->GetTransform(0).GetRotation().Euler().X;
		for (int Frame = 0; Frame < 10; Frame++)
		{
			
			FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_AngularTorque);
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, UniformVector1->NewCopy() });
			UnitTest.Solver->GetPerSolverField().AddTransientCommand({ TargetName, UniformVector2->NewCopy() });
			
			UnitTest.Advance();

			Chaos::TPBDGeometryCollectionParticles<Chaos::FReal, 3>& Particles = UnitTest.Solver->GetParticles().GetGeometryCollectionParticles();
			EXPECT_NE(FMath::Abs(Collection->DynamicCollection->GetTransform(0).GetRotation().Euler().Y), SMALL_THRESHOLD); // not rotating in Y?
			EXPECT_GT(Particles.GetW(0).X, PreviousX); // rotating in X?
			EXPECT_LT(Particles.GetX(0).Z, PreviousHeight); // still falling?

			PreviousHeight = Particles.GetX(0).Z;
			PreviousX = Particles.GetW(0).X;
		}


	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_FieldResolutionMinimal)
	{
		FFramework UnitTest;

		// Construct Rest Collection...
		TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(0, 0, 0)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(100, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(200, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(300, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(400, 0, 0)), FVector(1.0)));
		RestCollection->Transform[0].SetTranslation(FVector3f(0.0f));
		RestCollection->SimulationType[0] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[1] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[2] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[3] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[4] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 0, { 1 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 1, { 2 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 2, { 3 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 3, { 4 });

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 91.0f, 92.0f, 93.0f, 94.0f };
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);

		FRadialFalloff* FalloffField = new FRadialFalloff();
		FalloffField->Magnitude = -100;
		FalloffField->Radius = 1000.0;
		FalloffField->Position = FVector(0.0, 0.0, 0.0);
		FalloffField->Falloff = EFieldFalloffType::Field_FallOff_None;

		UnitTest.Initialize();
		UnitTest.Advance();

		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			Collection->PhysObject->GetParticle_Internal(4),
		};
		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();

		
		Chaos::TArrayCollectionArray<Chaos::FRealSingle>& StrainArray = UnitTest.Solver->GetEvolution()->GetRigidClustering().GetStrainArray();

		EXPECT_FALSE(ParticleHandles[0]->Disabled());
		EXPECT_TRUE(ParticleHandles[1]->Disabled());
		EXPECT_TRUE(ParticleHandles[2]->Disabled());
		EXPECT_TRUE(ParticleHandles[3]->Disabled());
		EXPECT_TRUE(ParticleHandles[4]->Disabled());

		FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_InternalClusterStrain);
		FFieldSystemCommand Command(TargetName, FalloffField->NewCopy());
		FFieldSystemMetaDataProcessingResolution* ResolutionData = new FFieldSystemMetaDataProcessingResolution(EFieldResolutionType::Field_Resolution_Minimal);
		Command.MetaData.Add(FFieldSystemMetaData::EMetaType::ECommandData_ProcessingResolution, TUniquePtr< FFieldSystemMetaDataProcessingResolution >(ResolutionData));
		UnitTest.Solver->GetPerSolverField().AddTransientCommand(Command);
		UnitTest.Advance();

		// We expect only active clusters and their children to be affected by the field with Field_Resolution_Minimal
		EXPECT_GT(StrainArray[0], 0.0f);
		EXPECT_GT(StrainArray[1], 0.0f);
		EXPECT_GT(0.0f, StrainArray[2]);
		EXPECT_GT(0.0f, StrainArray[3]);

	}

	GTEST_TEST(AllTraits, GeometryCollection_RigidBodies_Field_FieldResolutionMaximum)
	{
		FFramework UnitTest;

		// Construct Rest Collection...
		TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(0, 0, 0)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(100, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(200, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(300, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(400, 0, 0)), FVector(1.0)));
		RestCollection->Transform[0].SetTranslation(FVector3f(0.0f));
		RestCollection->SimulationType[0] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[1] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[2] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[3] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[4] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 0, { 1 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 1, { 2 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 2, { 3 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 3, { 4 });

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 91.0f, 92.0f, 93.0f, 94.0f };
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);

		FRadialFalloff* FalloffField = new FRadialFalloff();
		FalloffField->Magnitude = -100;
		FalloffField->Radius = 1000.0;
		FalloffField->Position = FVector(0.0, 0.0, 0.0);
		FalloffField->Falloff = EFieldFalloffType::Field_FallOff_None;

		UnitTest.Initialize();
		UnitTest.Advance();

		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal, 3>*> ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			Collection->PhysObject->GetParticle_Internal(4),
		};
		auto& Clustering = UnitTest.Solver->GetEvolution()->GetRigidClustering();
		const auto& ClusterMap = Clustering.GetChildrenMap();


		Chaos::TArrayCollectionArray<Chaos::FRealSingle>& StrainArray = UnitTest.Solver->GetEvolution()->GetRigidClustering().GetStrainArray();

		EXPECT_FALSE(ParticleHandles[0]->Disabled());
		EXPECT_TRUE(ParticleHandles[1]->Disabled());
		EXPECT_TRUE(ParticleHandles[2]->Disabled());
		EXPECT_TRUE(ParticleHandles[3]->Disabled());
		EXPECT_TRUE(ParticleHandles[4]->Disabled());

		FName TargetName = GetFieldPhysicsName(EFieldPhysicsType::Field_InternalClusterStrain);
		FFieldSystemCommand Command(TargetName, FalloffField->NewCopy());
		FFieldSystemMetaDataProcessingResolution* ResolutionData = new FFieldSystemMetaDataProcessingResolution(EFieldResolutionType::Field_Resolution_Maximum);
		Command.MetaData.Add(FFieldSystemMetaData::EMetaType::ECommandData_ProcessingResolution, TUniquePtr< FFieldSystemMetaDataProcessingResolution >(ResolutionData));
		UnitTest.Solver->GetPerSolverField().AddTransientCommand(Command);
		UnitTest.Advance();
		UnitTest.Advance();
		// We expect all clusters to be affected by the field with Field_Resolution_Maximum
		EXPECT_GT(0.0f, StrainArray[0]);
		EXPECT_GT(0.0f, StrainArray[1]);
		EXPECT_GT(0.0f, StrainArray[2]);
		EXPECT_GT(0.0f, StrainArray[3]);
		
	}

}



