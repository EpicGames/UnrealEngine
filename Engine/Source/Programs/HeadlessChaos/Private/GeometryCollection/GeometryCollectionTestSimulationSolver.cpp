// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestSimulationSolver.h"
#include "GeometryCollection/GeometryCollectionTestFramework.h"
#include "GeometryCollection/GeometryCollectionTestUtility.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/TransformCollection.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"


#include "Field/FieldSystem.h"
#include "Field/FieldSystemNodes.h"
#include "GeometryCollectionProxyData.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "PhysicsProxy/PhysicsProxies.h"
#include "Chaos/ErrorReporter.h"
#include "Chaos/Defines.h"
#include "ChaosSolversModule.h"
#include "PhysicsSolver.h"
#include "EventsData.h"
#include "HeadlessChaosTestUtility.h"

#define SMALL_THRESHOLD 1e-4


namespace GeometryCollectionTest
{
using namespace ChaosTest;
	
	GTEST_TEST(AllTraits, GeometryCollection_Solver_AdvanceNoObjects)
	{
		FFramework UnitTest;
		UnitTest.Initialize();
		UnitTest.Advance();

		{
			// just makiung sure we did not crash in this test
		}
	}


	
	GTEST_TEST(AllTraits, GeometryCollection_Solver_AdvanceDisabledObjects)
	{
		CreationParameters Params; Params.Simulating = false;
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();

		FFramework UnitTest;
		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();
		UnitTest.Advance();

		{
			// never touched
			TManagedArray<FTransform3f>& RestTransform = Collection->RestCollection->Transform;
			EXPECT_LT(FMath::Abs(RestTransform[0].GetTranslation().Z), SMALL_THRESHOLD);

			// simulated
			EXPECT_EQ(Collection->DynamicCollection->GetNumTransforms(), 1);
			EXPECT_LT(FMath::Abs(Collection->DynamicCollection->GetTransform(0).GetTranslation().Z), SMALL_THRESHOLD);
		}
	}

	
	GTEST_TEST(AllTraits, GeometryCollection_Solver_AdvanceDisabledClusteredObjects)
	{
		FFramework UnitTest;

		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();
		UnitTest.AddSimulationObject(Floor);

		TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, -10, 10)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, 10, 10)), FVector(1.0)));
		EXPECT_EQ(RestCollection->Transform.Num(), 2);

		FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(RestCollection.Get());
		EXPECT_EQ(RestCollection->Transform.Num(), 3);
		RestCollection->Transform[2] = FTransform3f(FQuat4f::MakeFromEuler(FVector3f(90.f, 0, 0.)), FVector3f(0, 0, 40));

		//GeometryCollectionAlgo::PrintParentHierarchy(RestCollection.Get());

		CreationParameters Params;
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 1000.f };
		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();

		UnitTest.AddSimulationObject(Collection);
		UnitTest.Initialize();

		FVector3f StartingClusterPosition;
		FReal StartingRigidDistance;

		UnitTest.Solver->RegisterSimOneShotCallback([&]()
		{
			// Dynamic collection doesn't change the simulation type, RestCollection can be used instead
			TManagedArray<int32>& SimulationType = RestCollection->SimulationType;
			EXPECT_EQ(SimulationType[0],FGeometryCollection::ESimulationTypes::FST_Rigid);
			EXPECT_EQ(SimulationType[1],FGeometryCollection::ESimulationTypes::FST_Rigid);
			EXPECT_EQ(SimulationType[2],FGeometryCollection::ESimulationTypes::FST_Clustered);

			EXPECT_EQ(Collection->DynamicCollection->GetParent(0),2);
			EXPECT_EQ(Collection->DynamicCollection->GetParent(1),2);
			EXPECT_EQ(Collection->DynamicCollection->GetParent(2),-1);

			// Set the one cluster to disabled
			UnitTest.Solver->GetEvolution()->DisableParticle(Collection->PhysObject->GetSolverClusterHandle_Internal(0));


			StartingRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size();
			EXPECT_LT(StartingRigidDistance - 20.0f,SMALL_THRESHOLD);
			StartingClusterPosition = Collection->DynamicCollection->GetTransform(2).GetTranslation();
		});

		FReal CurrentRigidDistance = 0.0;

		for (int Frame = 0; Frame < 10; Frame++)
		{
			UnitTest.Advance();

			// Distance between gc cubes remains the same
			CurrentRigidDistance = (Collection->DynamicCollection->GetTransform(1).GetTranslation() - Collection->DynamicCollection->GetTransform(0).GetTranslation()).Size();
			EXPECT_LT(StartingRigidDistance-CurrentRigidDistance, SMALL_THRESHOLD);

			// Clustered particle doesn't move
			EXPECT_LT((StartingClusterPosition - Collection->DynamicCollection->GetTransform(2).GetTranslation()).Size(), SMALL_THRESHOLD);
		}
		
	}


	
	GTEST_TEST(AllTraits, DISABLED_GeometryCollection_Solver_ValidateReverseMapping)
	{
		// Todo: this test is missing code for InitCollections that seems to be no longer in the code base.  
		// This makes it unclear how this test is setup

		/*
		TUniquePtr<Chaos::FChaosPhysicsMaterial> PhysicalMaterial = MakeUnique<Chaos::FChaosPhysicsMaterial>();
		InitMaterialToZero(PhysicalMaterial.Get());

		Chaos::FPBDRigidsSolver* Solver = FChaosSolversModule::GetModule()->CreateSolver(nullptr, ESolverFlags::Standalone);
		//Solver->SetHasFloor(true);
		Solver->SetEnabled(true);
		TArray<TSharedPtr<FGeometryCollection> > RestArray;
		TArray<TSharedPtr<FGeometryDynamicCollection> > DynamicArray;

		for (int32 i = 0; i < 10; i++)
		{
			TSharedPtr<FGeometryCollection> RestCollection = nullptr;
			TSharedPtr<FGeometryDynamicCollection> DynamicCollection = nullptr;

			//InitCollectionsParameters InitParams = { FTransform::Identity, FVector(1.0), nullptr, (int32)EObjectStateTypeEnum::Chaos_Object_Kinematic };
			//InitCollections(PhysicalMaterial, RestCollection, DynamicCollection, InitParams);

			FGeometryCollectionPhysicsProxy* PhysObject = RigidBodySetup(PhysicalMaterial, RestCollection, DynamicCollection);

#if CHAOS_PARTICLEHANDLE_TODO
			Solver->RegisterObject(PhysObject);
#endif
			//PhysObject->ActivateBodies();

			RestArray.Add(RestCollection);
			DynamicArray.Add(DynamicCollection);
		}


		Solver->AdvanceSolverBy(1 / 24.);

#if TODO_REIMPLEMENT_PHYSICS_PROXY_REVERSE_MAPPING
		const Chaos::TArrayCollectionArray<PhysicsProxyWrapper> & SolverMapping = Solver->GetPhysicsProxyReverseMapping();
		const Chaos::TArrayCollectionArray<int32> & ParticleMapping = Solver->GetParticleIndexReverseMapping();

		EXPECT_EQ(SolverMapping.Num(), 11);
		EXPECT_EQ(ParticleMapping.Num(), 11);

		EXPECT_EQ(ParticleMapping[0], INDEX_NONE);
		EXPECT_EQ(ParticleMapping[1], 0);

		EXPECT_EQ(SolverMapping[0].PhysicsProxy, nullptr);
		EXPECT_EQ(SolverMapping[0].Type, EPhysicsProxyType::NoneType);

		EXPECT_EQ(SolverMapping[5].PhysicsProxy != nullptr);
		EXPECT_EQ(SolverMapping[5].Type, EPhysicsProxyType::GeometryCollectionType);

		const TManagedArray<int32> & RigidBodyID = static_cast<FGeometryCollectionPhysicsProxy*>(SolverMapping[5].PhysicsProxy)->
			GetGeometryDynamicCollection_PhysicsThread()->GetAttribute<int32>("RigidBodyID", FGeometryCollection::TransformGroup);
		EXPECT_EQ(RigidBodyID.Num(), 1);
		EXPECT_EQ(RigidBodyID[0], 5);
#endif

		FChaosSolversModule::GetModule()->DestroySolver(Solver);
		*/
	}
	
	TSharedPtr<FGeometryCollection> CommonInit(int32 NumObjects, bool UseClusters)
	{	
		TSharedPtr<FGeometryCollection> RestCollectionOut;

		for (int32 i = 0; i < NumObjects; i++)
		{

			TSharedPtr<FGeometryCollection> RestCollection;

			if (UseClusters)
			{
				RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, -10, 10)), FVector(1.0));
				RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, 10, 10)), FVector(1.0)));
				RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, 0, 0)), FVector(1.0)));
				RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, 0, 10)), FVector(1.0)));
				FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(RestCollection.Get());
				RestCollection->Transform[4] = FTransform3f(FQuat4f::MakeFromEuler(FVector3f(90.f, 0, 0.)), FVector3f(0, 0, 40));
			}
			else
			{
				RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(i * 3, 0, 5)), FVector(1.0));
			}
						

			if (i == 0)
			{
				RestCollectionOut = RestCollection;
			}
			else
			{
				RestCollectionOut->AppendGeometry(*RestCollection);			
			}
		}		

		return RestCollectionOut;
	}

	class FEventHarvester
	{
	public:
		FEventHarvester(Chaos::FPBDRigidsSolver* Solver)
		{
			Solver->GetEventManager()->template RegisterHandler<Chaos::FCollisionEventData>(Chaos::EEventType::Collision, this, &FEventHarvester::HandleCollisionEvents);
			Solver->GetEventManager()->template RegisterHandler<Chaos::FBreakingEventData>(Chaos::EEventType::Breaking, this, &FEventHarvester::HandleBreakingEvents);
		}

		void HandleCollisionEvents(const Chaos::FCollisionEventData& Events)
		{
			CollisionEventData = Events;
		}

		void HandleBreakingEvents(const Chaos::FBreakingEventData& Events)
		{
			BreakingEventData = Events;
		}

		Chaos::FCollisionEventData CollisionEventData;
		Chaos::FBreakingEventData BreakingEventData;
	};

	
	GTEST_TEST(AllTraits, GeometryCollection_Solver_CollisionEventFilter)
	{
		FReal TestMassThreshold = 7.0;

		FGeometryCollectionWrapper* Collection[10];
		for (int i=0; i<10; i++)
		{
			TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0)), FVector(0,0,0)), FVector(1.0));

			CreationParameters Params;
			Params.RestCollection = RestCollection;
			Params.SimplicialType = ESimplicialType::Chaos_Simplicial_Box;
			Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
			Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
			Params.RootTransform.SetLocation(FVector(i * 10.0f, 0.f, 10.f));
			Params.Simulating = true;
			
			Collection[i] = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(Params)->template As<FGeometryCollectionWrapper>();
		}
		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();

		FFramework UnitTest;
		for (int i=0; i<10; i++)
		{
			TManagedArray<float>& Mass = Collection[i]->RestCollection->template ModifyAttribute<float>("Mass", FTransformCollection::TransformGroup);
			Mass[0] = i + 1;
			UnitTest.AddSimulationObject(Collection[i]);
		}
		UnitTest.AddSimulationObject(Floor);
		UnitTest.Initialize();

		// setup collision filter
		FSolverCollisionFilterSettings CollisionFilterSettings;
		CollisionFilterSettings.FilterEnabled = true;
		CollisionFilterSettings.MinImpulse = 0;
		CollisionFilterSettings.MinMass = TestMassThreshold;
		CollisionFilterSettings.MinSpeed = 0;

		UnitTest.Solver->SetGenerateCollisionData(true);
		UnitTest.Solver->SetCollisionFilterSettings(CollisionFilterSettings);
		FEventHarvester Events(UnitTest.Solver);

		bool Impact = false;
		for(int LoopCount=0; LoopCount<10; LoopCount++)
		{
			// Events data on physics thread is appended until the game thread has had a chance to Tick & read it
			UnitTest.Advance();

			const auto& AllCollisionsArray = Events.CollisionEventData.CollisionData.AllCollisionsArray;
			Impact = (AllCollisionsArray.Num() > 0);

			if (Impact)
			{
				// any objects with a mass of less than 6 are removed from returned collision data
				EXPECT_LE(AllCollisionsArray.Num(), 4);

				for (const auto& Collision : AllCollisionsArray)
				{ 
					EXPECT_GE(Collision.Mass1, TestMassThreshold);
					EXPECT_EQ(Collision.Mass2, 0.0f);
					EXPECT_LT(Collision.Velocity2.Z, SMALL_NUMBER);
				}
				break;
			}
		} 

		EXPECT_EQ(Impact, true);
	}

	
	GTEST_TEST(AllTraits, GeometryCollection_Solver_BreakingEventFilter)
	{
		FFramework UnitTest;
		
		TSharedPtr<FGeometryCollection> RestCollection = CreateClusteredBody_OnePartent_FourBodies(FVector(0));

		CreationParameters Params;
		Params.RootTransform.SetLocation(FVector(0, 0, 20));
		Params.RestCollection = RestCollection;
		Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_LevelSet;
		Params.Simulating = true;
		Params.EnableClustering = true;
		Params.DamageThreshold = { 1.0 };
		Params.MaxClusterLevel = 1000;
		Params.ClusterConnectionMethod = Chaos::FClusterCreationParameters::EConnectionMethod::DelaunayTriangulation;
		Params.ClusterGroupIndex = 0;

		RigidBodyWrapper* Floor = TNewSimulationObject<GeometryType::RigidFloor>::Init()->template As<RigidBodyWrapper>();
		UnitTest.AddSimulationObject(Floor);

		FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->template As<FGeometryCollectionWrapper>();
		UnitTest.AddSimulationObject(Collection);
		Collection->PhysObject->SetCollisionParticlesPerObjectFraction(1.0);		

		UnitTest.Initialize();
		TArray<Chaos::TPBDRigidClusteredParticleHandle<FReal,3>*> ParticleHandles;
		FReal TestMass = 7.0;

		UnitTest.Solver->RegisterSimOneShotCallback([&]()
		{


			// setup breaking filter
			FSolverBreakingFilterSettings BreakingFilterSettings;
			BreakingFilterSettings.FilterEnabled = true;
			BreakingFilterSettings.MinMass = TestMass;
			BreakingFilterSettings.MinSpeed = 0;
			BreakingFilterSettings.MinVolume = 0;

			UnitTest.Solver->SetGenerateBreakingData(true);
			UnitTest.Solver->SetBreakingFilterSettings(BreakingFilterSettings);

			ParticleHandles = {
			Collection->PhysObject->GetParticle_Internal(0),
			Collection->PhysObject->GetParticle_Internal(1),
			Collection->PhysObject->GetParticle_Internal(2),
			Collection->PhysObject->GetParticle_Internal(3),
			};

			ParticleHandles[0]->SetM(TestMass + 1.0f);
			ParticleHandles[1]->SetM(TestMass - 1.0f);
			ParticleHandles[2]->SetM(TestMass - 2.0f);
			ParticleHandles[3]->SetM(TestMass + 2.0f);
		});


		FEventHarvester Events(UnitTest.Solver);

		bool Impact = false;
		for(int Loop=0; Loop<50; Loop++)
		{
			// Events data on physics thread is appended until the game thread has had a chance to Tick & read it
			UnitTest.Advance();

			const auto& AllBreakingsArray = Events.BreakingEventData.BreakingData.AllBreakingsArray;
			Impact = (AllBreakingsArray.Num() > 0);

			if(Impact)
			{
				EXPECT_EQ(ParticleHandles[0]->Disabled(),false); // piece1 active 6 mass
				EXPECT_EQ(ParticleHandles[1]->Disabled(),false); // piece2 active 0.5 mass
				EXPECT_EQ(ParticleHandles[2]->Disabled(),false); // piece3 active 0.5 mass
				EXPECT_EQ(ParticleHandles[3]->Disabled(),false); // cluster active 7 mass
				EXPECT_EQ(ParticleHandles[4]->Disabled(),true); // cluster

				// breaking data
				EXPECT_EQ(AllBreakingsArray.Num(),2); // 2 pieces filtered out of 4

				EXPECT_LT(AllBreakingsArray[0].Mass - (TestMass + 2.0f),SMALL_THRESHOLD);
				EXPECT_LT(AllBreakingsArray[1].Mass - (TestMass + 1.0f),SMALL_THRESHOLD);
				break;
			}


		}
	}
}




