// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestSimulationStreaming.h"
#include "GeometryCollection/GeometryCollectionTestFramework.h"
#include "GeometryCollection/GeometryCollectionTestUtility.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/TransformCollection.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "Field/FieldSystem.h"
#include "Field/FieldSystemNodes.h"
#include "GeometryCollectionProxyData.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "PhysicsProxy/PhysicsProxies.h"
#include "Chaos/ErrorReporter.h"
#include "ChaosSolversModule.h"
#include "PBDRigidsSolver.h"

#define SMALL_THRESHOLD 1e-4

namespace GeometryCollectionTest
{
	void RigidBodies_Streaming_StartSolverEmpty()
	{
		FFramework UnitTest;
		
		// no floor
		UnitTest.Initialize();

		UnitTest.Advance();
								
		for (int32 Frame = 1; Frame < 1000; Frame++)
		{
			UnitTest.Advance();
			if (Frame % 100 == 0)
			{
				TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, 0, 0)), FVector(1.0));

				CreationParameters Params;
				Params.RestCollection = RestCollection;
				Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
				Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
				Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
				Params.Simulating = true;				

				FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->As<FGeometryCollectionWrapper>();
				UnitTest.AddSimulationObject(Collection);
							
				// todo: this is crashing
				UnitTest.Solver->RegisterObject(Collection->PhysObject);
				UnitTest.Solver->AddDirtyProxy(Collection->PhysObject);
				Collection->PhysObject->Initialize(UnitTest.Solver->GetEvolution());
			}
		}
		
		EXPECT_EQ(UnitTest.PhysicsObjects.Num(), 9);
		for (int32 i = 0; i < UnitTest.PhysicsObjects.Num() - 1; i++)
		{
			FGeometryCollectionWrapper* GCW = UnitTest.PhysicsObjects[i]->As<FGeometryCollectionWrapper>();
			FGeometryCollectionWrapper* GCW2 = UnitTest.PhysicsObjects[i + 1]->As<FGeometryCollectionWrapper>();
			EXPECT_EQ(GCW->PhysObject->GetUnorderedParticles_Internal().Num(), 1);
			EXPECT_EQ(GCW2->PhysObject->GetUnorderedParticles_Internal().Num(), 1);
			EXPECT_EQ(GCW->PhysObject->GetNumTransforms(), 1);
			EXPECT_EQ(GCW2->PhysObject->GetNumTransforms(), 1);
			EXPECT_LT(GCW->PhysObject->GetParticle_Internal(0)->GetX().Z, GCW2->PhysObject->GetParticle_Internal(0)->GetX().Z);
		}
	}


	void RigidBodies_Streaming_BulkInitialization()
	{
		FFramework UnitTest;

		// no floor
		UnitTest.Initialize();

		UnitTest.Advance();	
		
		for (int32 Frame = 1; Frame < 1000; Frame++)
		{
			UnitTest.Advance();
			if (Frame % 100 == 0)
			{
				TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, 0, 0)), FVector(1.0));

				CreationParameters Params;
				Params.RestCollection = RestCollection;
				Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
				Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
				Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
				Params.Simulating = true;
				Params.CollisionGroup = -1;

				FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->As<FGeometryCollectionWrapper>();
				UnitTest.AddSimulationObject(Collection);

				// todo: this is crashing
				UnitTest.Solver->RegisterObject(Collection->PhysObject);
				UnitTest.Solver->AddDirtyProxy(Collection->PhysObject);
				Collection->PhysObject->Initialize(UnitTest.Solver->GetEvolution());
			}
		}
		
		/*
		todo: what is the replacement for ActivateBodies?
		for (auto Obj : Collections)
			Obj->PhysicsProxy->ActivateBodies();
		*/

		for (int32 Frame = 1; Frame < 100; Frame++)
		{
			UnitTest.Advance();
		}

		EXPECT_EQ(UnitTest.PhysicsObjects.Num(), 9);
		for (int32 i = 0; i < UnitTest.PhysicsObjects.Num() - 1; i++)
		{
			FGeometryCollectionWrapper* GCW = UnitTest.PhysicsObjects[i]->As<FGeometryCollectionWrapper>();
			FGeometryCollectionWrapper* GCW2 = UnitTest.PhysicsObjects[i + 1]->As<FGeometryCollectionWrapper>();
			EXPECT_EQ(GCW->PhysObject->GetUnorderedParticles_Internal().Num(), 1);
			EXPECT_EQ(GCW2->PhysObject->GetUnorderedParticles_Internal().Num(), 1);
			EXPECT_EQ(GCW->PhysObject->GetNumTransforms(), 1);
			EXPECT_EQ(GCW2->PhysObject->GetNumTransforms(), 1);
			EXPECT_LT(FMath::Abs(GCW->PhysObject->GetParticle_Internal(0)->GetX().Z - GCW2->PhysObject->GetParticle_Internal(0)->GetX().Z), KINDA_SMALL_NUMBER);
		}
	}



	void RigidBodies_Streaming_DeferedClusteringInitialization()
	{
		FFramework UnitTest;

		// no floor
		UnitTest.Initialize();

		UnitTest.Advance();
		
		for (int32 Frame = 1; Frame < 1000; Frame++)
		{
			UnitTest.Advance();

			if (Frame % 100 == 0)
			{
				TSharedPtr<FGeometryCollection> RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0, 0, 0.)), FVector(0, 0, 0)), FVector(1.0));

				CreationParameters Params;
				Params.RestCollection = RestCollection;
				Params.DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
				Params.CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
				Params.ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Box;
				Params.Simulating = true;
				Params.CollisionGroup = -1;
				Params.EnableClustering = true;
				Params.ClusterGroupIndex = 1;

				FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(Params)->As<FGeometryCollectionWrapper>();
				UnitTest.AddSimulationObject(Collection);

				// todo: this is crashing
				UnitTest.Solver->RegisterObject(Collection->PhysObject);
				UnitTest.Solver->AddDirtyProxy(Collection->PhysObject);
				Collection->PhysObject->Initialize(UnitTest.Solver->GetEvolution());
			}
		}
	
		// all particles should be disabled
		EXPECT_EQ(UnitTest.PhysicsObjects.Num(), 9);
		for (int32 i = 0; i < UnitTest.PhysicsObjects.Num(); i++)
		{
			FGeometryCollectionWrapper* GCW = UnitTest.PhysicsObjects[i]->As<FGeometryCollectionWrapper>();
			EXPECT_EQ(GCW->PhysObject->GetUnorderedParticles_Internal().Num(), 1);
			EXPECT_EQ(GCW->PhysObject->GetNumTransforms(), 1);
			EXPECT_EQ(GCW->PhysObject->GetParticle_Internal(0)->Disabled(), true);
		}

		/*
		todo: what is the replacement for ActivateBodies
		for (auto Obj : Collections)
			Obj->PhysicsProxy->ActivateBodies();
		*/

		// all particles should be enabled
		EXPECT_EQ(UnitTest.PhysicsObjects.Num(), 9);
		for (int32 i = 0; i < UnitTest.PhysicsObjects.Num(); i++)
		{
			FGeometryCollectionWrapper* GCW = UnitTest.PhysicsObjects[i]->As<FGeometryCollectionWrapper>();
			EXPECT_EQ(GCW->PhysObject->GetUnorderedParticles_Internal().Num(), 1);
			EXPECT_EQ(GCW->PhysObject->GetNumTransforms(), 1);
			EXPECT_EQ(GCW->PhysObject->GetParticle_Internal(0)->Disabled(), false);
		}

		for (int32 Frame = 1; Frame < 100; Frame++)
		{
			UnitTest.Advance();			
		}

		// new cluster parent should be falling
		// todo: the logic here is probably off.  How is the new cluster made? Is this at odds with each particle being in a separate GC
		// Difficult to tell based on missing code in initialization
		EXPECT_EQ(UnitTest.PhysicsObjects.Num(), 10);
		for (int32 i = 0; i < UnitTest.PhysicsObjects.Num() - 1; i++)
		{
			FGeometryCollectionWrapper* GCW = UnitTest.PhysicsObjects[i]->As<FGeometryCollectionWrapper>();
			EXPECT_EQ(GCW->PhysObject->GetUnorderedParticles_Internal().Num(), 1);
			EXPECT_EQ(GCW->PhysObject->GetNumTransforms(), 1);
			EXPECT_EQ(GCW->PhysObject->GetParticle_Internal(0)->Disabled(), true);
		}
		FGeometryCollectionWrapper* GCW = UnitTest.PhysicsObjects[UnitTest.PhysicsObjects.Num() - 1]->As<FGeometryCollectionWrapper>();
		EXPECT_EQ(GCW->PhysObject->GetParticle_Internal(0)->Disabled(), false);

		EXPECT_LT(GCW->PhysObject->GetParticle_Internal(0)->GetX().Z, -1.f);
	}


}

