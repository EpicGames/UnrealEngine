// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestSpatialHash.h"

#include "Chaos/SpatialHash.h"
#include "Chaos/Vector.h"
#include "Chaos/UniformGrid.h"

using namespace Chaos;

namespace GeometryCollectionTest
{
	TArray<FVec3> Particles_1000 = {
		FVec3(242.847260, 481.256165, -13.585100),
		FVec3(435.885071, -282.438385, 139.666855),
		FVec3(-445.298431, -480.780701, 249.928925),
		FVec3(-366.115631, 188.174911, -100.105080),
		FVec3(-39.497826, -355.566345, 410.021149),
		FVec3(453.887482, 342.295929, -386.697174),
		FVec3(-166.218842, 235.897385, 48.100235),
		FVec3(-135.756531, -115.386566, 135.813873),
		FVec3(-209.570068, -405.279755, 92.008408),
		FVec3(116.359421, 442.871704, -340.068665),
		FVec3(-199.028992, -455.636810, 149.907349),
		FVec3(392.379395, -68.763680, -214.853882),
		FVec3(424.941986, 118.032127, 241.568237),
		FVec3(-138.925217, 207.964340, -467.774963),
		FVec3(-293.256439, 327.879272, 431.063690),
		FVec3(-443.070923, -242.276810, -466.837311),
		FVec3(-295.278717, 113.062233, 255.074921),
		FVec3(-485.050446, -376.803070, -363.403687),
		FVec3(8.656306, 323.288849, -465.676056),
		FVec3(-331.124115, 318.202667, 352.432983),
		FVec3(219.598160, -427.561310, -292.257568),
		FVec3(-324.289490, -38.059406, 411.193909),
		FVec3(211.905167, -495.017914, -80.945450),
		FVec3(93.131706, 79.776901, -300.230011),
		FVec3(-144.997726, 221.176514, 226.593231),
		FVec3(415.781769, -63.400192, 169.896378),
		FVec3(-330.696533, -489.526794, -400.910492),
		FVec3(251.829620, -31.448917, -382.271606),
		FVec3(-425.152679, -458.614655, 51.982407),
		FVec3(164.195389, 208.738022, -230.793777),
		FVec3(-17.935198, 281.633972, -351.171600),
		FVec3(173.350342, -437.429382, 91.033569),
		FVec3(-70.044777, 193.334167, -457.776947),
		FVec3(-110.712700, 394.487610, -117.631584),
		FVec3(282.877594, -76.428200, -275.897461),
		FVec3(-296.953613, 248.468033, 473.379364),
		FVec3(31.360893, -333.663116, 46.762947),
		FVec3(-198.153778, -445.840729, 79.908501),
		FVec3(128.918915, -295.266418, 314.310822),
		FVec3(316.397064, 492.675201, -22.353359),
		FVec3(-491.591217, 69.599388, 231.100067),
		FVec3(291.210999, 101.786095, 120.066353),
		FVec3(423.432739, -116.051521, 63.189121),
		FVec3(26.807938, -58.308739, 196.850739),
		FVec3(304.386108, 20.512476, -332.404999),
		FVec3(-58.052666, 446.858154, -268.646576),
		FVec3(287.384186, -11.148350, -427.704193),
		FVec3(-323.134003, -476.013550, -239.163696),
		FVec3(-173.272537, 242.319489, -92.413162),
		FVec3(-411.048706, -376.898102, -439.149384),
		FVec3(-50.688236, -39.021629, -265.853241),
		FVec3(-14.189429, 307.057281, -136.264999),
		FVec3(411.916718, -84.504379, 176.262268),
		FVec3(-301.438812, -16.466000, 452.945557),
		FVec3(-388.439606, 173.828278, -251.492279),
		FVec3(81.262894, 270.865540, 79.648582),
		FVec3(269.125061, -165.186569, -418.280579),
		FVec3(330.922028, -214.150269, -428.172791),
		FVec3(-62.023949, 74.801361, 248.813751),
		FVec3(-304.046417, 39.656052, -266.617676),
		FVec3(22.606955, -18.725525, -447.961426),
		FVec3(36.056423, -448.865997, -264.509033),
		FVec3(137.205154, 346.061523, -321.620117),
		FVec3(376.354828, -102.586586, 76.467255),
		FVec3(-366.762238, 97.189377, 107.817734),
		FVec3(115.618156, 320.713226, -234.216919),
		FVec3(76.897949, -153.455826, -250.158554),
		FVec3(-315.094116, 289.712494, -111.678459),
		FVec3(404.938019, 78.903725, 311.296692),
		FVec3(100.629440, -91.474968, 346.249451),
		FVec3(-432.814850, 120.009628, -236.909195),
		FVec3(474.372375, 89.540321, -218.007278),
		FVec3(-323.821411, -233.918503, -91.109077),
		FVec3(483.060181, 203.055328, -205.172928),
		FVec3(-249.791428, 84.899841, 228.451096),
		FVec3(294.685486, 325.919922, -0.660057),
		FVec3(-14.357883, 335.928558, -161.407257),
		FVec3(372.147278, -70.000824, -212.874008),
		FVec3(-327.496155, -64.205551, -426.782471),
		FVec3(436.892456, 172.015228, -332.489990),
		FVec3(234.468964, -140.563171, -356.856842),
		FVec3(393.857178, -456.811340, 215.239670),
		FVec3(-435.665466, 161.061432, 380.296021),
		FVec3(129.007431, -426.722839, 227.838455),
		FVec3(194.553925, -36.660133, -377.745667),
		FVec3(81.091064, -14.494259, -4.698598),
		FVec3(477.823120, -10.339551, -7.439713),
		FVec3(329.294647, 124.934334, -31.539204),
		FVec3(-422.967651, -152.839935, 151.815918),
		FVec3(-308.065582, -433.543488, 131.536697),
		FVec3(-127.482574, 153.704529, 379.379822),
		FVec3(-319.516907, -306.675568, -38.896194),
		FVec3(-496.178192, 362.924774, 445.143158),
		FVec3(-224.624146, -133.979889, 191.335220),
		FVec3(-95.945503, -392.088684, -354.644684),
		FVec3(-495.001221, -203.064224, 341.611084),
		FVec3(215.307480, -8.802383, -109.866089),
		FVec3(182.149399, 338.504395, -108.400612),
		FVec3(-113.992783, -94.048676, 118.602669),
		FVec3(-70.679138, -154.011536, 429.839661),
		FVec3(332.228394, 87.196899, -45.364479),
		FVec3(-175.798126, 103.366310, 385.613434),
		FVec3(-25.403736, -189.473389, 107.609818),
		FVec3(158.140579, 231.748474, -174.921295),
		FVec3(484.330750, -382.601440, 355.660980),
		FVec3(-404.474609, -274.204559, 452.758545),
		FVec3(402.803925, 301.364594, -453.891449),
		FVec3(502.352783, 36.137814, 337.968567),
		FVec3(65.537506, -379.603210, -95.491714),
		FVec3(89.513794, 281.446289, -234.911484),
		FVec3(-102.142372, -439.478943, -174.701263),
		FVec3(-20.548677, 473.107239, -323.945831),
		FVec3(-155.732178, 68.468956, 191.888657),
		FVec3(-246.176056, -293.127228, -238.780548),
		FVec3(-95.137886, -473.429840, -335.837036),
		FVec3(-322.929810, -116.150703, 106.371223),
		FVec3(-151.638062, 294.123230, 165.768921),
		FVec3(-10.748709, 189.459122, -180.566452),
		FVec3(-344.513885, 127.258980, 310.102112),
		FVec3(283.779053, -425.878815, 253.507355),
		FVec3(-18.693506, -423.215637, 241.473343),
		FVec3(35.291294, -463.113495, 182.119644),
		FVec3(303.328033, -484.533508, -263.075684),
		FVec3(87.316788, -224.655762, -103.466148),
		FVec3(212.502106, 291.240143, -37.698509),
		FVec3(-210.226746, -327.624908, 395.187469),
		FVec3(-140.676331, -233.533524, -479.738281),
		FVec3(497.348663, -369.330139, 444.607452),
		FVec3(-238.005859, -224.585403, 4.934497),
		FVec3(45.412025, 32.898960, -293.969574),
		FVec3(98.159065, 9.253073, 4.286590),
		FVec3(319.760895, 224.277390, -338.729858),
		FVec3(-42.885143, 115.941803, -123.498299),
		FVec3(108.643486, 313.689636, 187.122147),
		FVec3(406.882111, -121.451370, -84.784233),
		FVec3(146.114685, 38.404434, 407.395233),
		FVec3(160.098679, 365.562958, 128.340561),
		FVec3(-386.604553, 16.026035, -306.665375),
		FVec3(-460.128113, -274.787689, 313.081543),
		FVec3(-338.667877, -167.673691, 449.656464),
		FVec3(-478.769501, -58.871258, 187.899368),
		FVec3(298.361237, 172.653214, -422.237030),
		FVec3(41.355042, 440.904236, -66.577209),
		FVec3(-265.183990, 229.690628, -382.926849),
		FVec3(-230.065567, 315.269165, 389.309387),
		FVec3(-316.057129, -355.076782, 178.763885),
		FVec3(85.727577, 147.384888, 168.909027),
		FVec3(-47.438934, -140.508743, 389.811249),
		FVec3(-430.916351, 442.169373, -481.091522),
		FVec3(395.342865, 287.928619, 40.688911),
		FVec3(217.577255, -229.733948, 39.387207),
		FVec3(343.270477, 344.142456, 45.235668),
		FVec3(-401.817596, 28.426838, -119.573418),
		FVec3(128.027695, -406.198761, 284.970947),
		FVec3(50.713970, 155.177231, -86.203201),
		FVec3(-402.503448, -423.153809, -344.427216),
		FVec3(-46.463890, 124.469421, 97.708450),
		FVec3(-6.806405, -119.038109, -451.151428),
		FVec3(-1.850709, 375.763916, 56.923214),
		FVec3(-170.375183, -292.288879, 422.587708),
		FVec3(-249.433105, 418.539520, 3.763342),
		FVec3(468.953613, 416.916473, 418.092651),
		FVec3(116.477264, -272.551880, -419.505920),
		FVec3(-438.834564, 441.874298, -209.974182),
		FVec3(179.419418, 468.942902, 178.058395),
		FVec3(494.977844, -347.529480, -234.845490),
		FVec3(-26.445614, 391.839844, -24.952581),
		FVec3(357.358093, 347.280396, -111.459297),
		FVec3(-296.790741, -460.193146, -227.551010),
		FVec3(-319.615082, -463.213837, -440.750702),
		FVec3(206.071945, -241.811203, 289.759521),
		FVec3(-53.932663, 130.412476, 115.283928),
		FVec3(-69.709816, 498.642181, 447.779358),
		FVec3(407.621704, 476.327332, -256.166016),
		FVec3(21.676502, 453.380371, 178.511337),
		FVec3(-290.401917, 355.511597, 382.167267),
		FVec3(504.071777, -360.653137, 229.144562),
		FVec3(-243.678436, 155.493469, 183.954086),
		FVec3(-467.052673, 352.004578, 52.671970),
		FVec3(468.161835, -226.483109, 491.347473),
		FVec3(217.355408, 362.222626, 89.066299),
		FVec3(126.294121, 403.935944, 82.119270),
		FVec3(293.756897, -428.025848, 391.344910),
		FVec3(-325.266998, -277.345398, 286.974884),
		FVec3(-480.968597, 94.638840, 83.035339),
		FVec3(236.918701, 7.528419, -109.665215),
		FVec3(139.164246, -93.484459, -48.898651),
		FVec3(237.845810, -151.939453, 280.384460),
		FVec3(333.581909, 479.386780, -278.951172),
		FVec3(-341.485321, -93.856857, 207.125443),
		FVec3(-86.909576, -188.278214, 143.421204),
		FVec3(-66.238350, -203.608887, -37.058567),
		FVec3(-64.851662, -66.259056, -397.260559),
		FVec3(157.171814, 240.519943, -148.522537),
		FVec3(-306.800476, 116.314789, -444.796082),
		FVec3(-99.924103, 302.389313, 194.601913),
		FVec3(415.608643, -107.468674, 450.370361),
		FVec3(-165.727753, 219.635284, -200.704056),
		FVec3(-263.506897, 386.968872, -347.490540),
		FVec3(255.214966, 499.137726, -262.156403),
		FVec3(346.128967, 359.255890, 0.684109),
		FVec3(-286.785858, -440.778961, 215.371262),
		FVec3(-51.834534, 389.829651, 318.104858),
		FVec3(-97.144279, -466.515533, 252.337067),
		FVec3(-416.057007, -230.970139, 75.810295),
		FVec3(6.348568, 51.064812, -258.457550),
		FVec3(442.648163, -426.150421, 101.559387),
		FVec3(-265.301544, -145.404190, 271.145844),
		FVec3(-96.758461, 78.118881, -384.166840),
		FVec3(421.493286, -229.252975, 269.107758),
		FVec3(-182.264236, -322.529449, -64.740425),
		FVec3(-173.167313, -320.209869, -496.485718),
		FVec3(106.462219, -307.318665, -279.109558),
		FVec3(304.753632, -437.096771, -225.148392),
		FVec3(458.123566, 113.099739, 207.414810),
		FVec3(307.121307, 485.922760, 318.148193),
		FVec3(-452.206238, 322.692352, 375.763641),
		FVec3(263.728455, -362.559357, 400.060303),
		FVec3(-309.801331, -220.855164, 279.631866),
		FVec3(369.716461, -120.790161, 112.604385),
		FVec3(-386.976440, 135.268738, -156.356934),
		FVec3(-285.658813, -202.765656, -255.164474),
		FVec3(420.791290, 258.702515, -423.125671),
		FVec3(-16.065334, -74.793091, 111.739700),
		FVec3(233.123444, -114.529510, 359.088043),
		FVec3(-486.324463, 467.615784, -357.605743),
		FVec3(63.094154, 280.725525, -317.769196),
		FVec3(185.273453, -199.382828, 237.923233),
		FVec3(-71.881752, 244.433304, -101.894142),
		FVec3(-395.057831, -160.960007, -495.831970),
		FVec3(22.908978, 276.191620, 60.804462),
		FVec3(-101.668625, -405.397156, 417.193207),
		FVec3(504.664032, -387.371979, -21.519670),
		FVec3(378.585876, 24.861179, 118.139366),
		FVec3(-329.140381, -56.256165, -10.868471),
		FVec3(-439.060638, 462.255035, 488.677979),
		FVec3(136.785736, 241.239883, -472.301270),
		FVec3(364.087372, 398.172729, -340.828979),
		FVec3(496.814514, 189.570526, -48.344467),
		FVec3(-469.635376, -342.848877, -323.324615),
		FVec3(297.816284, -133.993637, -180.453491),
		FVec3(32.932522, 229.530457, 289.877899),
		FVec3(151.280884, -456.170288, -319.627869),
		FVec3(-420.404480, -293.554932, 270.833588),
		FVec3(-166.668228, 109.309616, 405.425049),
		FVec3(441.952728, 326.790924, 59.650425),
		FVec3(214.444229, 332.840210, -372.769104),
		FVec3(489.963867, 255.110016, -415.670074),
		FVec3(466.107330, -350.707520, -219.496033),
		FVec3(-8.776596, -270.673126, -425.700897),
		FVec3(259.316559, 49.319492, -65.606682),
		FVec3(186.495071, -473.441284, 73.194115),
		FVec3(-136.528824, 235.461594, 330.558777),
		FVec3(2.054496, -221.085983, -34.610844),
		FVec3(-208.910797, -484.323486, 99.621880),
		FVec3(-414.959259, 318.349701, 115.878944),
		FVec3(154.392975, 297.179962, -326.410278),
		FVec3(307.997650, -70.161720, -474.591980),
		FVec3(-217.917801, -478.218353, -12.422220),
		FVec3(467.680939, -2.456307, 58.813637),
		FVec3(86.730492, 443.689087, 97.530373),
		FVec3(-397.356812, 340.257629, 155.705902),
		FVec3(69.964935, -70.281136, 419.339935),
		FVec3(183.865814, -364.347839, 335.858521),
		FVec3(-302.469086, 21.223116, -303.115265),
		FVec3(-474.538330, 242.544510, 427.714325),
		FVec3(-431.092438, -78.810555, 189.516037),
		FVec3(6.765585, -67.465637, -115.456863),
		FVec3(-276.002167, -472.678253, -253.048294),
		FVec3(-199.849228, -227.568649, -160.294403),
		FVec3(-108.410561, 44.930061, 423.044373),
		FVec3(-346.054016, 418.617706, -19.676897),
		FVec3(-342.096344, -477.604767, -477.088806),
		FVec3(479.841034, 386.763885, -42.855141),
		FVec3(-104.704468, 74.109421, -157.756973),
		FVec3(191.766357, -450.928650, 10.097423),
		FVec3(435.326416, 109.941628, -81.168564),
		FVec3(342.072723, -276.451630, -209.329865),
		FVec3(-202.457809, -140.131271, -85.280479),
		FVec3(302.150665, 208.333298, 229.783981),
		FVec3(97.332260, 285.877869, 132.989716),
		FVec3(337.881348, 282.068085, -163.986832),
		FVec3(289.349670, 283.998779, -73.085823),
		FVec3(110.647957, -205.171188, -447.227661),
		FVec3(468.525665, 292.997345, 101.583885),
		FVec3(-291.419373, -53.174358, 253.407928),
		FVec3(273.323639, -495.496765, 158.377808),
		FVec3(-270.494995, 434.222015, -237.417007),
		FVec3(194.389664, -381.760742, 4.799733),
		FVec3(433.793304, 127.222366, -91.119720),
		FVec3(417.663879, -82.795349, -385.559875),
		FVec3(401.849792, -412.919983, 294.427124),
		FVec3(254.382568, 390.357788, -143.784927),
		FVec3(-241.637207, -393.453949, 0.316825),
		FVec3(-179.876526, -43.772667, -200.043716),
		FVec3(-442.323883, -441.111206, -10.510733),
		FVec3(-240.411072, -155.672272, -47.652023),
		FVec3(497.738770, -212.067123, 401.503906),
		FVec3(402.815002, -143.715179, -188.653061),
		FVec3(-489.982269, 316.128601, 345.323730),
		FVec3(300.345062, -155.779419, 204.738510),
		FVec3(77.386360, -36.566715, 102.221626),
		FVec3(-335.144714, 280.103424, -292.268463),
		FVec3(-186.268845, -359.714447, 255.017151),
		FVec3(-482.140747, -277.856232, -99.578735),
		FVec3(-25.758312, 457.362823, 445.665161),
		FVec3(240.899567, -426.061676, -260.382721),
		FVec3(55.926075, -427.102051, -359.296631),
		FVec3(158.620926, -267.245697, -235.343918),
		FVec3(245.058151, -199.262283, 225.736847),
		FVec3(419.409363, 173.585281, 498.865143),
		FVec3(-412.597717, 90.972641, 317.952545),
		FVec3(-62.846321, 451.261017, -458.941254),
		FVec3(-2.112951, 131.697693, -266.969543),
		FVec3(-71.756104, 197.926178, -429.611969),
		FVec3(11.005249, -204.399704, -209.509171),
		FVec3(212.048920, -94.764648, 47.061363),
		FVec3(41.838615, 315.251892, 358.783783),
		FVec3(64.526154, 282.856964, 364.213318),
		FVec3(-382.895752, -74.937675, -170.513199),
		FVec3(-229.276535, 214.105637, -39.135128),
		FVec3(75.192513, -194.803925, 495.327209),
		FVec3(354.394287, 92.570404, 483.486603),
		FVec3(-244.930145, 285.403839, 234.446716),
		FVec3(275.039795, 364.044403, 393.990021),
		FVec3(58.853001, -418.431580, 365.284119),
		FVec3(48.189995, 429.757599, 459.396606),
		FVec3(-330.372833, 186.196960, 256.824799),
		FVec3(-166.618195, -74.863319, 187.036301),
		FVec3(345.432831, -357.747986, 471.830658),
		FVec3(-412.335114, 164.854645, 291.070343),
		FVec3(-159.294495, -422.622314, -274.262207),
		FVec3(460.284729, -490.375885, 370.148071),
		FVec3(-495.377197, -483.675629, 125.418221),
		FVec3(312.226471, -382.799805, -271.448212),
		FVec3(231.756332, 343.922577, -237.322403),
		FVec3(475.138306, -281.960419, -257.061768),
		FVec3(109.477234, 137.296082, 328.897614),
		FVec3(75.855858, -389.354401, -165.045700),
		FVec3(-365.439606, 409.793213, -344.337372),
		FVec3(190.476425, -101.289902, -414.852570),
		FVec3(279.838470, 233.962219, -253.028793),
		FVec3(494.554565, -19.035469, -454.912781),
		FVec3(454.232025, 35.941002, -190.781250),
		FVec3(-56.776436, -279.633240, 189.594269),
		FVec3(310.733643, 398.778229, 115.157578),
		FVec3(-263.777832, 172.952682, 282.845673),
		FVec3(186.960114, 100.191246, -10.737085),
		FVec3(42.900192, -386.867004, -338.782196),
		FVec3(-478.718567, 366.690582, -20.681892),
		FVec3(-472.947754, 143.301193, 470.928955),
		FVec3(359.031982, 460.143951, 15.223279),
		FVec3(154.614395, 25.273403, 294.822174),
		FVec3(-241.822739, -151.170242, 406.183472),
		FVec3(55.765213, -299.225281, 160.921265),
		FVec3(-301.307129, -37.103012, -360.608032),
		FVec3(305.408875, -40.611671, 76.352386),
		FVec3(-497.209076, -469.388916, -459.909882),
		FVec3(286.822144, -57.591499, 389.806213),
		FVec3(162.141525, -503.654846, -275.600830),
		FVec3(-182.711426, 315.135590, 404.320831),
		FVec3(337.266113, 51.407089, 262.918945),
		FVec3(-406.571289, -117.127121, 70.271652),
		FVec3(114.971306, 195.756882, 243.510223),
		FVec3(-362.377289, -262.646118, 91.277992),
		FVec3(-392.248230, -211.562515, 154.876434),
		FVec3(-292.514435, -322.307587, -351.594696),
		FVec3(-182.941010, 423.471985, -404.903961),
		FVec3(165.530823, -206.140228, -298.378845),
		FVec3(-416.617889, -257.617523, 199.112396),
		FVec3(436.571655, -452.221466, -363.210236),
		FVec3(-389.141083, -448.004852, 467.495148),
		FVec3(-208.353821, 1.455436, -61.053047),
		FVec3(-348.666504, -360.129059, 108.620987),
		FVec3(322.255554, 224.415009, 129.610779),
		FVec3(148.647110, -171.054153, 380.197510),
		FVec3(75.727539, -392.240784, 178.413345),
		FVec3(461.597107, 276.866791, 259.316498),
		FVec3(-188.851700, -506.120392, -319.079224),
		FVec3(-289.873566, -213.433243, -476.541992),
		FVec3(-429.349792, -436.986786, -341.291595),
		FVec3(-450.236450, 293.807220, -95.385590),
		FVec3(-148.397507, -481.259491, 60.426525),
		FVec3(-38.914791, -320.977356, 239.912582),
		FVec3(-236.772995, -13.772430, 18.371197),
		FVec3(-154.725891, -247.160095, -457.026764),
		FVec3(463.904083, 491.605286, -375.312134),
		FVec3(-212.990738, 281.237244, 101.881554),
		FVec3(323.667542, -288.400696, 309.234619),
		FVec3(-326.013184, 373.602356, -51.600929),
		FVec3(-190.772171, 474.976013, -433.544464),
		FVec3(-333.529053, 315.955322, -257.584686),
		FVec3(134.168442, -207.142593, -237.205246),
		FVec3(296.645691, 365.430115, -112.621956),
		FVec3(-468.717041, 170.278976, 195.752731),
		FVec3(-438.273071, -493.193787, -182.837982),
		FVec3(231.807053, 223.102142, 399.603668),
		FVec3(-375.413666, 197.269531, 313.079163),
		FVec3(-237.321411, 109.774902, -339.539185),
		FVec3(-166.832520, 96.780251, 359.589966),
		FVec3(-267.849762, -280.576569, 475.385895),
		FVec3(-252.170349, -314.955566, 194.826065),
		FVec3(-273.626068, -25.612484, 429.888763),
		FVec3(-245.710617, -31.107174, 359.815826),
		FVec3(338.739014, -305.491547, -372.876892),
		FVec3(347.317963, -343.655151, -226.199097),
		FVec3(289.561157, 391.568512, -474.444702),
		FVec3(220.610565, -88.228981, -361.703247),
		FVec3(-32.137150, 279.242523, 198.910324),
		FVec3(-8.652587, 364.053284, 238.536423),
		FVec3(40.961662, -460.244446, -364.799927),
		FVec3(361.533600, 22.532135, -321.670197),
		FVec3(171.925217, 220.863037, -46.238152),
		FVec3(463.140991, 8.700814, -463.133301),
		FVec3(-59.747097, 281.821930, 428.465240),
		FVec3(257.694885, 466.107269, 318.531647),
		FVec3(459.052460, -122.432251, 258.799042),
		FVec3(-206.521057, -310.223267, 42.945801),
		FVec3(-407.930511, 132.792847, 301.889709),
		FVec3(-39.667675, -282.247284, 349.816925),
		FVec3(-465.366119, 352.759094, -173.244797),
		FVec3(-413.010193, -477.918304, 355.389771),
		FVec3(-450.918243, -48.948486, 261.834137),
		FVec3(437.413147, -346.686432, -68.407776),
		FVec3(470.035217, -269.245605, -108.699028),
		FVec3(-35.032028, 135.417145, 257.586334),
		FVec3(75.725121, -276.935791, -326.210388),
		FVec3(-304.540161, 423.531372, 42.857964),
		FVec3(-345.388794, 171.240051, -295.388153),
		FVec3(388.476990, 106.416595, 346.194427),
		FVec3(-485.112671, -156.939224, -200.522675),
		FVec3(-437.545441, 27.217869, 498.712799),
		FVec3(204.316681, -224.850067, -339.840942),
		FVec3(-151.853455, 248.350433, -187.171616),
		FVec3(-307.778015, 163.964737, -86.852859),
		FVec3(-205.014755, -419.126160, -385.212982),
		FVec3(227.781250, -349.337646, 137.707748),
		FVec3(-10.358829, -194.446289, 338.442078),
		FVec3(-417.454987, -191.506866, 42.314259),
		FVec3(82.920601, -52.581219, -3.668561),
		FVec3(-274.859253, 197.977814, 290.917664),
		FVec3(-4.264119, 305.476501, 124.254005),
		FVec3(129.336411, 183.637756, 481.332550),
		FVec3(173.257721, 443.100189, 135.882431),
		FVec3(-87.686172, 9.278550, 95.688065),
		FVec3(61.007362, -356.733215, 369.337708),
		FVec3(1.323771, 453.525726, 54.349140),
		FVec3(-16.970961, -330.322845, 438.671906),
		FVec3(138.369293, -393.005615, -467.887878),
		FVec3(-427.099731, -227.800430, 254.199707),
		FVec3(-281.752869, 169.402893, -391.806885),
		FVec3(-142.945038, 316.538391, 9.495449),
		FVec3(-72.653976, -399.100006, 410.704193),
		FVec3(231.520157, -316.594879, -272.584106),
		FVec3(429.107941, 117.408150, -313.621460),
		FVec3(452.603210, -146.106125, -427.421448),
		FVec3(-375.593292, 94.462799, 53.581905),
		FVec3(-55.000862, 72.902519, -214.643555),
		FVec3(-248.467163, 87.782814, 257.279053),
		FVec3(58.364777, 1.555672, -432.226685),
		FVec3(179.784897, 296.020782, -493.251495),
		FVec3(133.515884, 364.398621, 153.062073),
		FVec3(-121.021133, 115.270088, 429.297211),
		FVec3(437.074677, 340.916077, 321.067566),
		FVec3(-333.861237, -36.074306, 279.406219),
		FVec3(-290.295868, 300.841583, 455.068359),
		FVec3(-124.641281, 30.949348, -411.464447),
		FVec3(133.962753, -440.555389, -299.577881),
		FVec3(396.931213, -245.877213, -190.799362),
		FVec3(356.561005, 23.755669, -227.658783),
		FVec3(-474.655884, 385.636536, -66.421341),
		FVec3(139.589798, 69.430588, 170.049866),
		FVec3(-180.300003, -298.759399, 258.781311),
		FVec3(20.341511, -83.750069, 375.604584),
		FVec3(-399.209808, 206.307449, -457.492279),
		FVec3(-156.781555, -425.006805, 247.062256),
		FVec3(-358.142517, 323.665619, 221.020615),
		FVec3(-421.605438, -109.430000, 48.458839),
		FVec3(196.358795, 387.362946, 200.109055),
		FVec3(148.137253, -442.203552, 455.346405),
		FVec3(-372.717529, -181.955505, -73.061211),
		FVec3(290.558380, 108.710373, -495.497101),
		FVec3(-31.508862, 83.961823, -477.688690),
		FVec3(-292.825073, 401.452118, 40.023769),
		FVec3(305.654327, 344.614014, -484.734528),
		FVec3(-96.112999, 358.886139, 343.926056),
		FVec3(-77.975006, -200.611725, 436.614349),
		FVec3(-380.808777, -163.183243, -320.000305),
		FVec3(62.807953, -454.936523, 7.864151),
		FVec3(-187.136566, 436.261383, -459.129639),
		FVec3(319.254303, -140.143600, 260.507874),
		FVec3(364.995758, 234.210785, -314.793121),
		FVec3(273.834473, 228.884338, -373.565582),
		FVec3(-114.984818, -150.249619, 97.978058),
		FVec3(-248.512909, -421.554565, 213.249817),
		FVec3(-492.481293, 469.495331, -255.983932),
		FVec3(-16.879589, 12.693143, 457.127899),
		FVec3(-53.525154, -326.802917, 24.634113),
		FVec3(-334.570618, -152.623810, 437.137939),
		FVec3(90.214264, 175.983749, -8.492594),
		FVec3(31.551510, 428.279022, -300.236816),
		FVec3(317.410583, 254.281158, -45.379452),
		FVec3(373.628571, 191.565323, -198.368378),
		FVec3(64.589760, 398.502136, 497.781342),
		FVec3(-0.037627, 84.928665, 97.766304),
		FVec3(-8.477995, -301.029846, 501.411163),
		FVec3(-280.640747, 77.739326, 76.746017),
		FVec3(266.135651, -270.332184, -361.952667),
		FVec3(-383.617249, 148.858490, 182.130127),
		FVec3(403.772430, -251.491165, -221.811371),
		FVec3(-294.830170, -490.427002, -170.026215),
		FVec3(53.154369, 467.205048, -448.689087),
		FVec3(-129.967773, 428.182007, -46.346104),
		FVec3(191.463226, 266.097168, 264.876465),
		FVec3(-437.797546, -450.429962, -259.037445),
		FVec3(-215.436646, -151.502197, 194.260773),
		FVec3(484.741058, -173.245331, 123.784691),
		FVec3(34.323772, -78.265396, 57.515343),
		FVec3(370.565460, 36.129570, 70.543785),
		FVec3(180.909195, -214.449432, -177.712982),
		FVec3(285.844330, -316.742218, 215.586060),
		FVec3(35.213249, -146.679428, -493.202515),
		FVec3(233.160828, -256.384430, 194.695938),
		FVec3(-88.168633, 93.744286, -437.774414),
		FVec3(271.689056, 471.789734, 45.359272),
		FVec3(317.096863, 264.992401, -403.157959),
		FVec3(-402.694550, 325.856262, 478.495819),
		FVec3(316.440643, -246.744217, -93.578743),
		FVec3(-395.915802, -468.904175, 130.210083),
		FVec3(42.076511, -406.658875, -341.539703),
		FVec3(36.240406, 141.743973, -193.116684),
		FVec3(-152.039856, 77.129044, 398.644958),
		FVec3(-355.971039, 425.330536, 460.346649),
		FVec3(60.770130, -395.947510, 147.437332),
		FVec3(-214.917877, 59.491798, -444.995972),
		FVec3(140.642242, -59.842747, -38.536938),
		FVec3(188.785370, -317.715149, 55.586563),
		FVec3(295.941864, -399.058563, -309.803619),
		FVec3(-112.782791, -464.218567, -28.075764),
		FVec3(-82.076530, 38.060524, -20.929352),
		FVec3(-469.907013, -281.681427, -78.295082),
		FVec3(119.298927, -250.638474, -85.414314),
		FVec3(-128.127975, 191.990936, 61.930561),
		FVec3(239.815323, 462.536224, -400.664368),
		FVec3(-287.905029, 272.487396, -464.064850),
		FVec3(-494.923065, -497.227448, -166.287704),
		FVec3(18.459965, -105.025818, 208.102951),
		FVec3(-53.974419, 369.376190, -311.218903),
		FVec3(-153.935562, -369.343689, 26.955284),
		FVec3(146.761658, 265.337189, -393.263428),
		FVec3(-146.298996, -136.789230, -325.342255),
		FVec3(-100.364998, 41.565681, 457.100800),
		FVec3(277.951416, -408.010315, 407.601990),
		FVec3(147.904114, -311.830902, -218.217270),
		FVec3(466.625061, 323.143860, 439.660309),
		FVec3(-318.369232, 293.648956, -205.911560),
		FVec3(-23.487226, 264.688660, -465.763947),
		FVec3(-220.470505, -491.979248, 177.967453),
		FVec3(140.845749, 402.280426, 62.372787),
		FVec3(17.263393, -441.855255, 491.291290),
		FVec3(214.466415, -32.933830, 294.420410),
		FVec3(-346.603973, 9.985657, 60.781410),
		FVec3(-195.714737, 199.831345, -430.127625),
		FVec3(167.359177, -508.080200, 183.680038),
		FVec3(-373.838013, 401.451385, 314.726532),
		FVec3(41.688915, -286.448090, 122.907547),
		FVec3(-382.816803, 425.577545, -39.088512),
		FVec3(240.300323, 36.714249, -103.350105),
		FVec3(23.932772, 139.608200, -13.257769),
		FVec3(-21.746332, 276.681641, 79.779503),
		FVec3(-283.722321, -67.413910, -375.658569),
		FVec3(-271.633575, -485.069733, -366.141876),
		FVec3(-348.512238, 47.829094, -129.952454),
		FVec3(-252.011780, 158.338440, -257.046661),
		FVec3(108.062462, -496.475952, -461.153839),
		FVec3(262.097015, -56.576897, -440.833496),
		FVec3(305.524475, -230.071289, 331.628082),
		FVec3(-22.767382, -439.109314, 408.974548),
		FVec3(141.733932, 246.540741, 311.647552),
		FVec3(-264.110657, 381.439331, -446.906342),
		FVec3(-275.137939, 88.810036, -264.103394),
		FVec3(-53.893646, 18.765507, -275.342865),
		FVec3(-41.145519, 198.195770, 86.848518),
		FVec3(494.006226, -439.392975, -332.041748),
		FVec3(196.101349, 47.783283, 449.556702),
		FVec3(499.082855, -187.807938, 393.422974),
		FVec3(453.721741, 91.746445, -412.286835),
		FVec3(200.884109, 219.703476, -195.398315),
		FVec3(-176.367020, -146.422821, 254.175781),
		FVec3(-292.394165, -405.298157, -189.712097),
		FVec3(-235.512955, 37.762169, -413.338989),
		FVec3(375.974091, -172.405563, 394.345856),
		FVec3(-475.005432, 114.267296, 8.441391),
		FVec3(53.267559, -0.840073, -392.984711),
		FVec3(461.063263, 253.957397, -328.321167),
		FVec3(311.979065, -311.673889, -437.875366),
		FVec3(397.504364, -382.269562, -348.164490),
		FVec3(-412.530670, 487.768219, -494.902313),
		FVec3(-12.422531, -108.480331, 389.718872),
		FVec3(-498.177856, -10.606319, -468.527710),
		FVec3(461.339600, -185.411560, -97.692200),
		FVec3(-325.283905, -486.712250, 226.168793),
		FVec3(30.354443, 204.111557, -467.552856),
		FVec3(325.571350, 322.428223, 487.607727),
		FVec3(250.228699, -267.591461, -280.306427),
		FVec3(110.416611, -422.861145, 93.341728),
		FVec3(-108.744652, -75.164307, -27.947912),
		FVec3(-410.901947, 29.794373, -453.682526),
		FVec3(227.952576, -149.207458, 97.670715),
		FVec3(-217.920776, -446.394104, -330.747803),
		FVec3(-493.794678, -136.821960, 89.575592),
		FVec3(29.973211, -467.271637, 299.598663),
		FVec3(-289.283325, 3.323531, 329.302887),
		FVec3(-315.627838, 22.672043, -19.657959),
		FVec3(290.506439, -245.193787, -283.277618),
		FVec3(82.247894, 152.804947, 232.322479),
		FVec3(-460.359863, 120.355774, 287.276947),
		FVec3(-153.289673, -83.147911, -156.624023),
		FVec3(90.157425, -19.796318, 273.277435),
		FVec3(477.792419, -236.119202, 229.204010),
		FVec3(-256.717163, -482.556976, 73.805763),
		FVec3(193.965469, 335.502594, 407.249084),
		FVec3(-317.301361, -150.861511, -267.027252),
		FVec3(-234.212891, 169.193054, 12.468929),
		FVec3(-122.958778, 184.225220, -281.832764),
		FVec3(172.889267, -244.481308, -13.491559),
		FVec3(-455.732452, 49.398464, -487.256653),
		FVec3(-159.274445, 459.628967, -212.599030),
		FVec3(-387.381226, -152.931030, -90.010475),
		FVec3(307.135651, -367.885498, -283.762726),
		FVec3(328.798981, -28.141937, 463.444977),
		FVec3(380.734222, -358.904053, 189.726608),
		FVec3(468.789429, 147.742126, -324.019531),
		FVec3(42.212719, -194.596542, -74.217674),
		FVec3(59.225803, 408.615936, -400.222412),
		FVec3(370.151001, -91.616234, 261.225952),
		FVec3(-470.818481, -468.467255, 78.603943),
		FVec3(138.056259, 381.496429, 343.030334),
		FVec3(193.678894, 455.798859, -211.394592),
		FVec3(-229.254257, 221.660156, 389.758728),
		FVec3(-5.426409, -338.579559, 312.063416),
		FVec3(257.437988, -221.585373, 387.992676),
		FVec3(252.890198, 253.480331, -242.888504),
		FVec3(209.788422, -122.203262, -205.307999),
		FVec3(-318.215576, -212.441605, -161.507248),
		FVec3(-126.018051, -14.938378, -1.914866),
		FVec3(459.003052, -27.093025, -119.271439),
		FVec3(-135.497726, -67.959251, -299.740143),
		FVec3(-80.400352, 405.449738, -25.305824),
		FVec3(331.517822, 394.212524, 247.202957),
		FVec3(-200.813217, -309.824463, -405.956848),
		FVec3(-441.577362, -177.000031, 187.491730),
		FVec3(74.007347, 374.367279, -95.020447),
		FVec3(121.376175, -351.564667, -99.246315),
		FVec3(-262.610840, 30.166679, 29.652109),
		FVec3(-314.689545, 497.851013, -483.978546),
		FVec3(313.566833, 81.872368, 127.936455),
		FVec3(-188.062775, -27.774809, 115.440559),
		FVec3(-341.138977, 357.470673, -145.548004),
		FVec3(486.098114, 274.056396, 222.557999),
		FVec3(-72.550583, 344.608856, 105.476044),
		FVec3(264.338104, 74.919327, -229.340485),
		FVec3(-175.829681, -369.921143, -270.168060),
		FVec3(-439.134644, -89.340073, 31.625027),
		FVec3(453.240509, -441.942139, 95.360649),
		FVec3(154.489624, -154.061584, 400.117432),
		FVec3(-313.318878, -33.365116, 106.443497),
		FVec3(330.858704, -237.466278, 123.621101),
		FVec3(-45.459476, 78.616837, -341.070862),
		FVec3(441.839722, 32.821724, -157.368378),
		FVec3(109.813148, -52.941776, 239.197540),
		FVec3(427.729797, 150.852295, -305.669281),
		FVec3(352.765961, -344.256653, -172.584900),
		FVec3(220.896759, -161.305695, 178.608856),
		FVec3(448.311707, 38.369102, 399.040466),
		FVec3(472.728241, -392.619904, 241.380356),
		FVec3(368.655579, -266.348083, 167.329361),
		FVec3(-123.666168, 378.069885, -89.481934),
		FVec3(-336.294861, 51.444096, -87.166710),
		FVec3(-369.083496, -458.447845, 98.342430),
		FVec3(-55.750771, 339.577606, -408.659973),
		FVec3(-482.072876, 150.491394, 397.955597),
		FVec3(409.708344, -323.285004, 156.616943),
		FVec3(-314.299561, -338.588318, -299.799225),
		FVec3(-192.229752, 106.959435, 207.915497),
		FVec3(-452.993591, 259.077301, -69.118973),
		FVec3(-197.157608, 70.561081, 64.728676),
		FVec3(-114.701218, 134.810394, -192.752838),
		FVec3(153.844879, 24.124008, 162.939880),
		FVec3(67.719635, -194.299057, -166.483276),
		FVec3(1.335489, 359.135803, 46.276474),
		FVec3(424.380859, -412.495636, -53.856392),
		FVec3(-139.507996, 313.362976, 41.137463),
		FVec3(214.556305, -439.377991, 480.978149),
		FVec3(-158.288101, -315.038330, -205.470322),
		FVec3(379.632813, 49.237480, 417.151611),
		FVec3(202.860367, -314.930786, -198.860855),
		FVec3(-317.129272, -133.717133, 28.658985),
		FVec3(63.951645, -112.618736, -89.152397),
		FVec3(17.702831, 234.660385, 180.245789),
		FVec3(-377.576660, -24.887493, -469.610931),
		FVec3(345.185242, 213.848083, -256.143799),
		FVec3(142.740067, 87.932396, -289.893524),
		FVec3(-22.680691, -423.115723, -375.482117),
		FVec3(129.229919, -20.208080, -435.842743),
		FVec3(394.180267, 183.998291, 237.897949),
		FVec3(-156.247345, -345.617188, -184.086685),
		FVec3(364.063629, 24.309664, -5.958552),
		FVec3(188.311218, -466.302551, 434.813477),
		FVec3(-469.209778, -112.618576, -184.919434),
		FVec3(-451.891876, 497.062286, -135.598953),
		FVec3(378.478271, -145.985580, -348.906616),
		FVec3(219.920868, -29.429941, -389.540527),
		FVec3(-254.188583, -402.559814, 317.781769),
		FVec3(27.369122, -331.625244, 274.275543),
		FVec3(-217.105591, 367.324432, -38.490540),
		FVec3(203.528549, -139.666687, -314.880646),
		FVec3(154.908478, -25.651628, -420.904816),
		FVec3(-469.438873, 87.876419, 66.652931),
		FVec3(-333.965698, 360.856689, 72.119110),
		FVec3(488.991943, -214.173782, 419.365845),
		FVec3(-277.820953, -254.352295, -91.200630),
		FVec3(439.502411, 26.660156, 426.572296),
		FVec3(468.905945, -11.083944, 155.270737),
		FVec3(-495.571320, 387.675476, 462.737518),
		FVec3(-186.521759, 101.496666, -261.693604),
		FVec3(90.920319, 71.203964, 109.361191),
		FVec3(17.147041, 423.844116, -181.388031),
		FVec3(369.990997, -292.010284, 288.890625),
		FVec3(60.613506, 159.628769, -103.001495),
		FVec3(-257.314941, -166.768173, -243.437576),
		FVec3(77.085098, 3.844237, 419.141571),
		FVec3(-458.424011, 453.982086, 335.668457),
		FVec3(303.147461, -477.858582, -135.595367),
		FVec3(-222.218338, 123.374260, -54.167019),
		FVec3(-494.821838, -138.728119, 258.205933),
		FVec3(-136.733047, -466.762512, -186.554367),
		FVec3(204.075165, -192.734528, 433.739014),
		FVec3(-319.658051, 292.774658, 32.510475),
		FVec3(12.988228, 32.229332, -259.525146),
		FVec3(-204.495483, 113.742813, 72.229324),
		FVec3(-313.391907, 268.926453, 237.994965),
		FVec3(242.856934, -416.193878, 40.125309),
		FVec3(-165.330872, -395.039185, 418.327393),
		FVec3(132.115372, -386.139526, -370.297211),
		FVec3(-407.942047, 387.773682, -38.150467),
		FVec3(147.654724, -210.536514, -391.827484),
		FVec3(-30.641947, 302.514984, -240.807571),
		FVec3(396.797455, 465.584167, -356.327606),
		FVec3(-305.894348, 64.769714, -49.091156),
		FVec3(250.053772, -158.923706, -142.490967),
		FVec3(-401.443176, 418.603394, 176.916687),
		FVec3(-359.780640, 400.231323, 497.037109),
		FVec3(401.878052, 321.170319, 298.311890),
		FVec3(181.906036, 267.527496, -17.216560),
		FVec3(445.974976, 338.542999, 201.290375),
		FVec3(89.655312, 379.368988, -201.071686),
		FVec3(6.037948, 284.681519, 54.224926),
		FVec3(387.153778, -196.234482, -233.471909),
		FVec3(-217.539261, -418.254059, -285.199127),
		FVec3(303.981140, 27.526760, -62.991222),
		FVec3(-44.497070, 444.238495, 142.227112),
		FVec3(-99.575264, 332.114532, 35.407833),
		FVec3(195.880249, -260.194214, -362.574951),
		FVec3(-310.030914, 3.017864, -272.879150),
		FVec3(309.401825, 131.976395, -162.977493),
		FVec3(177.787186, 331.832581, 483.424805),
		FVec3(111.223648, 171.324600, 22.910141),
		FVec3(-320.300995, 32.750374, 155.601318),
		FVec3(136.154068, -243.289734, 435.059937),
		FVec3(31.219498, -384.007172, -14.303829),
		FVec3(346.465851, 161.306412, 51.708912),
		FVec3(441.630371, -348.427002, -263.559570),
		FVec3(-58.018818, -284.034180, 370.620483),
		FVec3(295.010162, 259.240082, 449.701813),
		FVec3(-202.479752, -105.063347, -493.480896),
		FVec3(-429.426727, 434.960815, 393.926544),
		FVec3(370.617340, -167.715836, -253.901047),
		FVec3(447.859619, 319.031830, 391.865295),
		FVec3(-418.072998, -350.835144, -340.582916),
		FVec3(-71.606926, -98.131294, 311.736633),
		FVec3(-12.847464, -193.450745, 191.561478),
		FVec3(-287.912384, -489.076385, 170.124374),
		FVec3(-468.832947, -202.537277, -48.793205),
		FVec3(-10.518171, -6.994138, -135.136917),
		FVec3(266.009308, -153.235062, -30.693954),
		FVec3(37.256340, 385.157379, 87.387573),
		FVec3(-140.082138, -425.385742, 341.019958),
		FVec3(62.312660, -298.999664, -129.484314),
		FVec3(234.936493, -375.094360, 175.418106),
		FVec3(-497.933197, -132.034790, 32.442165),
		FVec3(110.264862, 182.039871, -406.137421),
		FVec3(91.399551, -66.263954, 47.827297),
		FVec3(43.856106, -75.838997, 365.306824),
		FVec3(-441.755920, 357.432800, 345.410248),
		FVec3(219.876984, 198.773697, 237.144592),
		FVec3(-159.568146, -90.452614, -18.121225),
		FVec3(367.322815, -401.861237, -172.313202),
		FVec3(136.421295, -497.713348, 263.639435),
		FVec3(396.573547, -77.614662, 381.327637),
		FVec3(-98.037285, -392.667755, -204.347610),
		FVec3(-53.931679, 289.289734, 248.584946),
		FVec3(231.024994, 250.363159, -5.442340),
		FVec3(-402.099304, 243.218750, 361.723175),
		FVec3(-1.935601, 307.764435, -339.409912),
		FVec3(88.147064, -463.239990, 468.840607),
		FVec3(-445.917816, -256.462616, 436.532898),
		FVec3(485.821045, 273.518890, 78.126137),
		FVec3(258.126770, 305.317322, 50.478436),
		FVec3(346.955383, 222.997009, -72.118675),
		FVec3(35.718273, -279.336914, -245.215576),
		FVec3(17.232561, 221.282135, 280.632568),
		FVec3(-246.400085, 304.329712, 486.540649),
		FVec3(-418.094635, 188.411026, -260.046844),
		FVec3(486.912903, 44.777508, 443.996185),
		FVec3(17.106686, 218.798126, 386.416229),
		FVec3(-199.415588, -120.824661, -229.835831),
		FVec3(-142.555557, -216.238892, 442.719879),
		FVec3(329.952850, 289.765747, 183.226959),
		FVec3(337.699524, -415.055847, 380.550476),
		FVec3(59.022850, -231.557846, 410.690521),
		FVec3(77.245583, -351.342957, 472.001251),
		FVec3(271.513184, 475.373505, -70.126228),
		FVec3(425.505096, -439.727844, -181.198532),
		FVec3(-192.034790, -414.214447, 324.280182),
		FVec3(423.361084, 340.175171, -81.980225),
		FVec3(-325.619263, -157.283859, 259.649689),
		FVec3(-359.773743, -297.939148, -26.119272),
		FVec3(200.246796, 246.236435, 323.897949),
		FVec3(37.607941, 501.192535, -15.849398),
		FVec3(-209.371429, 330.160065, 373.491333),
		FVec3(70.644516, 442.666077, 72.106834),
		FVec3(-310.009735, -379.638428, -89.026405),
		FVec3(-249.964935, 434.356415, -299.653320),
		FVec3(138.560974, -403.578217, 172.654572),
		FVec3(-194.141479, -397.234161, -2.258315),
		FVec3(304.175171, 101.337479, 357.824341),
		FVec3(332.062500, 435.855438, -331.452698),
		FVec3(-317.252686, 231.718094, -228.020706),
		FVec3(-485.773590, -56.148117, 396.037659),
		FVec3(372.522552, 288.538727, 248.570251),
		FVec3(272.492981, -110.849731, -259.996185),
		FVec3(-260.067932, 174.032272, -312.912628),
		FVec3(-394.346222, -55.168137, -439.820313),
		FVec3(-307.365601, -105.651131, 432.960999),
		FVec3(48.140404, -479.203003, -443.260468),
		FVec3(-330.982269, -367.176636, 114.752853),
		FVec3(397.082916, 244.712494, 127.284767),
		FVec3(343.038757, 18.491329, -69.749680),
		FVec3(263.772644, 405.052185, 9.259210),
		FVec3(-359.616364, -187.057220, -106.649704),
		FVec3(-353.354492, -412.111328, 203.061890),
		FVec3(102.048737, 495.084595, 442.758209),
		FVec3(-456.208130, 31.061054, -7.214787),
		FVec3(-87.615944, -60.893303, 490.913055),
		FVec3(312.474152, 277.960449, 327.582214),
		FVec3(419.000427, -340.344177, -202.477768),
		FVec3(77.631340, -359.942352, 86.030457),
		FVec3(-448.060791, -253.719330, 158.454056),
		FVec3(-323.966583, 254.895660, 498.216370),
		FVec3(-257.010193, -63.992962, -238.286270),
		FVec3(-152.542801, 142.612427, 278.131805),
		FVec3(-290.204071, 17.431198, 455.071289),
		FVec3(-84.454140, -481.627380, -81.845963),
		FVec3(-49.251705, 492.814240, -400.990356),
		FVec3(133.434860, 211.286331, 264.486816),
		FVec3(160.666306, -453.340088, -267.420532),
		FVec3(-102.160011, -81.536636, 435.788422),
		FVec3(-327.481659, 53.605957, -5.233283),
		FVec3(-390.630493, 263.710205, 250.741226),
		FVec3(-212.689941, -290.729736, -452.812622),
		FVec3(323.304108, -189.292877, 282.542725),
		FVec3(200.388931, 88.663307, 264.143341),
		FVec3(395.102051, -342.192535, -432.549072),
		FVec3(95.515923, 104.375679, -226.292114),
		FVec3(-78.489449, 133.073410, 166.310944),
		FVec3(59.702744, -322.175354, 64.118088),
		FVec3(176.845703, -421.282440, -451.053467),
		FVec3(23.051291, -238.145432, 124.031837),
		FVec3(-322.084900, 336.274353, 475.586243),
		FVec3(228.963364, 280.838470, 445.000275),
		FVec3(-67.625320, 43.560417, 162.803314),
		FVec3(313.961578, 144.568924, 27.833916),
		FVec3(-437.552124, 39.851318, 246.364685),
		FVec3(293.991394, 26.638645, -466.483185),
		FVec3(-15.643284, -32.327515, -108.133560),
		FVec3(-174.976273, 282.457123, 49.708904),
		FVec3(-403.294220, 175.853073, -278.381775),
		FVec3(260.787811, -468.081085, 73.374443),
		FVec3(-298.561798, -196.181564, 296.001373),
		FVec3(-108.103348, 341.735809, 404.463043),
		FVec3(280.621155, -392.227814, 274.551453),
		FVec3(48.302628, 274.015106, -217.826645),
		FVec3(198.892838, -469.775574, -230.302353),
		FVec3(146.142456, 13.736415, 198.004166),
		FVec3(68.312065, 412.105286, -43.762749),
		FVec3(414.404816, 439.540619, -112.330765),
		FVec3(-108.059639, -180.368195, -196.176239),
		FVec3(187.269501, 225.511902, 71.477402),
		FVec3(485.541351, 122.598534, 324.112823),
		FVec3(-31.154594, -159.426315, -140.581116),
		FVec3(452.245392, 59.953800, 147.865860),
		FVec3(-358.633209, 37.629639, -147.439911),
		FVec3(363.269043, -186.428619, 180.844971),
		FVec3(-362.337738, -259.660675, -238.166443),
		FVec3(-290.981842, 248.433441, -320.657440),
		FVec3(228.107422, -423.695282, 73.042740),
		FVec3(-178.730164, -220.365707, -27.782463),
		FVec3(-360.515015, 17.043581, 432.639526),
		FVec3(88.342270, 184.069382, 491.987976),
		FVec3(-112.328255, 366.327332, 269.742981),
		FVec3(-93.778374, 263.024628, -443.282806),
		FVec3(-63.793636, -481.682739, 401.505432),
		FVec3(285.116882, -33.987297, 27.663197),
		FVec3(-118.394234, -450.533020, 462.363037),
		FVec3(-330.916351, 17.028519, 351.805328),
		FVec3(386.197723, -264.465179, 262.607361),
		FVec3(276.885010, -183.275894, -265.133789),
		FVec3(-379.101471, 84.408394, -147.433487),
		FVec3(342.329346, 268.923859, 253.932343),
		FVec3(-435.106415, -184.667892, 463.423523),
		FVec3(-100.211296, -182.488327, -477.911987),
		FVec3(432.110748, 298.131561, -231.481735),
		FVec3(366.625366, 466.135040, 240.753647),
		FVec3(-483.727173, -109.737434, 47.036072),
		FVec3(176.910110, 77.377747, 396.179718),
		FVec3(-169.598343, -471.004517, 57.093140),
		FVec3(-168.729630, 230.448380, 106.623581),
		FVec3(-232.800537, 383.889404, -474.429871),
		FVec3(61.024345, -219.489746, 263.532928),
		FVec3(44.017464, -82.668983, -207.454208),
		FVec3(-372.388428, 363.136261, -177.321442),
		FVec3(221.367264, -486.005707, 42.866577),
		FVec3(366.508972, -503.322815, 41.103863),
		FVec3(-425.799866, 335.874146, 31.120129),
		FVec3(-244.356094, -159.913300, -226.389099),
		FVec3(-465.340149, 77.238518, -330.249329),
		FVec3(54.811268, -224.414108, 391.816467),
		FVec3(-416.615875, 253.186554, -183.422897),
		FVec3(-471.347626, 111.408249, -131.995667),
		FVec3(-328.718872, -181.054932, 308.038177),
		FVec3(26.176086, -140.261536, -308.605072),
		FVec3(452.054291, -152.797791, 105.357086),
		FVec3(460.707245, -452.626190, -245.706589),
		FVec3(144.994659, -173.984665, 20.164036),
		FVec3(353.056702, 356.093658, -337.417816),
		FVec3(420.200165, -369.286713, 53.554878),
		FVec3(-498.443176, 397.906555, 161.707779),
		FVec3(204.318344, 340.892700, 248.180893),
		FVec3(-103.978828, 79.519547, -176.997665),
		FVec3(300.538391, -106.618523, 376.380005),
		FVec3(-88.699852, -298.883545, 463.044220),
		FVec3(-189.133194, -120.374062, 471.720123),
		FVec3(-232.826477, 191.292572, -386.231079),
		FVec3(-503.215485, 422.156708, 227.584488),
		FVec3(-41.182899, -445.773254, -311.649292),
		FVec3(-170.514404, 504.357971, 228.764755),
		FVec3(215.648605, 395.522705, -1.628571),
		FVec3(-213.455185, -319.496399, 139.415359),
		FVec3(-130.202209, 440.260559, -155.865982),
		FVec3(67.849815, -280.295532, -388.072510),
		FVec3(121.543549, -331.289948, -44.286308),
		FVec3(-317.588348, 407.176056, -57.182713),
		FVec3(-365.286926, 120.554642, 389.323456),
		FVec3(-257.592987, 489.830963, -430.179535),
		FVec3(-137.035355, 156.895401, -78.515114),
		FVec3(252.097168, 391.660858, 476.311401),
		FVec3(-67.612610, -471.811188, -409.967621),
		FVec3(107.611862, -289.668182, 215.391602),
		FVec3(42.352196, 391.627838, -207.757401),
		FVec3(-131.048630, 400.819214, -298.737122),
		FVec3(-33.783573, 291.254639, -107.200447),
		FVec3(245.426544, -13.329924, 49.717831),
		FVec3(-261.175873, -217.770386, -21.352859),
		FVec3(-329.423218, 76.655823, -35.446362),
		FVec3(-374.217041, -359.624146, -497.894775),
		FVec3(99.257133, 494.434082, -181.241455),
		FVec3(-132.451004, 406.858673, -34.047302),
		FVec3(-131.861893, -321.896790, -94.569695),
		FVec3(477.703705, -324.566528, -400.340240),
		FVec3(-356.651123, -208.791153, 307.570404),
		FVec3(412.649048, -390.665863, -121.070847),
		FVec3(256.742371, -60.640480, -494.739807),
		FVec3(489.897217, -288.228210, 456.471680),
		FVec3(-508.189087, 95.922569, -32.517235),
		FVec3(306.148956, 304.306885, -231.333603),
		FVec3(85.316963, -304.270569, 292.722046),
		FVec3(285.805328, -30.749529, 114.658882),
		FVec3(63.009415, -470.683807, -199.886246),
		FVec3(-387.160553, 54.602676, -190.766418),
		FVec3(-176.493454, 415.552216, -223.084976),
		FVec3(216.595490, -177.922791, 68.296440),
		FVec3(308.156647, -350.138519, 148.473419),
		FVec3(-209.073914, 424.546326, -465.904449),
		FVec3(-433.986603, 384.053772, -36.132854),
		FVec3(-478.188690, -490.482056, 117.804314),
		FVec3(-357.817078, 212.460587, -371.945007),
		FVec3(-259.424530, -339.205597, -341.291931),
		FVec3(-359.928253, -54.288857, 190.037048),
		FVec3(321.722168, -401.531952, 301.523346)
	};

	void GetClosestPointsTest1()
	{
		TSpatialHash<FReal> SpatialHash(Particles_1000, 18.0);
		// TSpatialHash<FReal>::Init() Time is 0.000123
		FVec3 Particle_LookUp(122.0, 214.0, 3.0);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints = SpatialHash.GetClosestPoints(Particle_LookUp, 122.0);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000096

		EXPECT_EQ(ClosestPoints.Num(), 7);

		TSet<int32> CPSet(ClosestPoints);
		EXPECT_TRUE(CPSet.Contains(767));
		EXPECT_TRUE(CPSet.Contains(499));
		EXPECT_TRUE(CPSet.Contains(412));
		EXPECT_TRUE(CPSet.Contains(754));
		EXPECT_TRUE(CPSet.Contains(898));
		EXPECT_TRUE(CPSet.Contains(55));
		EXPECT_TRUE(CPSet.Contains(802));

		EXPECT_FALSE(CPSet.Contains(800));
		EXPECT_FALSE(CPSet.Contains(700));
		EXPECT_FALSE(CPSet.Contains(1));
		EXPECT_FALSE(CPSet.Contains(900));
		EXPECT_FALSE(CPSet.Contains(498));

		// ----------------------------------------------------------

		FVec3 Particle_LookUp1(-234.78, -19.31, 444.12);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints1 = SpatialHash.GetClosestPoints(Particle_LookUp1, 199.0);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000121

		EXPECT_EQ(ClosestPoints1.Num(), 24);

		TSet<int32> CPSet1(ClosestPoints1);
		EXPECT_TRUE(CPSet1.Contains(402));
		EXPECT_TRUE(CPSet1.Contains(53));
		EXPECT_TRUE(CPSet1.Contains(862));
		EXPECT_TRUE(CPSet1.Contains(403));
		EXPECT_TRUE(CPSet1.Contains(21));
		EXPECT_TRUE(CPSet1.Contains(844));
		EXPECT_TRUE(CPSet1.Contains(952));
		EXPECT_TRUE(CPSet1.Contains(612));
		EXPECT_TRUE(CPSet1.Contains(908));
		EXPECT_TRUE(CPSet1.Contains(531));
		EXPECT_TRUE(CPSet1.Contains(353));
		EXPECT_TRUE(CPSet1.Contains(915));
		EXPECT_TRUE(CPSet1.Contains(270));
		EXPECT_TRUE(CPSet1.Contains(867));
		EXPECT_TRUE(CPSet1.Contains(551));
		EXPECT_TRUE(CPSet1.Contains(101));
		EXPECT_TRUE(CPSet1.Contains(244));
		EXPECT_TRUE(CPSet1.Contains(399));
		EXPECT_TRUE(CPSet1.Contains(854));
		EXPECT_TRUE(CPSet1.Contains(498));
		EXPECT_TRUE(CPSet1.Contains(462));
		EXPECT_TRUE(CPSet1.Contains(139));
		EXPECT_TRUE(CPSet1.Contains(464));
		EXPECT_TRUE(CPSet1.Contains(963));

		EXPECT_FALSE(CPSet1.Contains(1));

		// ----------------------------------------------------------

		FVec3 Particle_LookUp2(499.0, 222.0, -12.0);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints2 = SpatialHash.GetClosestPoints(Particle_LookUp2, 52.0);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000008

		EXPECT_EQ(ClosestPoints2.Num(), 1);

		TSet<int32> CPSet2(ClosestPoints2);
		EXPECT_TRUE(CPSet2.Contains(238));

		EXPECT_FALSE(CPSet2.Contains(403));

		// ----------------------------------------------------------

		FVec3 Particle_LookUp3(-333.0, -129.0, -489.0);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints3 = SpatialHash.GetClosestPoints(Particle_LookUp3, 122.0);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000035

		EXPECT_EQ(ClosestPoints3.Num(), 5);

		TSet<int32> CPSet3(ClosestPoints3);
		EXPECT_TRUE(CPSet3.Contains(229));
		EXPECT_TRUE(CPSet3.Contains(78));
		EXPECT_TRUE(CPSet3.Contains(379));
		EXPECT_TRUE(CPSet3.Contains(843));
		EXPECT_TRUE(CPSet3.Contains(700));

		EXPECT_FALSE(CPSet3.Contains(1));
	}

	void GetClosestPointsTest2()
	{
		TSpatialHash<FReal> SpatialHash(Particles_1000);
		// TSpatialHash<FReal>::Init() Time is 0.000080
		FVec3 Particle_LookUp(122.0, 214.0, 3.0);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints = SpatialHash.GetClosestPoints(Particle_LookUp, 122.0);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000039

		EXPECT_EQ(ClosestPoints.Num(), 7);

		TSet<int32> CPSet(ClosestPoints);
		EXPECT_TRUE(CPSet.Contains(767));
		EXPECT_TRUE(CPSet.Contains(499));
		EXPECT_TRUE(CPSet.Contains(412));
		EXPECT_TRUE(CPSet.Contains(754));
		EXPECT_TRUE(CPSet.Contains(898));
		EXPECT_TRUE(CPSet.Contains(55));
		EXPECT_TRUE(CPSet.Contains(802));

		EXPECT_FALSE(CPSet.Contains(900));
		EXPECT_FALSE(CPSet.Contains(498));

		// ----------------------------------------------------------

		FVec3 Particle_LookUp1(-234.78, -19.31, 444.12);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints1 = SpatialHash.GetClosestPoints(Particle_LookUp1, 199.0);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000046

		EXPECT_EQ(ClosestPoints1.Num(), 24);

		TSet<int32> CPSet1(ClosestPoints1);
		EXPECT_TRUE(CPSet1.Contains(402));
		EXPECT_TRUE(CPSet1.Contains(53));
		EXPECT_TRUE(CPSet1.Contains(862));
		EXPECT_TRUE(CPSet1.Contains(403));
		EXPECT_TRUE(CPSet1.Contains(21));
		EXPECT_TRUE(CPSet1.Contains(844));
		EXPECT_TRUE(CPSet1.Contains(952));
		EXPECT_TRUE(CPSet1.Contains(612));
		EXPECT_TRUE(CPSet1.Contains(908));
		EXPECT_TRUE(CPSet1.Contains(531));
		EXPECT_TRUE(CPSet1.Contains(353));
		EXPECT_TRUE(CPSet1.Contains(915));
		EXPECT_TRUE(CPSet1.Contains(270));
		EXPECT_TRUE(CPSet1.Contains(867));
		EXPECT_TRUE(CPSet1.Contains(551));
		EXPECT_TRUE(CPSet1.Contains(101));
		EXPECT_TRUE(CPSet1.Contains(244));
		EXPECT_TRUE(CPSet1.Contains(399));
		EXPECT_TRUE(CPSet1.Contains(854));
		EXPECT_TRUE(CPSet1.Contains(498));
		EXPECT_TRUE(CPSet1.Contains(462));
		EXPECT_TRUE(CPSet1.Contains(139));
		EXPECT_TRUE(CPSet1.Contains(464));
		EXPECT_TRUE(CPSet1.Contains(963));

		EXPECT_FALSE(CPSet1.Contains(1));

		// ----------------------------------------------------------

		FVec3 Particle_LookUp2(499.0, 222.0, -12.0);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints2 = SpatialHash.GetClosestPoints(Particle_LookUp2, 52.0);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000003

		EXPECT_EQ(ClosestPoints2.Num(), 1);

		TSet<int32> CPSet2(ClosestPoints2);
		EXPECT_TRUE(CPSet2.Contains(238));

		EXPECT_FALSE(CPSet2.Contains(403));

		// ----------------------------------------------------------

		FVec3 Particle_LookUp3(-333.0, -129.0, -489.0);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints3 = SpatialHash.GetClosestPoints(Particle_LookUp3, 122.0);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000020

		EXPECT_EQ(ClosestPoints3.Num(), 5);

		TSet<int32> CPSet3(ClosestPoints3);
		EXPECT_TRUE(CPSet3.Contains(229));
		EXPECT_TRUE(CPSet3.Contains(78));
		EXPECT_TRUE(CPSet3.Contains(379));
		EXPECT_TRUE(CPSet3.Contains(843));
		EXPECT_TRUE(CPSet3.Contains(700));

		EXPECT_FALSE(CPSet3.Contains(1));

	}

	void GetClosestPointsTest3()
	{
		TSpatialHash<FReal> SpatialHash(Particles_1000);
		// TSpatialHash<FReal>::Init() Time is 0.000080
		FVec3 Particle_LookUp(122.0, 214.0, 3.0);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints = SpatialHash.GetClosestPoints(Particle_LookUp, 122.0, 12);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000050

		EXPECT_EQ(ClosestPoints.Num(), 7);

		EXPECT_EQ(ClosestPoints[0], 767);
		EXPECT_EQ(ClosestPoints[1], 499);
		EXPECT_EQ(ClosestPoints[2], 412);
		EXPECT_EQ(ClosestPoints[3], 754);
		EXPECT_EQ(ClosestPoints[4], 898);
		EXPECT_EQ(ClosestPoints[5], 55);
		EXPECT_EQ(ClosestPoints[6], 802);

		// ----------------------------------------------------------

		FVec3 Particle_LookUp1(-234.78, -19.31, 444.12);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints1 = SpatialHash.GetClosestPoints(Particle_LookUp1, 199.0, 20);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000065

		EXPECT_EQ(ClosestPoints1.Num(), 20);

		EXPECT_EQ(ClosestPoints1[0], 402);
		EXPECT_EQ(ClosestPoints1[1], 53);
		EXPECT_EQ(ClosestPoints1[2], 862);
		EXPECT_EQ(ClosestPoints1[3], 403);
		EXPECT_EQ(ClosestPoints1[4], 21);
		EXPECT_EQ(ClosestPoints1[5], 844);
		EXPECT_EQ(ClosestPoints1[6], 952);
		EXPECT_EQ(ClosestPoints1[7], 612);
		EXPECT_EQ(ClosestPoints1[8], 908);
		EXPECT_EQ(ClosestPoints1[9], 531);
		EXPECT_EQ(ClosestPoints1[10], 353);
		EXPECT_EQ(ClosestPoints1[11], 915);
		EXPECT_EQ(ClosestPoints1[12], 270);
		EXPECT_EQ(ClosestPoints1[13], 867);
		EXPECT_EQ(ClosestPoints1[14], 551);
		EXPECT_EQ(ClosestPoints1[15], 101);
		EXPECT_EQ(ClosestPoints1[16], 244);
		EXPECT_EQ(ClosestPoints1[17], 399);
		EXPECT_EQ(ClosestPoints1[18], 854);
		EXPECT_EQ(ClosestPoints1[19], 498);

		// ----------------------------------------------------------

		FVec3 Particle_LookUp2(499.0, 222.0, -12.0);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints2 = SpatialHash.GetClosestPoints(Particle_LookUp2, 200.0, 5);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000048

		EXPECT_EQ(ClosestPoints2.Num(), 5);

		EXPECT_EQ(ClosestPoints2[0], 238);
		EXPECT_EQ(ClosestPoints2[1], 807);
		EXPECT_EQ(ClosestPoints2[2], 149);
		EXPECT_EQ(ClosestPoints2[3], 284);
		EXPECT_EQ(ClosestPoints2[4], 245);

		// ----------------------------------------------------------

		FVec3 Particle_LookUp3(-333.0, -129.0, -489.0);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints3 = SpatialHash.GetClosestPoints(Particle_LookUp3, 122.0, 12);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000024

		EXPECT_EQ(ClosestPoints3.Num(), 5);

		EXPECT_EQ(ClosestPoints3[0], 229);
		EXPECT_EQ(ClosestPoints3[1], 78);
		EXPECT_EQ(ClosestPoints3[2], 379);
		EXPECT_EQ(ClosestPoints3[3], 843);
		EXPECT_EQ(ClosestPoints3[4], 700);

		// ----------------------------------------------------------

		FVec3 Particle_LookUp4(1.1, 2.2, 3.3);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints4 = SpatialHash.GetClosestPoints(Particle_LookUp4, 250.0, 100000);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000147

		EXPECT_EQ(ClosestPoints4.Num(), 63);

		// ----------------------------------------------------------

		FVec3 Particle_LookUp5(1.1, 2.2, 3.3);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints5 = SpatialHash.GetClosestPoints(Particle_LookUp5, 1250.0, 100000);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000782

		EXPECT_EQ(ClosestPoints5.Num(), 1000);

	}
	
	void GetClosestPointTest()
	{
		TSpatialHash<FReal> SpatialHash(Particles_1000);
		// TSpatialHash<FReal>::Init() Time is 0.000087
		FVec3 Particle_LookUp(0.0, 122.0, -23.0);

		// Get all the points in MaxRadius
		int32 ClosestPoint = SpatialHash.GetClosestPoint(Particle_LookUp);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000005

		EXPECT_EQ(ClosestPoint, 568);

		// ----------------------------------------------------------

		FVec3 Particle_LookUp1(-34.0, -56.0, 444.0);

		// Get all the points in MaxRadius
		int32 ClosestPoint1 = SpatialHash.GetClosestPoint(Particle_LookUp1);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000005

		EXPECT_EQ(ClosestPoint1, 854);

		// ----------------------------------------------------------

		FVec3 Particle_LookUp2(12.34, 437.8, -345.67);

		// Get all the points in MaxRadius
		int32 ClosestPoint2 = SpatialHash.GetClosestPoint(Particle_LookUp2);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000005

		EXPECT_EQ(ClosestPoint2, 500);

		// ----------------------------------------------------------

		FVec3 Particle_LookUp3(10000.0, 10000.0, 10000.0);

		int32 ClosestPoint3 = SpatialHash.GetClosestPoint(Particle_LookUp3);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000005

		EXPECT_EQ(ClosestPoint3, 161);

		// ----------------------------------------------------------

		FVec3 Particle_LookUp4(-1234.0, 0.1234, 1234.0);

		int32 ClosestPoint4 = SpatialHash.GetClosestPoint(Particle_LookUp4);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000005

		EXPECT_EQ(ClosestPoint4, 431);

	}
	
	void HashTableUpdateTest()
	{
		TSpatialHash<FReal> SpatialHash(Particles_1000);
		// TSpatialHash<FReal>::Init() Time is 0.000087
		FVec3 Particle_LookUp(0.0, 122.0, -23.0);

		int32 ClosestPoint = SpatialHash.GetClosestPoint(Particle_LookUp);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000005

		EXPECT_EQ(ClosestPoint, 568);

		SpatialHash.Update(15.0);
		int32 ClosestPoint1 = SpatialHash.GetClosestPoint(Particle_LookUp);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000005

		EXPECT_EQ(ClosestPoint1, 568);

		// ----------------------------------------------------------

		TSpatialHash<FReal> SpatialHash2(Particles_1000);
		FVec3 Particle_LookUp2(-333.0, -129.0, -489.0);

		// Get all the points in MaxRadius
		TArray<int32> ClosestPoints2 = SpatialHash2.GetClosestPoints(Particle_LookUp2, 122.0, 12);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000024

		EXPECT_EQ(ClosestPoints2.Num(), 5);

		EXPECT_EQ(ClosestPoints2[0], 229);
		EXPECT_EQ(ClosestPoints2[1], 78);
		EXPECT_EQ(ClosestPoints2[2], 379);
		EXPECT_EQ(ClosestPoints2[3], 843);
		EXPECT_EQ(ClosestPoints2[4], 700);

		SpatialHash2.Update(35.0);
		TArray<int32> ClosestPoints3 = SpatialHash2.GetClosestPoints(Particle_LookUp2, 122.0, 12);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.000005

		EXPECT_EQ(ClosestPoints3.Num(), 5);

		EXPECT_EQ(ClosestPoints3[0], 229);
		EXPECT_EQ(ClosestPoints3[1], 78);
		EXPECT_EQ(ClosestPoints3[2], 379);
		EXPECT_EQ(ClosestPoints3[3], 843);
		EXPECT_EQ(ClosestPoints3[4], 700);

	}

	void HashTablePressureTest()
	{
		const int32 NUM_PARTICLES = 1000000;
		const FReal BOUNDARY_MIN = -1000.0;
		const FReal BOUNDARY_MAX = 1000.0;
		TArray<FVec3> Particles;
		for (int32 Idx = 0; Idx < NUM_PARTICLES; ++Idx)
		{
			Particles.Add(FVec3(FMath::RandRange(BOUNDARY_MIN, BOUNDARY_MAX), FMath::RandRange(BOUNDARY_MIN, BOUNDARY_MAX), FMath::RandRange(BOUNDARY_MIN, BOUNDARY_MAX)));
		}

		TSpatialHash<FReal> SpatialHash(Particles);
		// TSpatialHash<FReal>::Init() Time is 0.029013
		
		FVec3 Particle_LookUp(0.0, 122.0, -23.0);
		TArray<int32> ClosestPoints = SpatialHash.GetClosestPoints(Particle_LookUp, 500.0, 150);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.073857

		TArray<int32> ClosestPoints2 = SpatialHash.GetClosestPoints(Particle_LookUp, 500.0);
		// TSpatialHash<FReal>::GetClosestPoints() Time is 0.043953

		EXPECT_EQ(ClosestPoints.Num(), 150);

	}
	
}

