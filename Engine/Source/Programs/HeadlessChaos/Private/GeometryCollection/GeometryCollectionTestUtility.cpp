// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestUtility.h"
#include "UObject/UObjectGlobals.h"
#include "GeometryCollectionProxyData.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionSimulationCoreTypes.h"

#include "PBDRigidsSolver.h"
#include "PhysicsProxy/PhysicsProxies.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "../Resource/FracturedGeometry.h"

#include "Chaos/ErrorReporter.h"


namespace GeometryCollectionTest {

	TSharedPtr<FGeometryDynamicCollection> GeometryCollectionToGeometryDynamicCollection(TSharedPtr<const FGeometryCollection> InputCollection, EObjectStateTypeEnum DynamicStateDefault)
	{
		TSharedPtr<FGeometryDynamicCollection> NewCollection(new FGeometryDynamicCollection(InputCollection));
		// TransformAttribute will be initialized by a lazy initializer pattern
		NewCollection->CopyAttribute(*InputCollection, FTransformCollection::ParentAttribute, FGeometryCollection::TransformGroup);
		NewCollection->CopyAttribute(*InputCollection, FTransformCollection::ChildrenAttribute, FGeometryCollection::TransformGroup);
		NewCollection->CopyAttribute(*InputCollection, FGeometryCollection::SimulationTypeAttribute, FGeometryCollection::TransformGroup);
		NewCollection->CopyAttribute(*InputCollection, FGeometryCollection::StatusFlagsAttribute, FGeometryCollection::TransformGroup);

		for (int i = 0; i < NewCollection->NumElements(FTransformCollection::TransformGroup);i++)
		{
			NewCollection->DynamicState[i] = (int32)DynamicStateDefault;
			NewCollection->Active[i] = true ;
		}

		NewCollection->CopyMatchingAttributesFrom(*InputCollection);
		return NewCollection;
	}


	TSharedPtr<FGeometryCollection>	CreateClusteredBody(FVector Position)
	{
		TSharedPtr<FGeometryCollection> RestCollection;
		RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(-1.f, 0.f, 0.f)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(1.f,0.f,0.f)), FVector(1.0)));

		RestCollection->AddElements(1, FGeometryCollection::TransformGroup);
		// @todo(ClusteringUtils) This is a bad assumption, the state flags should be initialized to zero.
		RestCollection->SimulationType[2] = FGeometryCollection::ESimulationTypes::FST_Clustered;

		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 2, { 0,1 });
		RestCollection->Transform[2].SetTranslation(FVector3f(Position));

		return RestCollection;
	}

	TSharedPtr<FGeometryCollection>	CreateClusteredBody_OnePartent_FourBodies(FVector Position)
	{
		TSharedPtr<FGeometryCollection> RestCollection;
		RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(-2.f, 0.f, 0.f)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(2.f, 0.f, 0.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(0.f, 0.f, -2.f)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(0.f, 0.f, 2.f)), FVector(1.0)));

		RestCollection->AddElements(1, FGeometryCollection::TransformGroup);
		// @todo(ClusteringUtils) This is a bad assumption, the state flags should be initialized to zero.
		RestCollection->SimulationType[4] = FGeometryCollection::ESimulationTypes::FST_Clustered;

		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 4, { 0,1,2,3 });
		RestCollection->Transform[4].SetTranslation(FVector3f(Position));

		return RestCollection;
	}

	TSharedPtr<FGeometryCollection>	CreateClusteredBody_TwoParents_TwoBodies(FVector Position)
	{
		TSharedPtr<FGeometryCollection> RestCollection;
		RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), Position), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(30.f)), FVector(1.0)));

		RestCollection->AddElements(2, FGeometryCollection::TransformGroup);

		// @todo(ClusteringUtils) This is a bad assumption, the state flags should be initialized to zero.
		RestCollection->SimulationType[2] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[3] = FGeometryCollection::ESimulationTypes::FST_Clustered;

		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 2, { 0,1 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 3, { 2 });

		return RestCollection;
	}

	TSharedPtr<FGeometryCollection>	CreateClusteredBody_TwoParents_TwoBodiesB(FVector Position)
	{
		TSharedPtr<FGeometryCollection> RestCollection;
		RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(-10.f, 10.f, 0.f)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(10.f,10.f,0.f)), FVector(1.0)));

		RestCollection->AddElements(2, FGeometryCollection::TransformGroup);

		// @todo(ClusteringUtils) This is a bad assumption, the state flags should be initialized to zero.
		RestCollection->SimulationType[2] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[3] = FGeometryCollection::ESimulationTypes::FST_Clustered;

		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 2, { 0,1 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 3, { 2 });
		RestCollection->Transform[2].SetTranslation(FVector3f(0.f, 10.f, 0.f));
		RestCollection->Transform[3].SetTranslation(FVector3f(Position));

		return RestCollection;
	}

	TSharedPtr<FGeometryCollection>	CreateClusteredBody_FourParents_OneBody(FVector Position)
	{
		TSharedPtr<FGeometryCollection> RestCollection;
		RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), Position), FVector(1.0));

		RestCollection->AddElements(4, FGeometryCollection::TransformGroup);

		// @todo(ClusteringUtils) This is a bad assumption, the state flags should be initialized to zero.
		RestCollection->SimulationType[1] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[2] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[3] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[4] = FGeometryCollection::ESimulationTypes::FST_Clustered;

		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 1, { 0 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 2, { 1 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 3, { 2 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 4, { 3 });

		return RestCollection;
	}



	TSharedPtr<FGeometryCollection>	CreateClusteredBody_TwoByTwo_ThreeTransform(FVector Position)
	{
		TSharedPtr<FGeometryCollection> RestCollection;
		RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(0,0,0)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(100,0,0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(200,0,0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(300,0,0)), FVector(1.0)));

		RestCollection->AddElements(3, FGeometryCollection::TransformGroup);
		RestCollection->Transform[6].SetTranslation(FVector3f(Position));

		// @todo(ClusteringUtils) This is a bad assumption, the state flags should be initialized to zero.
		RestCollection->SimulationType[0] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[1] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[2] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[3] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[4] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[5] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[6] = FGeometryCollection::ESimulationTypes::FST_Clustered;

		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 4, { 0,1 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 5, { 2,3 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 6, { 4,5 });

		return RestCollection;
	}



	TSharedPtr<FGeometryCollection>	CreateClusteredBody_ThreeByTwo_ThreeTransform(FVector Position)
	{
		TSharedPtr<FGeometryCollection> RestCollection;
		RestCollection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(0, 0, 0)), FVector(1.0));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(100, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(200, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(300, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(400, 0, 0)), FVector(1.0)));
		RestCollection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f)), FVector(500, 0, 0)), FVector(1.0)));

		RestCollection->AddElements(3, FGeometryCollection::TransformGroup);
		RestCollection->Transform[8].SetTranslation(FVector3f(Position));

		// @todo(ClusteringUtils) This is a bad assumption, the state flags should be initialized to zero.
		RestCollection->SimulationType[0] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[1] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[2] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[3] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[4] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[5] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[6] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[7] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[8] = FGeometryCollection::ESimulationTypes::FST_Clustered;

		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 6, { 0,1,2 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 7, { 3,4,5 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 8, { 6,7 });

		return RestCollection;
	}


	TSharedPtr<FGeometryCollection>	CreateClusteredBody_FracturedGeometry(FVector Position)
	{
		TSharedPtr<FGeometryCollection> RestCollection;

		RestCollection = TSharedPtr<FGeometryCollection>(FGeometryCollection::NewGeometryCollection(
			FracturedGeometry::RawVertexArray,
			FracturedGeometry::RawIndicesArray,
			FracturedGeometry::RawBoneMapArray,
			FracturedGeometry::RawTransformArray,
			FracturedGeometry::RawLevelArray,
			FracturedGeometry::RawParentArray,
			FracturedGeometry::RawChildrenArray,
			FracturedGeometry::RawSimulationTypeArray,
			FracturedGeometry::RawStatusFlagsArray));


		GeometryCollectionAlgo::ReCenterGeometryAroundCentreOfMass(RestCollection.Get(), false);
		TArray<TArray<int32>> ConnectionGraph = RestCollection->ConnectionGraph();


		RestCollection->AddElements(2, FGeometryCollection::TransformGroup);

		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(),11, { 1,2,5,6,7,8,10 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(),12, { 3,4,9 });
		GeometryCollectionAlgo::ParentTransforms(RestCollection.Get(), 0, { 11,12 });

		for (int i = 0; i < RestCollection->NumElements(FGeometryCollection::TransformGroup); i++)
			RestCollection->SimulationType[i] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		RestCollection->SimulationType[11] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[12] = FGeometryCollection::ESimulationTypes::FST_Clustered;
		RestCollection->SimulationType[0] = FGeometryCollection::ESimulationTypes::FST_Clustered;

		return RestCollection;
	}

	void InitMaterialToZero(Chaos::FChaosPhysicsMaterial * PhysicalMaterial)
	{
		PhysicalMaterial->Friction = 0;
		PhysicalMaterial->Restitution = 0;
		PhysicalMaterial->SleepingLinearThreshold = 0;
		PhysicalMaterial->SleepingAngularThreshold = 0;
		PhysicalMaterial->DisabledLinearThreshold = 0;
		PhysicalMaterial->DisabledAngularThreshold = 0;
	}

} // end namespace GeometryCollectionTest

