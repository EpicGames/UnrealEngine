// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionTestVisibility.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/TransformCollection.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "../Resource/FracturedGeometry.h"

namespace GeometryCollectionTest
{
	void TestHideVertices()
	{
		TSharedPtr<FGeometryCollection> Collection = GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.f, 0.f, 0.f)), FVector(1.0));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(1.f, 0.f, 0.f)), FVector(1.0)));
		Collection->AppendGeometry(*GeometryCollection::MakeCubeElement(FTransform(FQuat::MakeFromEuler(FVector(0.f, 0.f, 0.f)), FVector(0.5f, 0.f, 1.f)), FVector(1.0)));

		//  0
		//  ...1
		//  ......2
		(Collection->Parent)[0] = -1;
		(Collection->Children)[0].Add(1);
		(Collection->Parent)[1] = 0;
		(Collection->Children)[1].Add(2);
		(Collection->Parent)[2] = 1;

		FGeometryCollection* Coll = Collection.Get();
		int32 NumFaces = Coll->NumElements(FGeometryCollection::FacesGroup);

		const TManagedArray<FIntVector>& IndicesArray = Coll->Indices;
		const TManagedArray<int32>& BoneMapArray = Coll->BoneMap;
		const TManagedArray<bool>& VisibleArray = Coll->Visible;

		// expect all visible
		for (int32 Idx = 0; Idx < NumFaces; Idx++)
		{
			check(VisibleArray[Idx]);
			EXPECT_TRUE(VisibleArray[Idx]);
		}

		// hide node 1
		TArray<int32> NodeList;
		NodeList.Push(1);
		Coll->UpdateGeometryVisibility(NodeList, false);

		// expect only node 1 is hidden
		for (int32 Idx=0; Idx< NumFaces; Idx++)
		{
			switch (BoneMapArray[IndicesArray[Idx][0]])
			{
			case 0:
				EXPECT_TRUE(VisibleArray[Idx]);
			break;
			case 1:
				EXPECT_FALSE(VisibleArray[Idx]);
			break;
			case 2:
				EXPECT_TRUE(VisibleArray[Idx]);
			break;
			}			
		}

		NodeList.Reset();
		NodeList.Push(1);
		Coll->UpdateGeometryVisibility(NodeList, true);

		NodeList.Reset();
		NodeList.Push(0);
		NodeList.Push(2);
		Coll->UpdateGeometryVisibility(NodeList, false);

		// expect only node 1 is visible
		for (int32 Idx = 0; Idx < NumFaces; Idx++)
		{
			switch (BoneMapArray[IndicesArray[Idx][0]])
			{
			case 0:
				EXPECT_FALSE(VisibleArray[Idx]);
				break;
			case 1:
				EXPECT_TRUE(VisibleArray[Idx]);
				break;
			case 2:
				EXPECT_FALSE(VisibleArray[Idx]);
				break;
			}
		}
	}

}

