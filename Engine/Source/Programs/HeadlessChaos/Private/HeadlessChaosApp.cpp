// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosApp.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosTestBP.h"
#include "HeadlessChaosTestBroadphase.h"
#include "HeadlessChaosTestCloth.h"
#include "HeadlessChaosTestClustering.h"
#include "HeadlessChaosTestCollisions.h"
#include "HeadlessChaosTestForces.h"
#include "HeadlessChaosTestGJK.h"
#include "HeadlessChaosTestImplicits.h"
#include "HeadlessChaosTestRaycast.h"
#include "HeadlessChaosTestSerialization.h"
#include "HeadlessChaosTestSpatialHashing.h"
#include "HeadlessChaosTestTriangleMesh.h"
#include "Modules/ModuleManager.h"
#include "RequiredProgramMainCPPInclude.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/Box.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"
#include "HeadlessChaosTestParticleHandle.h"
#include "HeadlessChaosTestClustering.h"
#include "HeadlessChaosTestSerialization.h"
#include "HeadlessChaosTestBP.h"
#include "HeadlessChaosTestRaycast.h"
#include "HeadlessChaosTestSweep.h"
#include "HeadlessChaosTestOverlap.h"
#include "HeadlessChaosTestGJK.h"
#include "HeadlessChaosTestEPA.h"
#include "HeadlessChaosTestBroadphase.h"
#include "HeadlessChaosTestMostOpposing.h"
#include "HeadlessChaosTestSolverCommandList.h"
#include "HeadlessChaosTestSolverProxies.h"
#include "HeadlessChaosTestHandles.h"
#include "HeadlessChaosCache.h"


#include "GeometryCollection/GeometryCollectionTest.h"
#include "GeometryCollection/GeometryCollectionTestBoneHierarchy.h"
#include "GeometryCollection/GeometryCollectionTestClean.h"
#include "GeometryCollection/GeometryCollectionTestClustering.h"
#include "GeometryCollection/GeometryCollectionTestCollisionResolution.h"
#include "GeometryCollection/GeometryCollectionTestCreation.h"
#include "GeometryCollection/GeometryCollectionTestDecimation.h"
#include "GeometryCollection/GeometryCollectionTestFields.h"
#include "GeometryCollection/GeometryCollectionTestImplicitCapsule.h"
#include "GeometryCollection/GeometryCollectionTestImplicitCylinder.h"
#include "GeometryCollection/GeometryCollectionTestImplicitSphere.h"
#include "GeometryCollection/GeometryCollectionTestInitilization.h"
#include "GeometryCollection/GeometryCollectionTestMassProperties.h"
#include "GeometryCollection/GeometryCollectionTestMatrices.h"
#include "GeometryCollection/GeometryCollectionTestProximity.h"
#include "GeometryCollection/GeometryCollectionTestResponse.h"
#include "GeometryCollection/GeometryCollectionTestSimulation.h"
#include "GeometryCollection/GeometryCollectionTestSimulationField.h"
#include "GeometryCollection/GeometryCollectionTestSimulationSolver.h"
#include "GeometryCollection/GeometryCollectionTestSimulationStreaming.h"
#include "GeometryCollection/GeometryCollectionTestSpatialHash.h"
#include "GeometryCollection/GeometryCollectionTestVisibility.h"
#include "GeometryCollection/GeometryCollectionTestEvents.h"
#include "GeometryCollection/GeometryCollectionTestSerialization.h"

#include "CompGeom/ExactPredicates.h"

IMPLEMENT_APPLICATION(HeadlessChaos, "HeadlessChaos");

#define LOCTEXT_NAMESPACE "HeadlessChaos"

DEFINE_LOG_CATEGORY(LogHeadlessChaos);

TEST(ImplicitTests, Implicit) {
	ChaosTest::ImplicitPlane();
	ChaosTest::ImplicitTetrahedron();
	ChaosTest::ImplicitCube();
	ChaosTest::ImplicitSphere();
	ChaosTest::ImplicitCylinder();
	ChaosTest::ImplicitTaperedCylinder();
	ChaosTest::ImplicitTaperedCapsule();
	ChaosTest::ImplicitCapsule();
	ChaosTest::ImplicitScaled();
	ChaosTest::ImplicitScaled2();
	ChaosTest::ImplicitTransformed();
	ChaosTest::ImplicitIntersection();
	ChaosTest::ImplicitUnion();
	ChaosTest::UpdateImplicitUnion();
	// @todo: Make this work at some point
	//ChaosTest::ImplicitLevelset();

	SUCCEED();
}

TEST(ImplicitTests, Rasterization) {
	ChaosTest::RasterizationImplicit();
	ChaosTest::RasterizationImplicitWithHole();
}

TEST(ImplicitTests, ConvexHull) {
	ChaosTest::ConvexHull();
	ChaosTest::ConvexHull2();
	ChaosTest::Simplify();
}

TEST(CollisionTests, Collisions) {
	GEnsureOnNANDiagnostic = 1;

	ChaosTest::LevelsetConstraint();
	// ChaosTest::LevelsetConstraintGJK();
	ChaosTest::CollisionBoxPlane();
	ChaosTest::CollisionBoxPlaneZeroResitution();
	ChaosTest::CollisionBoxPlaneRestitution();
	ChaosTest::CollisionCubeCubeRestitution();
	ChaosTest::CollisionBoxToStaticBox();
	ChaosTest::CollisionCapsuleCapsule();
	ChaosTest::CollisionConvexConvex();

	// @ todo: Make this work at some point
	//ChaosTest::SpatialHashing();

	SUCCEED();
}

TEST(Clustering, Clustering) {
	ChaosTest::ImplicitCluster();
	ChaosTest::FractureCluster();
	ChaosTest::PartialFractureCluster();
	SUCCEED();
}

TEST(SerializationTests, Serialization) {
	// LWC-TODO : re-enable that when we have proper double serialization in LWC mode
#if 0
	ChaosTest::SimpleTypesSerialization();
#endif
	ChaosTest::SimpleObjectsSerialization();
	ChaosTest::SharedObjectsSerialization();
	ChaosTest::GraphSerialization();
	ChaosTest::ObjectUnionSerialization();
	ChaosTest::ParticleSerialization();
	ChaosTest::BVHSerialization();
	ChaosTest::RigidParticlesSerialization();
	ChaosTest::BVHParticlesSerialization();
	ChaosTest::HeightFieldSerialization();
	SUCCEED();
}

TEST(BroadphaseTests, Broadphase) {
	ChaosTest::BPPerfTest();
	//ChaosTest::SpatialAccelerationDirtyAndGlobalQueryStrestTest();
	SUCCEED();
}

//TEST(ClothTests, DeformableGravity) {
//	ChaosTest::DeformableGravity();
//
//	SUCCEED();
//}
//
//TEST(ClothTests, EdgeConstraints) {
//	ChaosTest::EdgeConstraints();
//
//	SUCCEED();
//}

TEST(ClothTests, ClothCollection) {
	ChaosTest::ClothCollection();
	SUCCEED();
}

TEST(RaycastTests, Raycast) {
	ChaosTest::SphereRaycast();
	ChaosTest::PlaneRaycast();
	//ChaosTest::CylinderRaycast();
	//ChaosTest::TaperedCylinderRaycast();
	ChaosTest::CapsuleRaycast();
	ChaosTest::CapsuleRaycastFastLargeDistance();
	ChaosTest::CapsuleRaycastMissWithEndPointOnBounds();
	ChaosTest::TriangleRaycast();
	ChaosTest::TriangleRaycastDenegerated();
	ChaosTest::BoxRaycast();
	ChaosTest::VectorizedAABBRaycast();
	ChaosTest::ScaledRaycast();
	//ChaosTest::TransformedRaycast();
	//ChaosTest::UnionRaycast();
	//ChaosTest::IntersectionRaycast();
	
	SUCCEED();
}

TEST(SweepTests, Sweep) {
	ChaosTest::CapsuleSweepAgainstTriMeshReal();
	
	SUCCEED();
}

// This test is disabled until we implement  local clipping feature
TEST(SweepTests, DISABLED_LargeSweep)
{
	ChaosTest::GJKLargeDistanceCapsuleSweep();

	SUCCEED();
}

TEST(OverlapTests, Overlap) {
	ChaosTest::OverlapTriMesh();

	SUCCEED();
}

TEST(MostOpposingTests, MostOpposing) {
	ChaosTest::TrimeshMostOpposing();
	ChaosTest::ConvexMostOpposing();
	ChaosTest::ScaledMostOpposing();

	SUCCEED();
}

TEST(GJK, Simplexes) {
	ChaosTest::SimplexLine();
	ChaosTest::SimplexTriangle();
	ChaosTest::SimplexTetrahedron();
	
	SUCCEED();
}

TEST(GJK, GJKIntersectTests) {
	ChaosTest::GJKSphereSphereTest();
	ChaosTest::GJKSphereBoxTest();
	ChaosTest::GJKSphereCapsuleTest();
	ChaosTest::GJKSphereConvexTest();
	ChaosTest::GJKSphereScaledSphereTest();
	
	SUCCEED();
}

TEST(GJK, GJKRaycastTests) {
	ChaosTest::GJKSphereSphereSweep();
	ChaosTest::GJKSphereBoxSweep();
	ChaosTest::GJKSphereCapsuleSweep();
	ChaosTest::GJKSphereConvexSweep();
	ChaosTest::GJKSphereScaledSphereSweep();
	ChaosTest::GJKBoxCapsuleSweep();
	ChaosTest::GJKBoxBoxSweep();
	ChaosTest::GJKCapsuleConvexInitialOverlapSweep();
	ChaosTest::GJKConvexCapsule();
	SUCCEED();
}

TEST(EPA, EPATests) {
	ChaosTest::EPAInitTest();
	ChaosTest::EPASimpleTest();
	SUCCEED();
}

TEST(BP, BroadphaseTests) {
	ChaosTest::AABBTreeDirtyGridFunctionsWithEdgeCase();
	ChaosTest::GridBPTest();
	ChaosTest::GridBPEarlyExitTest();
	ChaosTest::GridBPTest2();
	ChaosTest::AABBTreeTest();
	ChaosTest::AABBTreeTestDynamic();
	ChaosTest::AABBTreeDirtyTreeTest();
	ChaosTest::AABBTreeDirtyGridTest();
	ChaosTest::AABBTreeTimesliceTest();
	ChaosTest::DoForSweepIntersectCellsImpTest();
	ChaosTest::BoundingVolumeNoBoundsTest();
	ChaosTest::BroadphaseCollectionTest();
	SUCCEED();
}

TEST(ParticleHandle, ParticleHandleTests)
{
	ChaosTest::ParticleIteratorTest();
	ChaosTest::ParticleHandleTest();
	ChaosTest::AccelerationStructureHandleComparison();
	ChaosTest::HandleObjectStateChangeTest();
	SUCCEED();
}

TEST(Perf, PerfTests)
{
	ChaosTest::EvolutionPerfHarness();
	SUCCEED();
}

TEST(Handles, FrameworkTests)
{
	ChaosTest::Handles::HandleArrayTest();
	ChaosTest::Handles::HandleHeapTest();
	ChaosTest::Handles::HandleSerializeTest();
	ChaosTest::Handles::HandleArrayIteratorTest();
}

TEST(TriangleMesh, TriangleMeshTests) {
	ChaosTest::TriangleMeshProjectTest();
	SUCCEED();
}

//TEST(Vehicle, VehicleTests) {
//
//	ChaosTest::SystemTemplateTest<float>();
//
//	ChaosTest::AerofoilTestLiftDrag<float>();
//	
//	ChaosTest::TransmissionTestManualGearSelection<float>();
//	ChaosTest::TransmissionTestAutoGearSelection<float>();
//	ChaosTest::TransmissionTestGearRatios<float>();
//
//	ChaosTest::EngineRPM<float>();
//
//	ChaosTest::WheelLateralSlip<float>();
//	ChaosTest::WheelBrakingLongitudinalSlip<float>();
//	ChaosTest::WheelAcceleratingLongitudinalSlip<float>();
//
//	ChaosTest::SuspensionForce<float>();
//}


//////////////////////////////////////////////////////////
///// GEOMETRY COLLECTION ///////////////////////////////


// Matrices Tests
TEST(GeometryCollection_MatricesTest,BasicGlobalMatrices) { GeometryCollectionTest::BasicGlobalMatrices();SUCCEED(); }
TEST(GeometryCollection_MatricesTest,TransformMatrixElement) { GeometryCollectionTest::TransformMatrixElement(); SUCCEED(); }
TEST(GeometryCollection_MatricesTest,ReparentingMatrices) { GeometryCollectionTest::ReparentingMatrices(); SUCCEED(); }

// Creation Tests CollectionCycleTest
TEST(GeometryCollection_CreationTest, CheckClassTypes) { GeometryCollectionTest::CheckClassTypes(); SUCCEED(); }
TEST(GeometryCollection_CreationTest, CheckIncrementMask) { GeometryCollectionTest::CheckIncrementMask(); SUCCEED(); }
TEST(GeometryCollection_CreationTest,Creation) { GeometryCollectionTest::Creation(); SUCCEED(); }
TEST(GeometryCollection_CreationTest,Empty) { GeometryCollectionTest::Empty(); SUCCEED(); }
TEST(GeometryCollection_CreationTest,AppendTransformHierarchy) { GeometryCollectionTest::AppendTransformHierarchy(); SUCCEED(); }
TEST(GeometryCollection_CreationTest,ParentTransformTest) { GeometryCollectionTest::ParentTransformTest(); SUCCEED(); }
TEST(GeometryCollection_CreationTest,DeleteFromEnd) { GeometryCollectionTest::DeleteFromEnd(); SUCCEED(); }
TEST(GeometryCollection_CreationTest,DeleteFromStart) { GeometryCollectionTest::DeleteFromStart(); SUCCEED(); }
TEST(GeometryCollection_CreationTest,DeleteFromMiddle) { GeometryCollectionTest::DeleteFromMiddle(); SUCCEED(); }
TEST(GeometryCollection_CreationTest,DeleteBranch) { GeometryCollectionTest::DeleteBranch(); SUCCEED(); }
TEST(GeometryCollection_CreationTest,DeleteRootLeafMiddle) { GeometryCollectionTest::DeleteRootLeafMiddle(); SUCCEED(); }
TEST(GeometryCollection_CreationTest,DeleteEverything) { GeometryCollectionTest::DeleteEverything(); SUCCEED(); }
TEST(GeometryCollection_CreationTest,ReindexMaterialsTest) { GeometryCollectionTest::ReindexMaterialsTest(); SUCCEED(); }
TEST(GeometryCollection_CreationTest,ContiguousElementsTest) { GeometryCollectionTest::ContiguousElementsTest(); SUCCEED(); }
TEST(GeometryCollection_CreationTest, AttributeDependencyTest) { GeometryCollectionTest::AttributeDependencyTest(); SUCCEED(); }
TEST(GeometryCollection_CreationTest, IntListReindexOnDeletionTest) { GeometryCollectionTest::IntListReindexOnDeletionTest(); SUCCEED(); }
TEST(GeometryCollection_CreationTest, IntListSelfDependencyTest) { GeometryCollectionTest::IntListSelfDependencyTest(); SUCCEED(); }
TEST(GeometryCollection_CreationTest, AppendManagedArrayCollectionTest) { GeometryCollectionTest::AppendManagedArrayCollectionTest(); SUCCEED(); }
TEST(GeometryCollection_CreationTest, AppendTransformCollectionTest) { GeometryCollectionTest::AppendTransformCollectionTest(); SUCCEED(); }
TEST(GeometryCollection_CreationTest, CollectionCycleTest) { GeometryCollectionTest::CollectionCycleTest(); SUCCEED(); }


// Proximity Tests
TEST(GeometryCollection_ProximityTest,BuildProximity) { GeometryCollectionTest::BuildProximity(); SUCCEED(); }
TEST(GeometryCollection_ProximityTest,GeometryDeleteFromStart) { GeometryCollectionTest::GeometryDeleteFromStart(); SUCCEED(); }
TEST(GeometryCollection_ProximityTest,GeometryDeleteFromEnd) { GeometryCollectionTest::GeometryDeleteFromEnd(); SUCCEED(); }
TEST(GeometryCollection_ProximityTest,GeometryDeleteFromMiddle) { GeometryCollectionTest::GeometryDeleteFromMiddle(); SUCCEED(); }
TEST(GeometryCollection_ProximityTest,GeometryDeleteMultipleFromMiddle) { GeometryCollectionTest::GeometryDeleteMultipleFromMiddle(); SUCCEED(); }
TEST(GeometryCollection_ProximityTest,GeometryDeleteRandom) { GeometryCollectionTest::GeometryDeleteRandom(); SUCCEED(); }
TEST(GeometryCollection_ProximityTest,GeometryDeleteRandom2) { GeometryCollectionTest::GeometryDeleteRandom2(); SUCCEED(); }
TEST(GeometryCollection_ProximityTest,GeometryDeleteAll) { GeometryCollectionTest::GeometryDeleteAll(); SUCCEED(); }
TEST(GeometryCollection_ProximityTest,GeometrySwapFlat) { GeometryCollectionTest::GeometrySwapFlat(); SUCCEED(); }
TEST(GeometryCollection_ProximityTest,TestFracturedGeometry) { GeometryCollectionTest::TestFracturedGeometry(); SUCCEED(); }

// Clean Tests
TEST(GeometryCollection_CleanTest,TestDeleteCoincidentVertices) { GeometryCollectionTest::TestDeleteCoincidentVertices(); SUCCEED(); }
TEST(GeometryCollection_CleanTest,TestDeleteCoincidentVertices2) { GeometryCollectionTest::TestDeleteCoincidentVertices2(); SUCCEED(); }
TEST(GeometryCollection_CleanTest,TestDeleteZeroAreaFaces) { GeometryCollectionTest::TestDeleteZeroAreaFaces(); SUCCEED(); }
TEST(GeometryCollection_CleanTest,TestDeleteHiddenFaces) { GeometryCollectionTest::TestDeleteHiddenFaces(); SUCCEED(); }
TEST(GeometryCollection_CleanTest,TestFillHoles) { GeometryCollectionTest::TestFillHoles(); SUCCEED(); }

// SpatialHash Tests
TEST(GeometryCollection_SpatialHashTest,GetClosestPointsTest1) { GeometryCollectionTest::GetClosestPointsTest1(); SUCCEED(); }
TEST(GeometryCollection_SpatialHashTest,GetClosestPointsTest2) { GeometryCollectionTest::GetClosestPointsTest2(); SUCCEED(); }
TEST(GeometryCollection_SpatialHashTest,GetClosestPointsTest3) { GeometryCollectionTest::GetClosestPointsTest3(); SUCCEED(); }
TEST(GeometryCollection_SpatialHashTest,GetClosestPointTest) { GeometryCollectionTest::GetClosestPointTest(); SUCCEED(); }
TEST(GeometryCollection_SpatialHashTest,HashTableUpdateTest) { GeometryCollectionTest::HashTableUpdateTest(); SUCCEED(); }
TEST(GeometryCollection_SpatialHashTest,HashTablePressureTest) { GeometryCollectionTest::HashTablePressureTest(); SUCCEED(); }

// HideVertices Test
TEST(GeometryCollection_HideVerticesTest,TestHideVertices) { GeometryCollectionTest::TestHideVertices(); SUCCEED(); }

// Object Collision Test
//TEST(GeometryCollection_CollisionTest, DISABLED_TestGeometryDecimation) { GeometryCollectionTest::TestGeometryDecimation(); SUCCEED(); }  Fix or remove support for decimation
TEST(GeometryCollection_CollisionTest,TestImplicitCapsule) { GeometryCollectionTest::TestImplicitCapsule(); SUCCEED(); }
TEST(GeometryCollection_CollisionTest,TestImplicitCylinder) { GeometryCollectionTest::TestImplicitCylinder(); SUCCEED(); }
TEST(GeometryCollection_CollisionTest,TestImplicitSphere) { GeometryCollectionTest::TestImplicitSphere(); SUCCEED(); }
TEST(GeometryCollection_CollisionTest,TestImplicitBoneHierarchy) { GeometryCollectionTest::TestImplicitBoneHierarchy(); SUCCEED(); }

// Fields Tests
TEST(GeometryCollection_FieldTest,Fields_NoiseSample) { GeometryCollectionTest::Fields_NoiseSample(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_RadialIntMask) { GeometryCollectionTest::Fields_RadialIntMask(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_RadialFalloff) { GeometryCollectionTest::Fields_RadialFalloff(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_PlaneFalloff) { GeometryCollectionTest::Fields_PlaneFalloff(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_UniformVector) { GeometryCollectionTest::Fields_UniformVector(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_RaidalVector) { GeometryCollectionTest::Fields_RaidalVector(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_SumVectorFullMult) { GeometryCollectionTest::Fields_SumVectorFullMult(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_SumVectorFullDiv) { GeometryCollectionTest::Fields_SumVectorFullDiv(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_SumVectorFullAdd) { GeometryCollectionTest::Fields_SumVectorFullAdd(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_SumVectorFullSub) { GeometryCollectionTest::Fields_SumVectorFullSub(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_SumVectorLeftSide) { GeometryCollectionTest::Fields_SumVectorLeftSide(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_SumVectorRightSide) { GeometryCollectionTest::Fields_SumVectorRightSide(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_SumScalar) { GeometryCollectionTest::Fields_SumScalar(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_SumScalarRightSide) { GeometryCollectionTest::Fields_SumScalarRightSide(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_SumScalarLeftSide) { GeometryCollectionTest::Fields_SumScalarLeftSide(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_Culling) { GeometryCollectionTest::Fields_Culling(); SUCCEED(); }
TEST(GeometryCollection_FieldTest,Fields_SerializeAPI) { GeometryCollectionTest::Fields_SerializeAPI(); SUCCEED(); }

// Cache track compression
TEST(ChaosCacheTests, TrackCompression) 
{ 
	ChaosTest::ChaosCache::TrackCompressionTest();
	SUCCEED(); 
}

GTEST_TEST(ArrayTests, TestArrayMax)
{
	// The first 3 arrays without Reserve will over-allocate. We aren't testing anything
	// useful on these - they are just here for examples...

	// This allocates space for 4 elements
	TArray<int32> Ints1;
	Ints1.SetNum(1);
	EXPECT_LE(Ints1.Num(), Ints1.Max());

	// This allocates space for 5 elements
	TArray<int32> Ints2;
	Ints2.SetNum(5);
	EXPECT_LE(Ints2.Num(), Ints2.Max());

	// This allocates space for 4 elements and grows to 22 elements
	TArray<int32> Ints3;
	Ints3.SetNum(1);
	Ints3.SetNum(5);
	EXPECT_LE(Ints3.Num(), Ints3.Max());

	// We rely on tight-fitting arrays for memory conservation. Make sure that the
	// default Reserve policy still enforces tight-fitting arrays.
	TArray<int32> Ints4;
	Ints4.Reserve(1);
	Ints4.SetNum(1);
	EXPECT_EQ(Ints4.Max(), Ints4.Num());

	TArray<int32> Ints5;
	Ints5.Reserve(5);
	Ints5.SetNum(5);
	EXPECT_EQ(Ints5.Max(), Ints5.Num());

	TArray<int32> Ints6;
	Ints6.Reserve(1);
	Ints6.SetNum(1);
	Ints6.Reserve(5);
	Ints6.SetNum(5);
	EXPECT_EQ(Ints6.Max(), Ints6.Num());

	// Reset also sets the exact buffer size and we are relying on that too
	TArray<int32> Ints7;
	Ints7.Reset(1);
	Ints7.SetNum(1);
	EXPECT_EQ(Ints7.Max(), Ints7.Num());

	TArray<int32> Ints8;
	Ints8.Reset(5);
	Ints8.SetNum(5);
	EXPECT_EQ(Ints8.Max(), Ints8.Num());

	TArray<int32> Ints9;
	Ints9.Reset(1);
	Ints9.SetNum(1);
	Ints9.Reset(5);
	Ints9.SetNum(5);
	EXPECT_EQ(Ints9.Max(), Ints9.Num());

}

//TEST(GeometryCollectionTest,RigidBodies_CollisionGroup); // fix me
//
// Broken
//

/*
TEST(GeometryCollectionTest, RigidBodies_ClusterTest_KinematicAnchor) { GeometryCollectionTest::RigidBodies_ClusterTest_KinematicAnchor<float>(); SUCCEED(); }
TEST(GeometryCollectionTest, RigidBodies_ClusterTest_StaticAnchor) { GeometryCollectionTest::RigidBodies_ClusterTest_StaticAnchor<float>(); SUCCEED(); }
TEST(GeometryCollectionTest, RigidBodies_ClusterTest_ReleaseClusterParticles_AllLeafNodes) { GeometryCollectionTest::RigidBodies_ClusterTest_ReleaseClusterParticles_AllLeafNodes<float>(); SUCCEED(); }
TEST(GeometryCollectionTest, RigidBodies_ClusterTest_ReleaseClusterParticles_ClusterNodeAndSubClusterNode) { GeometryCollectionTest::RigidBodies_ClusterTest_ReleaseClusterParticles_ClusterNodeAndSubClusterNode<float>(); SUCCEED(); }
TEST(GeometryCollectionTest, RigidBodies_ClusterTest_RemoveOnFracture) { GeometryCollectionTest::RigidBodies_ClusterTest_RemoveOnFracture<float>(); SUCCEED(); }
TEST(GeometryCollectionTest, RigidBodiess_ClusterTest_ParticleImplicitCollisionGeometry) { GeometryCollectionTest::RigidBodiess_ClusterTest_ParticleImplicitCollisionGeometry<float>(); SUCCEED(); }
*/


// SimulationStreaming Tests
// broken and/or crashing
/*
TEST(GeometryCollectionTest, RigidBodies_Streaming_StartSolverEmpty) { GeometryCollectionTest::RigidBodies_Streaming_StartSolverEmpty<float>(); SUCCEED(); }
TEST(GeometryCollectionTest, RigidBodies_Streaming_BulkInitialization) { GeometryCollectionTest::RigidBodies_Streaming_BulkInitialization<float>(); SUCCEED(); }
TEST(GeometryCollectionTest, RigidBodies_Streaming_DeferedClusteringInitialization) { GeometryCollectionTest::RigidBodies_Streaming_DeferedClusteringInitialization<float>(); SUCCEED(); }
*/


// Secondary Particle Events
//TEST(GeometryCollectionTest, Solver_ValidateReverseMapping) { GeometryCollectionTest::Solver_ValidateReverseMapping<float>(); SUCCEED(); }



// Static and Skeletal Mesh Tests
// broken and/or crashing
/*
TEST(SkeletalMeshPhysicsProxyTest, RegistersCorrectly){GeometryCollectionTest::TestSkeletalMeshPhysicsProxy_Register<float>();SUCCEED();}
TEST(SkeletalMeshPhysicsProxyTest, KinematicBonesMoveCorrectly){GeometryCollectionTest::TestSkeletalMeshPhysicsProxy_Kinematic<float>();SUCCEED();}
TEST(SkeletalMeshPhysicsProxyTest, DynamicBonesMoveCorrectly){GeometryCollectionTest::TestSkeletalMeshPhysicsProxy_Dynamic<float>();SUCCEED();}
*/

// Serialization
TEST(GeometryCollectionSerializationTests,GeometryCollectionSerializesCorrectly){ GeometryCollectionTests::GeometryCollectionSerialization();SUCCEED(); }


/**/

class UEGTestPrinter : public ::testing::EmptyTestEventListener
{
    virtual void OnTestStart(const ::testing::TestInfo& TestInfo)
    {
        UE_LOGF(LogHeadlessChaos, Verbose, "Test %ls.%ls Starting", *FString(TestInfo.test_case_name()), *FString(TestInfo.name()));
    }

    virtual void OnTestPartResult(const ::testing::TestPartResult& TestPartResult)
    {
        if (TestPartResult.failed())
        {
            UE_LOGF(LogHeadlessChaos, Error, "FAILED in %ls:%d\n%ls", *FString(TestPartResult.file_name()), TestPartResult.line_number(), *FString(TestPartResult.summary()))
        }
        else
        {
            UE_LOGF(LogHeadlessChaos, Verbose, "Succeeded in %ls:%d\n%ls", *FString(TestPartResult.file_name()), TestPartResult.line_number(), *FString(TestPartResult.summary()))
        }
    }

    virtual void OnTestEnd(const ::testing::TestInfo& TestInfo)
    {
        UE_LOGF(LogHeadlessChaos, Verbose, "Test %ls.%ls Ending", *FString(TestInfo.test_case_name()), *FString(TestInfo.name()));
    }
};

int32 tchar_main(int32 ArgC, TCHAR* ArgV[])
{
	UE::Geometry::ExactPredicates::GlobalInit();

	// start up the main loop
	GEngineLoop.PreInit(ArgC, ArgV);
	FModuleManager::Get().StartProcessingNewlyLoadedObjects();
	
	::testing::InitGoogleTest(&ArgC, ArgV);

	// Add a UE-formatting printer
    ::testing::TestEventListeners& Listeners = ::testing::UnitTest::GetInstance()->listeners();
    Listeners.Append(new UEGTestPrinter);

	ensure(RUN_ALL_TESTS() == 0);

	RequestEngineExit(TEXT("Exiting"));
	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	FEngineLoop::AppExit();

	return 0;
}

// We need to provide a non-wide main otherwise we default to one inside gtest
// which doesn't correctly initialise the tests.
int32 main(int32 ArgC, ANSICHAR* Utf8ArgV[])
{
	// Allocated storage for TCHAR arguments
	TCHAR** LocalArgs = new TCHAR* [ArgC];

	// Gtest will directly modify the args array we pass in to the InitGoogleTest call
	// so we need to pass it a copy of the pointers otherwise we lose track and leak
	// the allocations for the args
	TCHAR** PassthroughArgs = new TCHAR* [ArgC];

	for(int32 ArgIndex = 0; ArgIndex < ArgC; ArgIndex++)
	{
		FUTF8ToTCHAR ConvertFromUtf8(Utf8ArgV[ArgIndex]);
		LocalArgs[ArgIndex] = new TCHAR[ConvertFromUtf8.Length() + 1];

		// Copy the pointer to give to Gtest
		PassthroughArgs[ArgIndex] = LocalArgs[ArgIndex];

		// Fill in the arg
		FCString::Strncpy(LocalArgs[ArgIndex], ConvertFromUtf8.Get(), ConvertFromUtf8.Length() + 1);
	}

	// Run our actual test main
	int32 Result = tchar_main(ArgC, PassthroughArgs);

	for(int32 ArgIndex = 0; ArgIndex < ArgC; ArgIndex++)
	{
		delete[] LocalArgs[ArgIndex];
	}

	delete[] PassthroughArgs;
	delete[] LocalArgs;

	return Result;
}

#undef LOCTEXT_NAMESPACE
