// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosCache.h"
#include "HeadlessChaos.h"

namespace
{
	static void CompressTransformTrack(TArray<FTransform>& InOutTransformTrack)
	{
		// we only need to compress if there's more than 3 keys
		if (InOutTransformTrack.Num() >= 3)
		{
			// simple compression algorithm to remove similar keys
			// we compare the resulting transform
			// the compression can be done in place because the number of resulting keys is always smaller than the original number of keys

			int32 CompressedKeyIndex = 0;

			for (int32 KeyIndex = 0; KeyIndex < InOutTransformTrack.Num(); KeyIndex++)
			{
				FTransform CurrentTransform = InOutTransformTrack[KeyIndex];
				// write current
				check(CompressedKeyIndex < InOutTransformTrack.Num());
				InOutTransformTrack[CompressedKeyIndex++] = InOutTransformTrack[KeyIndex];

				// find the next index where the transform is different
				int32 NextIndex = (KeyIndex + 1);
				for (; NextIndex < InOutTransformTrack.Num(); NextIndex++)
				{
					const FTransform NextTransform = InOutTransformTrack[NextIndex];
					if (!NextTransform.Equals(CurrentTransform))
					{
						break;
					}
				}
				// if there's at least one identical transform before the new one 
				const int32 LastSimilarIndex = (NextIndex - 1);
				if (LastSimilarIndex > KeyIndex)
				{
					// write same transform with oldest timestamp
					if (CompressedKeyIndex < InOutTransformTrack.Num())
					{
						InOutTransformTrack[CompressedKeyIndex++] = InOutTransformTrack[LastSimilarIndex];
					}
					KeyIndex = (NextIndex - 1); // account for the automatic increment of the loop
				}
			}

			// we are done we can now shrink the original arrays to the compressed size if necessary
			const int32 CompressedSize = CompressedKeyIndex;
			if (CompressedSize < InOutTransformTrack.Num())
			{
				InOutTransformTrack.SetNum(CompressedSize);
			}
		}
	}
}

namespace ChaosTest::ChaosCache
{
	using namespace Chaos;

	void TrackCompressionTest()
	{
		const FTransform TrA(FVector(1));
		const FTransform TrB(FVector(2));
		const FTransform TrC(FVector(3));
		const FTransform TrD(FVector(4));

		// [A B C D] ==> [A B C D]
		{
			TArray<FTransform> Track1({ TrA, TrB, TrC, TrD });
			CompressTransformTrack(Track1);
			EXPECT_TRUE(Track1.Num() == 4);
			EXPECT_TRUE(Track1[0].Equals(TrA, 0));
			EXPECT_TRUE(Track1[1].Equals(TrB, 0));
			EXPECT_TRUE(Track1[2].Equals(TrC, 0));
			EXPECT_TRUE(Track1[3].Equals(TrD, 0));
		}
		{
			// [A A B C] ==> [A A B C]
			TArray<FTransform> Track2({ TrA, TrA, TrB, TrC });
			CompressTransformTrack(Track2);
			EXPECT_TRUE(Track2.Num() == 4);
			EXPECT_TRUE(Track2[0].Equals(TrA, 0));
			EXPECT_TRUE(Track2[1].Equals(TrA, 0));
			EXPECT_TRUE(Track2[2].Equals(TrB, 0));
			EXPECT_TRUE(Track2[3].Equals(TrC, 0));
		}
		{
			// [A A A B C] ==> [A A B C]
			TArray<FTransform> Track3({ TrA, TrA, TrA, TrB, TrC });
			CompressTransformTrack(Track3);
			EXPECT_TRUE(Track3.Num() == 4);
			EXPECT_TRUE(Track3[0].Equals(TrA, 0));
			EXPECT_TRUE(Track3[1].Equals(TrA, 0));
			EXPECT_TRUE(Track3[2].Equals(TrB, 0));
			EXPECT_TRUE(Track3[3].Equals(TrC, 0));
		}
		{
			// [A A A A] ==> [A A]
			TArray<FTransform> Track4({ TrA, TrA, TrA, TrA });
			CompressTransformTrack(Track4);
			EXPECT_TRUE(Track4.Num() == 2);
			EXPECT_TRUE(Track4[0].Equals(TrA, 0));
			EXPECT_TRUE(Track4[1].Equals(TrA, 0));
		}
		{
			// [A B C D D] ==> [A B C D D]
			TArray<FTransform> Track5({ TrA, TrB, TrC, TrD, TrD });
			CompressTransformTrack(Track5);
			EXPECT_TRUE(Track5.Num() == 5);
			EXPECT_TRUE(Track5[0].Equals(TrA, 0));
			EXPECT_TRUE(Track5[1].Equals(TrB, 0));
			EXPECT_TRUE(Track5[2].Equals(TrC, 0));
			EXPECT_TRUE(Track5[3].Equals(TrD, 0));
			EXPECT_TRUE(Track5[4].Equals(TrD, 0));
		}
		{
			// [A B C D D D] ==> [A B C D D]
			TArray<FTransform> Track6({ TrA, TrB, TrC, TrD, TrD, TrD });
			CompressTransformTrack(Track6);
			EXPECT_TRUE(Track6.Num() == 5);
			EXPECT_TRUE(Track6[0].Equals(TrA, 0));
			EXPECT_TRUE(Track6[1].Equals(TrB, 0));
			EXPECT_TRUE(Track6[2].Equals(TrC, 0));
			EXPECT_TRUE(Track6[3].Equals(TrD, 0));
			EXPECT_TRUE(Track6[4].Equals(TrD, 0));
		}
		{
			// [A A A B C D D D] ==> [A A B C D D]
			TArray<FTransform> Track7({ TrA, TrA, TrA, TrB, TrC, TrD, TrD, TrD });
			CompressTransformTrack(Track7);
			EXPECT_TRUE(Track7.Num() == 6);
			EXPECT_TRUE(Track7[0].Equals(TrA, 0));
			EXPECT_TRUE(Track7[1].Equals(TrA, 0));
			EXPECT_TRUE(Track7[2].Equals(TrB, 0));
			EXPECT_TRUE(Track7[3].Equals(TrC, 0));
			EXPECT_TRUE(Track7[4].Equals(TrD, 0));
			EXPECT_TRUE(Track7[5].Equals(TrD, 0));
		}
		{
			// [A A A B C C C D ] ==> [A A B C C D]
			TArray<FTransform> Track8({ TrA, TrA, TrA, TrB, TrC, TrC, TrC, TrD });
			CompressTransformTrack(Track8);
			EXPECT_TRUE(Track8.Num() == 6);
			EXPECT_TRUE(Track8[0].Equals(TrA, 0));
			EXPECT_TRUE(Track8[1].Equals(TrA, 0));
			EXPECT_TRUE(Track8[2].Equals(TrB, 0));
			EXPECT_TRUE(Track8[3].Equals(TrC, 0));
			EXPECT_TRUE(Track8[4].Equals(TrC, 0));
			EXPECT_TRUE(Track8[5].Equals(TrD, 0));
		}
	}
}
