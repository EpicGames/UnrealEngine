// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/Matrix.h"
#include "Chaos/Utilities.h"
#include "Chaos/AABB.h"

namespace ChaosTest
{
	using namespace Chaos;

	void TestBounds(const FAABB3& A, const FAABB3& B, const FReal Eps = 1.e-6)
	{
		EXPECT_NEAR(A.Min().X, B.Min().X, Eps);
		EXPECT_NEAR(A.Min().Y, B.Min().Y, Eps);
		EXPECT_NEAR(A.Min().Z, B.Min().Z, Eps);
		EXPECT_NEAR(A.Max().X, B.Max().X, Eps);
		EXPECT_NEAR(A.Max().Y, B.Max().Y, Eps);
		EXPECT_NEAR(A.Max().Z, B.Max().Z, Eps);
	}

	// Transformed AABB by transforming all box verts
	// Used for reference when comparing to current TAABB::TransformedAABB results
	FAABB3 ReferenceTransformAABB(const FAABB3& AABB, const FRigidTransform3& Transform)
	{
		FVec3 Extents = AABB.Extents();
		FAABB3 NewAABB = FAABB3::EmptyAABB();
		NewAABB.GrowToInclude(Transform.TransformPosition(FVec3(AABB.Min())));
		NewAABB.GrowToInclude(Transform.TransformPosition(FVec3(AABB.Max())));

		for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
		{
			NewAABB.GrowToInclude(Transform.TransformPosition(FVec3(AABB.Min() + FVec3::AxisVector(AxisIndex) * Extents)));
			NewAABB.GrowToInclude(Transform.TransformPosition(FVec3(AABB.Max() - FVec3::AxisVector(AxisIndex) * Extents)));
		}

		return NewAABB;
	}

	// Inverse Transformed AABB by transforming all box verts
	// Used for reference when comparing to current TAABB::InverseTransformedAABB results
	FAABB3 ReferenceInverseTransformAABB(const FAABB3& AABB, const FRigidTransform3& Transform)
	{
		FVec3 Extents = AABB.Extents();
		FAABB3 NewAABB = FAABB3::EmptyAABB();
		NewAABB.GrowToInclude(Transform.InverseTransformPosition(FVec3(AABB.Min())));
		NewAABB.GrowToInclude(Transform.InverseTransformPosition(FVec3(AABB.Max())));

		for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
		{
			NewAABB.GrowToInclude(Transform.InverseTransformPosition(FVec3(AABB.Min() + FVec3::AxisVector(AxisIndex) * Extents)));
			NewAABB.GrowToInclude(Transform.InverseTransformPosition(FVec3(AABB.Max() - FVec3::AxisVector(AxisIndex) * Extents)));
		}

		return NewAABB;
	}

	// Check that identity transform does not affect an AABB
	GTEST_TEST(AABBTests, TestTransformAABB_Unit)
	{
		const FAABB3 Bounds = FAABB3(FVec3(-1,-2,-3), FVec3(1,2,3));
		const FRigidTransform3 Transform = FRigidTransform3::Identity;
		
		const FAABB3 TransformedBounds = Bounds.TransformedAABB(Transform);
		TestBounds(TransformedBounds, Bounds);

		const FAABB3 InverseTransformedBounds = Bounds.InverseTransformedAABB(Transform);
		TestBounds(InverseTransformedBounds, Bounds);
	}

	GTEST_TEST(AABBTests, TestTransformAABB_Translate)
	{
		const FAABB3 Bounds = FAABB3(FVec3(-3, -2, -1), FVec3(1, 2, 3));

		const FVec3 Offset = FVec3(2);
		const FRigidTransform3 Transform = FRigidTransform3(Offset, FRotation3::FromIdentity(), FVec3(1));

		const FAABB3 TransformedBounds = Bounds.TransformedAABB(Transform);
		TestBounds(TransformedBounds, FAABB3(Offset + Bounds.Min(), Offset + Bounds.Max()));

		const FAABB3 InverseTransformedBounds = Bounds.InverseTransformedAABB(Transform);
		TestBounds(InverseTransformedBounds, FAABB3(Bounds.Min() - Offset, Bounds.Max() - Offset));
	}

	GTEST_TEST(AABBTests, TestTransformAABB_Scale)
	{
		const FAABB3 Bounds = FAABB3(FVec3(-3, -2, -1), FVec3(1, 2, 3));

		const FVec3 Scale = FVec3(2);
		const FRigidTransform3 Transform = FRigidTransform3(FVec3(0), FRotation3::FromIdentity(), Scale);

		const FAABB3 TransformedBounds = Bounds.TransformedAABB(Transform);
		TestBounds(TransformedBounds, FAABB3(Scale * Bounds.Min(), Scale * Bounds.Max()));

		const FAABB3 InverseTransformedBounds = Bounds.InverseTransformedAABB(Transform);
		TestBounds(InverseTransformedBounds, FAABB3(Bounds.Min() / Scale, Bounds.Max() / Scale));
	}

	GTEST_TEST(AABBTests, TestTransformAABB_NegativeScale)
	{
		const FAABB3 Bounds = FAABB3(FVec3(-3, -2, -1), FVec3(1, 2, 3));

		const FVec3 Scale = FVec3(-2);
		const FRigidTransform3 Transform = FRigidTransform3(FVec3(0), FRotation3::FromIdentity(), Scale);

		const FAABB3 TransformedBounds = Bounds.TransformedAABB(Transform);
		TestBounds(TransformedBounds, FAABB3(Scale * Bounds.Max(), Scale * Bounds.Min()));

		const FAABB3 InverseTransformedBounds = Bounds.InverseTransformedAABB(Transform);
		TestBounds(InverseTransformedBounds, FAABB3(Bounds.Max() / Scale, Bounds.Min() / Scale));
	}

	GTEST_TEST(AABBTests, TestTransformAABB_ScaleRotateTranslate)
	{
		const FAABB3 Bounds = FAABB3(FVec3(-3, -2, -1), FVec3(1, 2, 3));
		const FRigidTransform3 Transform = FRigidTransform3(FVec3(4,7,-3), FRotation3::FromAxisAngle(FVec3(4,-3,5).GetUnsafeNormal(), FMath::DegreesToRadians(36.5)), FVec3(1, -4, 17));

		const FAABB3 TransformedBounds = Bounds.TransformedAABB(Transform);
		TestBounds(TransformedBounds, ReferenceTransformAABB(Bounds, Transform));

		const FAABB3 InverseTransformedBounds = Bounds.InverseTransformedAABB(Transform);
		TestBounds(InverseTransformedBounds, ReferenceInverseTransformAABB(Bounds, Transform));
	}


	// A test that was failing for 8-wide ISPC architectures
	GTEST_TEST(AABBTests, TestTransformAABB_ISPC8WideFail)
	{
		const FAABB3 AABB(FVec3(-51.1019897, -51.1019936, -51.1019974), FVec3(51.1019897, 51.1020012, 51.1019974));
		const FRigidTransform3 Transform(FVec3(1.16415322e-10, 5.82076609e-11, -0.0681053028), FRotation3::FromElements(0.579225421, -0.644822955, 0.0343584977, 0.497514546));

		const FAABB3 ExpectedAABB(FVec3(-79.1592407, -84.8343811, -87.7618637), FVec3(79.1592407, 84.8343811, 87.6256561));

		const FAABB3 ResultAABB = AABB.TransformedAABB(Transform);
		EXPECT_NEAR(ResultAABB.Min().X, ExpectedAABB.Min().X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ResultAABB.Min().Y, ExpectedAABB.Min().Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ResultAABB.Min().Z, ExpectedAABB.Min().Z, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ResultAABB.Max().X, ExpectedAABB.Max().X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ResultAABB.Max().Y, ExpectedAABB.Max().Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ResultAABB.Max().Z, ExpectedAABB.Max().Z, KINDA_SMALL_NUMBER);
	}


}
