// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestBP.h"
#include "HeadlessChaos.h"
#include "HAL/FileManager.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/ChaosPerfTest.h"


namespace ChaosTest
{
	using namespace Chaos;

	void BPPerfTest()
	{
#if CHAOS_PARTICLEHANDLE_TODO
		CHAOS_PERF_TEST(BPPerf, EChaosPerfUnits::Us)
		//perf files are not stored in p4 at the moment. Use your own local file by calling 'p.SerializeForPerfTest' while running a specific scene. The log will contain the file name generated
		TUniquePtr<FArchive> File(IFileManager::Get().CreateFileReader(TEXT("ChaosPerf_xxx.bin")));
		
		const FReal Dt = (FReal)1 / (FReal)60;
		if (File)
		{
			FChaosArchive Ar(*File);

			FPBDRigidParticles Particles;
			FPBDRigidsEvolution Evolution(MoveTemp(Particles));
			Evolution.SerializeForPerfTest(Ar);
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
	}
#endif
	}
}