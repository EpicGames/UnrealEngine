// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"

#include "Chaos/Utilities.h"
#include "Chaos/Triangle.h"
#include "ChaosLog.h"

namespace ChaosTest
{
	using namespace Chaos;

	// For non-degenerate triangles, the berycentric coords should be uniquely defined and match the input.
	// For degenerate triangles or external points, the barycentric coords are not unique but the calculated coords should generate the same point.
	void TestToFromBarycentric(const FVec3 Verts[], const FVec3& ExpectedBary, const bool bIsDegenerate, const bool bIsOutside)
	{
		const FVec3 ExpectedP = FromBarycentric(ExpectedBary, Verts[0], Verts[1], Verts[2]);

		const FVec3 CalculatedBary = ToBarycentric(ExpectedP, Verts[0], Verts[1], Verts[2]);

		const FVec3 CalculatedP = FromBarycentric(CalculatedBary, Verts[0], Verts[1], Verts[2]);

		const FReal Tolerance = 1.e-4;

		// Regardless of degeneracy, barycentric coords should sum to 1
		EXPECT_NEAR(CalculatedBary.X + CalculatedBary.Y + CalculatedBary.Z, 1.0, Tolerance);

		// If we are inside the triangle/line/point barycentric coords should be in [0,1]
		if (!bIsOutside)
		{
			EXPECT_GE(CalculatedBary.X, 0);
			EXPECT_GE(CalculatedBary.Y, 0);
			EXPECT_GE(CalculatedBary.Z, 0);
			EXPECT_LE(CalculatedBary.X, 1);
			EXPECT_LE(CalculatedBary.Y, 1);
			EXPECT_LE(CalculatedBary.Z, 1);
		}

		// Regardless of degeneracy, barycentric coords should reproduce the point
		EXPECT_NEAR(CalculatedP.X, ExpectedP.X, Tolerance);
		EXPECT_NEAR(CalculatedP.Y, ExpectedP.Y, Tolerance);
		EXPECT_NEAR(CalculatedP.Z, ExpectedP.Z, Tolerance);

		// for non-degenerate triangles we should calculate the barycentric coords exactly
		if (!bIsDegenerate)
		{
			EXPECT_NEAR(CalculatedBary.X + CalculatedBary.Y + CalculatedBary.Z, 1.0, Tolerance);
			EXPECT_NEAR(CalculatedBary.X, ExpectedBary.X, Tolerance);
			EXPECT_NEAR(CalculatedBary.Y, ExpectedBary.Y, Tolerance);
			EXPECT_NEAR(CalculatedBary.Z, ExpectedBary.Z, Tolerance);
		}
	}

	// Runa set of points through the barycentric test for the specified triangle
	void TestBarycentricPoints(const FVec3 Verts[], const bool bIsDegenerate)
	{
		TestToFromBarycentric(Verts, FVec3(0.2, 0.4, 0.4), bIsDegenerate, false);	// Inside
		TestToFromBarycentric(Verts, FVec3(0.2, 0.8, 0.0), bIsDegenerate, false);	// AB Edge
		TestToFromBarycentric(Verts, FVec3(0.0, 0.5, 0.5), bIsDegenerate, false);	// BC Edge
		TestToFromBarycentric(Verts, FVec3(0.3, 0.0, 0.7), bIsDegenerate, false);	// CA Edge
		TestToFromBarycentric(Verts, FVec3(1.0, 0.0, 0.0), bIsDegenerate, false);	// A Vertex
		TestToFromBarycentric(Verts, FVec3(0.0, 1.0, 0.0), bIsDegenerate, false);	// B Vertex
		TestToFromBarycentric(Verts, FVec3(0.0, 0.0, 1.0), bIsDegenerate, false);	// C Vertex
	}

	// Non-degenerate triangle test
	GTEST_TEST(BarycentricTests, TestBarycentric_Triangle_Fwd)
	{
		const FVec3 Verts[] = {
			FVec3(0,0,0),
			FVec3(100,0,0),
			FVec3(100,100,0)
		};

		TestBarycentricPoints(Verts, false);
	}

	// Non-degenerate triangle test - reverse winding
	GTEST_TEST(BarycentricTests, TestBarycentric_Triangle2_Rev)
	{
		const FVec3 Verts[] = {
			FVec3(0,0,0),
			FVec3(100,100,0),
			FVec3(100,0,0)
		};

		TestBarycentricPoints(Verts, false);
	}

	// Test Barycentric when the triangle is degenerate (a line A-B-C)
	GTEST_TEST(BarycentricTests, TestBarycentric_LineABC)
	{
		const FVec3 Verts[] = {
			FVec3(0,0,0),
			FVec3(100,0,0),
			FVec3(200,0,0)
		};

		TestBarycentricPoints(Verts, true);
	}

	// Test Barycentric when the triangle is degenerate - reverse winding (a line A-C-B)
	GTEST_TEST(BarycentricTests, TestBarycentric_LineACB)
	{
		const FVec3 Verts[] = {
			FVec3(0,0,0),
			FVec3(200,0,0),
			FVec3(100,0,0)
		};

		TestBarycentricPoints(Verts, true);
	}

	// Test Barycentric when the triangle is degenerate (a point A-B-C)
	GTEST_TEST(BarycentricTests, TestBarycentric_PointABC)
	{
		const FVec3 Verts[] = {
			FVec3(0,0,0),
			FVec3(0,0,0),
			FVec3(0,0,0)
		};

		TestBarycentricPoints(Verts, true);
	}

	// Test Barycentric when the triangle is degenerate (a line and point A-B)
	GTEST_TEST(BarycentricTests, TestBarycentric_PointAB)
	{
		const FVec3 Verts[] = {
			FVec3(0,0,0),
			FVec3(0,0,0),
			FVec3(100,0,0)
		};

		TestBarycentricPoints(Verts, true);
	}

	// Test Barycentric when the triangle is degenerate (a line and point B-C)
	GTEST_TEST(BarycentricTests, TestBarycentric_PointBC)
	{
		const FVec3 Verts[] = {
			FVec3(0,0,0),
			FVec3(100,0,0),
			FVec3(100,0,0)
		};

		TestBarycentricPoints(Verts, true);
	}

	// Test Barycentric when the triangle is degenerate (a line and point A-C)
	GTEST_TEST(BarycentricTests, TestBarycentric_PointCA)
	{
		const FVec3 Verts[] = {
			FVec3(0,0,0),
			FVec3(100,0,0),
			FVec3(0,0,0)
		};

		TestBarycentricPoints(Verts, true);
	}

	// Non-degenerate triangle test with outside points
	GTEST_TEST(BarycentricTests, TestBarycentric_Triangle_Outside)
	{
		const FVec3 Verts[] = {
			FVec3(50,0,0),
			FVec3(100,0,0),
			FVec3(100,100,0)
		};

		TestToFromBarycentric(Verts, FVec3(1.5, -0.5, 0.0), false, true);	// Outside
		TestToFromBarycentric(Verts, FVec3(9.0, -5.0, -3.0), false, true);	// Outside
	}

	// Degenerate triangle test with outside points (on line)
	GTEST_TEST(BarycentricTests, TestBarycentric_LineACB_Outside)
	{
		const FVec3 Verts[] = {
			FVec3(50,0,0),
			FVec3(200,0,0),
			FVec3(100,0,0)
		};

		TestToFromBarycentric(Verts, FVec3(1.5, -0.5, 0.0), true, true);	// Outside
		TestToFromBarycentric(Verts, FVec3(9.0, -5.0, -3.0), true, true);	// Outside
	}

}