// Copyright Epic Games, Inc. All Rights Reserved.


#include "HeadlessChaosTestBroadphase.h"

#include "HeadlessChaos.h"
#include "Chaos/Box.h"
#include "Chaos/BoundingVolume.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/AABBTree.h"
#include "ChaosLog.h"
#include "PBDRigidsSolver.h"
#include "Chaos/SpatialAccelerationCollection.h"

namespace ChaosTest
{
	using namespace Chaos;

	/*In general we want to test the following for each broadphase type:
	- simple intersection test as used by sim (IntersectAll)
	- ray, sweep, overlap
	- miss entire structure
	- stop mid structure
	- multi overlap
	- multi block (adjust length)
	- any
	*/

	struct FVisitor : ISpatialVisitor<int32>
	{
		const FGeometryParticles& Boxes;
		const FVec3 Start;
		const FVec3 Dir;
		FVec3 HalfExtents;
		const FReal Thickness;
		int32 BlockAfterN;
		bool bAny;

		FVisitor(const FVec3& InStart, const FVec3& InDir, const FReal InThickness, const FGeometryParticles& InBoxes)
		: Boxes(InBoxes)
		, Start(InStart)
		, Dir(InDir)
		, HalfExtents(0)
		, Thickness(InThickness)
		, BlockAfterN(TNumericLimits<int32>::Max())
		, bAny(false)
		{}

		enum class SQType
		{
			Raycast,
			Sweep,
			Overlap
		};

		template <SQType>
		bool Visit(int32 Idx, FQueryFastData& CurData)
		{
			const FRigidTransform3 BoxTM(Boxes.GetX(Idx), Boxes.GetR(Idx));
			FAABB3 Box = static_cast<const TBox<FReal, 3>*>(Boxes.GetGeometry(Idx).GetReference())->BoundingBox().TransformedAABB(BoxTM);
			FAABB3 ThicknedBox(Box.Min() - HalfExtents, Box.Max() + HalfExtents);

			FReal NewLength;
			FVec3 Position;
			FVec3 Normal;
			int32 FaceIndex;
			const FReal OldLength = CurData.CurrentLength;
			if (ThicknedBox.Raycast(Start, Dir, CurData.CurrentLength, 0, NewLength, Position, Normal, FaceIndex))
			{
				Instances.Add(Idx);
				if (bAny)
				{
					return false;
				}
			}
			
			return true;
		}

		bool VisitRaycast(TSpatialVisitorData<int32> Idx, FQueryFastData& CurData)
		{
			return Visit<SQType::Raycast>(Idx.Payload, CurData);
		}

		bool VisitSweep(TSpatialVisitorData<int32> Idx, FQueryFastData& CurData)
		{
			return Visit<SQType::Sweep>(Idx.Payload, CurData);
		}

		bool VisitOverlap(TSpatialVisitorData<int32> Idx)
		{
			check(false);
			return false;
		}

		virtual bool Overlap(const TSpatialVisitorData<int32>& Instance) override
		{
			return VisitOverlap(Instance);
		}

		virtual bool Raycast(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData) override
		{
			return VisitRaycast(Instance, CurData);
		}

		virtual bool Sweep(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData) override
		{
			return VisitSweep(Instance, CurData);
		}

		virtual bool HasBlockingHit() const override
		{ 
			return Instances.Num() >= BlockAfterN; 
		}

		TArray<int32> Instances;
	};

	struct FOverlapVisitor : public ISpatialVisitor<int32>
	{
		const FGeometryParticles& Boxes;
		const FAABB3 Bounds;
		bool bAny;

		FOverlapVisitor(const FAABB3& InBounds, const FGeometryParticles& InBoxes)
			: Boxes(InBoxes)
			, Bounds(InBounds)
			, bAny(false)
		{}

		bool VisitOverlap(TSpatialVisitorData<int32> Instance)
		{
			const int32 Idx = Instance.Payload;
			const FRigidTransform3 BoxTM(Boxes.GetX(Idx), Boxes.GetR(Idx));
			FAABB3 Box = static_cast<const TBox<FReal, 3>*>(Boxes.GetGeometry(Idx).GetReference())->BoundingBox().TransformedAABB(BoxTM);
			
			if (Box.Intersects(Bounds))
			{
				Instances.Add(Idx);
				if (bAny)
				{
					return false;
				}
			}

			return true;
		}

		bool VisitRaycast(TSpatialVisitorData<int32> Idx, FQueryFastData&)
		{
			check(false);
			return false;
		}

		bool VisitSweep(TSpatialVisitorData<int32> Idx, FQueryFastData&)
		{
			check(false);
			return false;
		}

		virtual bool Overlap(const TSpatialVisitorData<int32>& Instance) override
		{
			return VisitOverlap(Instance);
		}

		virtual bool Raycast(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData) override
		{
			return VisitRaycast(Instance, CurData);
		}

		virtual bool Sweep(const TSpatialVisitorData<int32>& Instance, FQueryFastData& CurData) override
		{
			return VisitSweep(Instance, CurData);
		}

		TArray<int32> Instances;
	};

	struct FStressTestVisitor : ISpatialVisitor<FAccelerationStructureHandle>
	{
		using FPayload = FAccelerationStructureHandle;

		FStressTestVisitor() {}

		enum class SQType
		{
			Raycast,
			Sweep,
			Overlap
		};

		bool VisitRaycast(const TSpatialVisitorData<FPayload>& Data, FQueryFastData& CurData)
		{
			return true;
		}

		bool VisitSweep(const TSpatialVisitorData<FPayload>& Data, FQueryFastData& CurData)
		{
			return true;
		}

		bool VisitOverlap(const TSpatialVisitorData<FPayload>& Data)
		{
			return true;
		}

		virtual bool Overlap(const TSpatialVisitorData<FPayload>& Instance) override
		{
			return VisitOverlap(Instance);
		}

		virtual bool Raycast(const TSpatialVisitorData<FPayload>& Instance, FQueryFastData& CurData) override
		{
			return VisitRaycast(Instance, CurData);
		}

		virtual bool Sweep(const TSpatialVisitorData<FPayload>& Instance, FQueryFastData& CurData) override
		{
			return VisitSweep(Instance, CurData);
		}
	};


	auto BuildBoxes(FImplicitObjectPtr& Box, FReal BoxSize = 100, const FVec3& BoxGridDimensions = FVec3(10,10,10), const FVec3 Offset = FVec3(0, 0, 0))
	{
		Box = MakeImplicitObjectPtr<TBox<FReal, 3>>(FVec3(0, 0, 0), FVec3(BoxSize, BoxSize, BoxSize));
		auto Boxes = MakeUnique<FGeometryParticles>();
		const int32 NumCols = BoxGridDimensions.X;
		const int32 NumRows = BoxGridDimensions.Y;
		const int32 NumHeight = BoxGridDimensions.Z;

		Boxes->AddParticles(NumRows * NumCols * NumHeight);

		int32 Idx = 0;
		for (int32 Height = 0; Height < NumHeight; ++Height)
		{
			for (int32 Row = 0; Row < NumRows; ++Row)
			{
				for (int32 Col = 0; Col < NumCols; ++Col)
				{
					Boxes->SetX(Idx, FVec3(Col * 100, Row * 100, Height * 100) + Offset);
					Boxes->SetR(Idx, FRotation3::Identity);
					Boxes->SetGeometry(Idx, Box);
					++Idx;
				}
			}
		}

		return Boxes;
	}
	
	template <typename TSpatial>
	void SpatialTestHelper(TSpatial& Spatial, FGeometryParticles* Boxes, FImplicitObjectPtr& Box, FSpatialAccelerationIdx SpatialIdx = FSpatialAccelerationIdx())
	{
		//raycast
		//miss
		{
			FVisitor Visitor(FVec3(-100, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
			Spatial.Raycast(Visitor.Start, Visitor.Dir, 1000, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 0);
		}

		//gather along ray
		{
			FVisitor Visitor(FVec3(10, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
			Spatial.Raycast(Visitor.Start, Visitor.Dir, 1000, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 10);
		}

		//gather along ray and then make modifications
		{
			auto Spatial2 = Spatial.Copy();
			FVisitor Visitor(FVec3(10, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
			Spatial2->Raycast(Visitor.Start, Visitor.Dir, 1000, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 10);

			//remove from structure
			Spatial2->RemoveElementFrom(Visitor.Instances[0], SpatialIdx);

			FVisitor Visitor2(FVec3(10, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
			Spatial2->Raycast(Visitor2.Start, Visitor2.Dir, 1000, Visitor2);
			EXPECT_EQ(Visitor2.Instances.Num(), 9);

			//move instance away
			{
				const int32 MoveIdx = Visitor2.Instances[0];
				Boxes->SetX(MoveIdx, Boxes->GetX(MoveIdx) + FVec3(1000, 0, 0));
				FAABB3 NewBounds = Boxes->GetGeometry(MoveIdx)->template GetObject<TBox<FReal, 3>>()->BoundingBox().TransformedAABB(FRigidTransform3(Boxes->GetX(MoveIdx), Boxes->GetR(MoveIdx)));
				Spatial2->UpdateElementIn(MoveIdx, NewBounds, true, SpatialIdx);

				FVisitor Visitor3(FVec3(10, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
				Spatial2->Raycast(Visitor3.Start, Visitor3.Dir, 1000, Visitor3);
				EXPECT_EQ(Visitor3.Instances.Num(), 8);

				//move instance back
				Boxes->SetX(MoveIdx, Boxes->GetX(MoveIdx) - FVec3(1000, 0, 0));
				NewBounds = Boxes->GetGeometry(MoveIdx)->template GetObject<TBox<FReal, 3>>()->BoundingBox().TransformedAABB(FRigidTransform3(Boxes->GetX(MoveIdx), Boxes->GetR(MoveIdx)));
				Spatial2->UpdateElementIn(MoveIdx, NewBounds, true, SpatialIdx);

				FVisitor Visitor4(FVec3(10, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
				Spatial2->Raycast(Visitor4.Start, Visitor4.Dir, 1000, Visitor4);
				EXPECT_EQ(Visitor4.Instances.Num(), 9);
			}

			//move other instance into view
			{
				const int32 MoveIdx = 5 * 5 * 5;
				const FVec3 OldPos = Boxes->GetX(MoveIdx);
				Boxes->SetX(MoveIdx, FVec3(0, 0, 0));
				FAABB3 NewBounds = Boxes->GetGeometry(MoveIdx)->template GetObject<TBox<FReal, 3>>()->BoundingBox().TransformedAABB(FRigidTransform3(Boxes->GetX(MoveIdx), Boxes->GetR(MoveIdx)));
				Spatial2->UpdateElementIn(MoveIdx, NewBounds, true, SpatialIdx);

				FVisitor Visitor3(FVec3(10, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
				Spatial2->Raycast(Visitor3.Start, Visitor3.Dir, 1000, Visitor3);
				EXPECT_EQ(Visitor3.Instances.Num(), 10);

				//move instance back
				Boxes->SetX(MoveIdx, OldPos);
				NewBounds = Boxes->GetGeometry(MoveIdx)->template GetObject<TBox<FReal, 3>>()->BoundingBox().TransformedAABB(FRigidTransform3(Boxes->GetX(MoveIdx), Boxes->GetR(MoveIdx)));
				Spatial2->UpdateElementIn(MoveIdx, NewBounds, true, SpatialIdx);
			}

			//move instance outside of grid bounds
			{
				const int32 MoveIdx = 5 * 5 * 5;
				const FVec3 OldPos = Boxes->GetX(MoveIdx);
				Boxes->SetX(MoveIdx, FVec3(-50, 0, 0));
				FAABB3 NewBounds = Boxes->GetGeometry(MoveIdx)->template GetObject<TBox<FReal, 3>>()->BoundingBox().TransformedAABB(FRigidTransform3(Boxes->GetX(MoveIdx), Boxes->GetR(MoveIdx)));
				Spatial2->UpdateElementIn(MoveIdx, NewBounds, true, SpatialIdx);

				FVisitor Visitor3(FVec3(10, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
				Spatial2->Raycast(Visitor3.Start, Visitor3.Dir, 1000, Visitor3);
				EXPECT_EQ(Visitor3.Instances.Num(), 10);

				//try ray outside of bounds which should hit
				FVisitor Visitor4(FVec3(-20, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
				Spatial2->Raycast(Visitor4.Start, Visitor4.Dir, 1000, Visitor4);
				EXPECT_EQ(Visitor4.Instances.Num(), 1);

				//delete dirty instance
				Spatial2->RemoveElementFrom(MoveIdx, SpatialIdx);
				FVisitor Visitor5(FVec3(-20, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
				Spatial2->Raycast(Visitor5.Start, Visitor5.Dir, 1000, Visitor4);
				EXPECT_EQ(Visitor5.Instances.Num(), 0);

				//move instance back
				Boxes->SetX(MoveIdx, OldPos);

				//create a new box
				const int32 NewIdx = Boxes->Size();
				Boxes->AddParticles(1);
				Boxes->SetX(NewIdx, FVec3(-20, 0, 0));
				Boxes->SetR(NewIdx, FRotation3::Identity);
				Boxes->SetGeometry(NewIdx, Box);
				NewBounds = Boxes->GetGeometry(NewIdx)->template GetObject<TBox<FReal, 3>>()->BoundingBox().TransformedAABB(FRigidTransform3(Boxes->GetX(NewIdx), Boxes->GetR(NewIdx)));
				Spatial2->UpdateElementIn(NewIdx, NewBounds, true, SpatialIdx);
				FVisitor Visitor6(FVec3(-20, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
				Spatial2->Raycast(Visitor6.Start, Visitor6.Dir, 1000, Visitor6);
				EXPECT_EQ(Visitor6.Instances.Num(), 1);
			}
		}

		//stop half way through
		{
			FVisitor Visitor(FVec3(10, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
			Spatial.Raycast(Visitor.Start, Visitor.Dir, 499, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 5);
		}

		//any
		{
			FVisitor Visitor(FVec3(10, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
			Visitor.bAny = true;
			Spatial.Raycast(Visitor.Start, Visitor.Dir, 1000, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 1);
		}

		//sweep
		//miss
		{
			FVisitor Visitor(FVec3(-100, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
			Visitor.HalfExtents = FVec3(10, 0, 0);
			Spatial.Sweep(Visitor.Start, Visitor.Dir, 1000, Visitor.HalfExtents, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 0);
		}

		//gather along ray
		{
			FVisitor Visitor(FVec3(-100, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
			Visitor.HalfExtents = FVec3(110, 0, 0);
			Spatial.Sweep(Visitor.Start, Visitor.Dir, 1000, Visitor.HalfExtents, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 10);
		}

		//stop half way through
		{
			FVisitor Visitor(FVec3(-100, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
			Visitor.HalfExtents = FVec3(110, 0, 0);
			Spatial.Sweep(Visitor.Start, Visitor.Dir, 499, Visitor.HalfExtents, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 5);
		}

		//right on edge and corner
		{
			FVisitor Visitor(FVec3(100, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
			Visitor.HalfExtents = FVec3(10, 0, 0);
			Spatial.Sweep(Visitor.Start, Visitor.Dir, 499, Visitor.HalfExtents, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 10);
		}

		//overlap
		//miss
		{
			FOverlapVisitor Visitor(FAABB3(FVec3(-100, 0, 0), FVec3(-10, 0, 0)), *Boxes);
			Spatial.Overlap(Visitor.Bounds, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 0);
		}

		//overlap some
		{
			FOverlapVisitor Visitor(FAABB3(FVec3(-100, 0, -10), FVec3(110, 110, 10)), *Boxes);
			Spatial.Overlap(Visitor.Bounds, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 4);
		}

		//overlap any
		{
			FOverlapVisitor Visitor(FAABB3(FVec3(-100, 0, -10), FVec3(110, 110, 10)), *Boxes);
			Visitor.bAny = true;
			Spatial.Overlap(Visitor.Bounds, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 1);
		}
	}

	void GridBPTest()
	{
		FImplicitObjectPtr Box;
		auto Boxes = BuildBoxes(Box);
		TBoundingVolume<int32> Spatial(MakeParticleView(Boxes.Get()));
		SpatialTestHelper(Spatial, Boxes.Get(), Box);
	}

	void GridBPEarlyExitTest()
	{
		FImplicitObjectPtr Box;
		auto Boxes = BuildBoxes(Box);
		TBoundingVolume<int32> Spatial(MakeParticleView(Boxes.Get()));
		// SpatialTestHelper(Spatial, Boxes.Get(), Box);

		//gather along ray
		{
			FVisitor Visitor(FVec3(10, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
			Spatial.Raycast(Visitor.Start, Visitor.Dir, 1000, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 10);
			EXPECT_EQ(Visitor.Instances[0], 0);
			EXPECT_EQ(Visitor.Instances[9], 90);
		}

		// Stop after first hits in the first cell
		{
			FVisitor Visitor(FVec3(10, 0, 0), FVec3(0, 1, 0), 0, *Boxes);
			Visitor.BlockAfterN = 1;
			Spatial.Raycast(Visitor.Start, Visitor.Dir, 1000, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 1);
			EXPECT_EQ(Visitor.Instances[0], 0);
		}

		// Stop after first hits in the first cell, going backward
		{
			FVisitor Visitor(FVec3(10, 1000, 0), FVec3(0, -1, 0), 0, *Boxes);
			Visitor.BlockAfterN = 1;
			Spatial.Raycast(Visitor.Start, Visitor.Dir, 1000, Visitor);
			EXPECT_EQ(Visitor.Instances.Num(), 1);
			EXPECT_EQ(Visitor.Instances[0], 90);
		}
	}

	void GridBPTest2()
	{
		FImplicitObjectPtr Box( new TBox<FReal, 3>(FVec3(0, 0, 0), FVec3(100, 100, 100)));
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs SOAs(UniqueIndices);
		const int32 NumRows = 10;
		const int32 NumCols = 10;
		const int32 NumHeight = 10;

		SOAs.CreateStaticParticles(NumRows * NumCols * NumHeight);
		auto& Boxes = SOAs.GetNonDisabledStaticParticles();
		int32 Idx = 0;
		for (int32 Height = 0; Height < NumHeight; ++Height)
		{
			for (int32 Row = 0; Row < NumRows; ++Row)
			{
				for (int32 Col = 0; Col < NumCols; ++Col)
				{
					Boxes.SetX(Idx, FVec3(Col * 100, Row * 100, Height * 100));
					Boxes.SetR(Idx, FRotation3::Identity);
					Boxes.SetGeometry(Idx, Box);
					++Idx;
				}
			}
		}

		TArray<TSOAView<FGeometryParticles>> TmpArray = { &Boxes };
		TBoundingVolume<FGeometryParticleHandle*> BV(MakeParticleView(MoveTemp(TmpArray)));
		TArray<FGeometryParticleHandle*> Handles = BV.FindAllIntersections(FAABB3(FVec3(0), FVec3(10)));
		EXPECT_EQ(Handles.Num(), 1);
		EXPECT_EQ(Handles[0], Boxes.Handle(0));

		Handles = BV.FindAllIntersections(FAABB3(FVec3(0), FVec3(0, 0, 110)));
		EXPECT_EQ(Handles.Num(), 2);

		//create BV with an array of handles instead (useful for partial structures)
		{
			TBoundingVolume<FGeometryParticleHandle*> BV2(MakeHandleView(Handles));
			TArray<FGeometryParticleHandle*> Handles2 = BV2.FindAllIntersections(FAABB3(FVec3(0), FVec3(10)));
			EXPECT_EQ(Handles2.Num(), 1);
			EXPECT_EQ(Handles2[0], Boxes.Handle(0));

			Handles2 = BV2.FindAllIntersections(FAABB3(FVec3(0), FVec3(0, 0, 110)));
			EXPECT_EQ(Handles2.Num(), 2);
		}
	}

	void AABBTreeTest()
	{
		using TreeType = TAABBTree<int32, TBoundingVolume<int32>>;
		{
			FImplicitObjectPtr Box;
			auto Boxes = BuildBoxes(Box);
			TreeType Spatial(MakeParticleView(Boxes.Get()));

			while (!Spatial.IsAsyncTimeSlicingComplete())
			{
				Spatial.ProgressAsyncTimeSlicing(false);
			}

			SpatialTestHelper(Spatial, Boxes.Get(), Box);
		}

		{
			FImplicitObjectPtr Box;
			auto Boxes = BuildBoxes(Box);
			TreeType Spatial(MakeParticleView(Boxes.Get()));

			while (!Spatial.IsAsyncTimeSlicingComplete())
			{
				Spatial.ProgressAsyncTimeSlicing(false);
			}

			SpatialTestHelper(Spatial, Boxes.Get(), Box);
		}

		{
			//too many boxes so reoptimize
			FImplicitObjectPtr Box;
			auto Boxes = BuildBoxes(Box);
			TreeType Spatial(MakeParticleView(Boxes.Get()));

			while(!Spatial.IsAsyncTimeSlicingComplete())
			{
				Spatial.ProgressAsyncTimeSlicing(false);
			}

			EXPECT_EQ(Spatial.NumDirtyElements(),0);

			//fill up until dirty limit
			int32 Count;
			for(Count = 1; Count <= 10; ++Count)
			{
				auto Boxes2 = BuildBoxes(Box);
				for(uint32 Idx = 0; Idx < Boxes2->Size(); ++Idx)
				{
					Spatial.UpdateElement(Idx + Boxes->Size() * Count,Boxes2->WorldSpaceInflatedBounds(Idx),true);
				}

				EXPECT_EQ(Spatial.NumDirtyElements(), (Count)*Boxes->Size());
			}

			//finally pass dirty limit so reset to 0 and then add the remaining new boxes
			auto Boxes2 = BuildBoxes(Box);
			for(uint32 Idx = 0; Idx < Boxes2->Size(); ++Idx)
			{
				Spatial.UpdateElement(Idx + Boxes->Size() * (Count),Boxes2->WorldSpaceInflatedBounds(Idx),true);
			}

			EXPECT_EQ(Spatial.NumDirtyElements(),Boxes->Size() - 1);
		}
	}


	void AABBTreeTestDynamic()
	{
		using TreeType = TAABBTree<int32, TAABBTreeLeafArray<int32>, true>;
		{
			FImplicitObjectPtr Box;
			auto Boxes = BuildBoxes(Box, 100, FVec3(10,10,10));
			TreeType Spatial(MakeParticleView(Boxes.Get()), TreeType::DefaultMaxChildrenInLeaf, TreeType::DefaultMaxTreeDepth, TreeType::DefaultMaxPayloadBounds, TreeType::DefaultMaxNumToProcess, true);
			EXPECT_EQ(Spatial.NumDirtyElements(), 0);
			SpatialTestHelper(Spatial, Boxes.Get(), Box);
			EXPECT_EQ(Spatial.NumDirtyElements(), 0);
		}
		
	}

	void AABBTreeDirtyTreeTest()
	{
		using TreeType = TAABBTree<int32, TAABBTreeLeafArray<int32>, true>;

		// Do the standard tests
		{
			FImplicitObjectPtr Box;
			auto Boxes = BuildBoxes(Box);

			TArray<TSOAView<FGeometryParticles>> EmptyArray;
			TreeType Spatial{ MakeParticleView(MoveTemp(EmptyArray)),5, 5, 10000.0f, 1000, false, true};

			int32 Idx;
			for (Idx = 0; Idx < (int32)Boxes->Size(); ++Idx)
			{
				Spatial.UpdateElement(Idx, Boxes->WorldSpaceInflatedBounds(Idx), true);
			}		

			SpatialTestHelper(Spatial, Boxes.Get(), Box);
		}
	}

	void AABBTreeDirtyGridTest()
	{
		using TreeType = TAABBTree<int32, TBoundingVolume<int32>>;

		// Save CVARS
		int32 DirtyElementGridCellSize = FAABBTreeDirtyGridCVars::DirtyElementGridCellSize;
		int32 DirtyElementMaxGridCellQueryCount = FAABBTreeDirtyGridCVars::DirtyElementMaxGridCellQueryCount;
		int32 DirtyElementMaxPhysicalSizeInCells = FAABBTreeDirtyGridCVars::DirtyElementMaxPhysicalSizeInCells;
		int32 DirtyElementMaxCellCapacity = FAABBTreeDirtyGridCVars::DirtyElementMaxCellCapacity;

		// Set CVARS to known values
		FAABBTreeDirtyGridCVars::DirtyElementGridCellSize = 100;
		FAABBTreeDirtyGridCVars::DirtyElementMaxGridCellQueryCount = 10000;
		FAABBTreeDirtyGridCVars::DirtyElementMaxPhysicalSizeInCells = 20;
		FAABBTreeDirtyGridCVars::DirtyElementMaxCellCapacity = 20;


		// Do the standard tests
		{
			FImplicitObjectPtr Box;
			auto Boxes = BuildBoxes(Box);
			TreeType Spatial{};

			int32 Idx;
			for (Idx = 0; Idx < (int32)Boxes->Size(); ++Idx)
			{
				Spatial.UpdateElement(Idx, Boxes->WorldSpaceInflatedBounds(Idx), true);
			}
			EXPECT_EQ(Spatial.NumDirtyElements(), Boxes->Size());

			SpatialTestHelper(Spatial, Boxes.Get(), Box);
		}


		// Repeat the standard tests with low cell capacity and different cell sizes
		{
			// Set CVARS to known values
			FAABBTreeDirtyGridCVars::DirtyElementGridCellSize = 44;
			FAABBTreeDirtyGridCVars::DirtyElementMaxCellCapacity = 2;

			FImplicitObjectPtr Box;
			auto Boxes = BuildBoxes(Box);
			TreeType Spatial{};

			int32 Idx;
			for (Idx = 0; Idx < (int32)Boxes->Size(); ++Idx)
			{
				Spatial.UpdateElement(Idx, Boxes->WorldSpaceInflatedBounds(Idx), true);
			}
			EXPECT_EQ(Spatial.NumDirtyElements(), Boxes->Size());

			SpatialTestHelper(Spatial, Boxes.Get(), Box);
		}


		// Make sure we get the same results, with and without the grid for sweeps and raycasts
		{
			FAABBTreeDirtyGridCVars::DirtyElementMaxCellCapacity = 7;
			FImplicitObjectPtr Box;
			FVec3 LargeOffset(10000000, 10000000, 10000000); // Test for floating point precision errors at large world offsets
			auto Boxes = BuildBoxes(Box, 100, FVec3(40, 40, 1), FVec3(-2000, -2000, -50) + LargeOffset);

			for (float Angle = 0.0; Angle < 2 * PI; Angle += (10.0f / 360.0f) * 2.0f * PI)
			{

				FVec3 Direction{ FMath::Cos(Angle), FMath::Sin(Angle), 0 };
				// With the grid
				FVisitor VisitorGrid(FVec3(53, 27, 0) + LargeOffset, Direction, 0, *Boxes);
				VisitorGrid.HalfExtents = FVec3(102, 20, 2);
				{
					FAABBTreeDirtyGridCVars::DirtyElementGridCellSize = 100;

					TreeType Spatial{};

					int32 Idx;
					for (Idx = 0; Idx < (int32)Boxes->Size(); ++Idx)
					{
						Spatial.UpdateElement(Idx, Boxes->WorldSpaceInflatedBounds(Idx), true);
					}
					EXPECT_EQ(Spatial.NumDirtyElements(), Boxes->Size());

					Spatial.Raycast(VisitorGrid.Start, VisitorGrid.Dir, 1900, VisitorGrid);
					Spatial.Sweep(VisitorGrid.Start, VisitorGrid.Dir, 1800, VisitorGrid.HalfExtents, VisitorGrid);
				}

				// Without the grid
				FVisitor VisitorNoGrid(FVec3(53, 27, 0) + LargeOffset, Direction, 0, *Boxes);
				VisitorNoGrid.HalfExtents = FVec3(102, 20, 2);
				{
					FAABBTreeDirtyGridCVars::DirtyElementGridCellSize = 0;
					TreeType Spatial{};

					int32 Idx;
					for (Idx = 0; Idx < (int32)Boxes->Size(); ++Idx)
					{
						Spatial.UpdateElement(Idx, Boxes->WorldSpaceInflatedBounds(Idx), true);
					}
					EXPECT_EQ(Spatial.NumDirtyElements(), Boxes->Size());

					Spatial.Raycast(VisitorNoGrid.Start, VisitorNoGrid.Dir, 1900, VisitorNoGrid);
					Spatial.Sweep(VisitorNoGrid.Start, VisitorNoGrid.Dir, 1800, VisitorNoGrid.HalfExtents, VisitorNoGrid);
				}
				// These will be in the same order, but we can drop this requirement in the future
				EXPECT_TRUE(VisitorNoGrid.Instances == VisitorGrid.Instances);  
			}
			
		}

		// Test a case that failed before (with an assert)
		{
			FAABBTreeDirtyGridCVars::DirtyElementGridCellSize = 1000;
			FAABBTreeDirtyGridCVars::DirtyElementMaxCellCapacity = 7;
			
			FImplicitObjectPtr Box;
			auto Boxes = BuildBoxes(Box, 100, FVec3(1, 1, 1), FVec3(-3000, -1000, -50)); // Just one box
			TreeType Spatial{};
			Spatial.UpdateElement(0, Boxes->WorldSpaceInflatedBounds(0), true);

			// Move the Box
			Boxes = BuildBoxes(Box, 100, FVec3(1, 1, 1), FVec3(-4000, -1000, -50)); // Change position of box
			Spatial.UpdateElement(0, Boxes->WorldSpaceInflatedBounds(0), true); // Check for no ensures

			// Move the Box
			Boxes = BuildBoxes(Box, 100, FVec3(1, 1, 1), FVec3(3000, 1000, -50)); // Change position of box
			Spatial.UpdateElement(0, Boxes->WorldSpaceInflatedBounds(0), true);

			Boxes = BuildBoxes(Box, 100, FVec3(1, 1, 1), FVec3(4000, 1000, -50)); // Change position of box
			Spatial.UpdateElement(0, Boxes->WorldSpaceInflatedBounds(0), true); // Check for no ensures

			// Move the Box
			Boxes = BuildBoxes(Box, 100, FVec3(1, 1, 1), FVec3(-10000003000.0f, -1000, -50)); // Change position of box
			Spatial.UpdateElement(0, Boxes->WorldSpaceInflatedBounds(0), true); // Check for no ensures

			// Move the Box
			Boxes = BuildBoxes(Box, 100, FVec3(1, 1, 1), FVec3(-10000004000.0f, -1000, -50)); // Change position of box
			Spatial.UpdateElement(0, Boxes->WorldSpaceInflatedBounds(0), true); // Check for no ensures
			
		}
		

		// Restore CVARS
		FAABBTreeDirtyGridCVars::DirtyElementGridCellSize = DirtyElementGridCellSize;
		FAABBTreeDirtyGridCVars::DirtyElementMaxGridCellQueryCount = DirtyElementMaxGridCellQueryCount;
		FAABBTreeDirtyGridCVars::DirtyElementMaxPhysicalSizeInCells = DirtyElementMaxPhysicalSizeInCells;
		FAABBTreeDirtyGridCVars::DirtyElementMaxCellCapacity = DirtyElementMaxCellCapacity;
	}
	
	void DoForSweepIntersectCellsImpTest()
	{
		{
			int32 NumFuncCalled = 0;
			int32  XArray[2];
			int32  YArray[2];

			DoForSweepIntersectCellsImp(1.4484817992026819, 1.4432470701435705, 1251.1886035677471, -1183.6311465697545, -866.67708504199993, -747.83750413730752, 1000.0, 0.001,
				[&](auto X, auto Y) {
					XArray[NumFuncCalled] = X;
					YArray[NumFuncCalled] = Y;
					++NumFuncCalled;

				});

			EXPECT_EQ(NumFuncCalled, 2);
			EXPECT_EQ(XArray[0], 1000);
			EXPECT_EQ(YArray[0], -2000);

			EXPECT_EQ(XArray[1], 0);
			EXPECT_EQ(YArray[1], -2000);
		}

		{
			int32 NumFuncCalled = 0;
			int32  XArray[2];
			int32  YArray[2];

			DoForSweepIntersectCellsImp(1.4484817992026819, 1.4432470701435705, 1251.1886035677471, -1183.6311465697545, 866.67708504199993, -747.83750413730752, 1000.0, 0.001,
				[&](auto X, auto Y) {
					XArray[NumFuncCalled] = X;
					YArray[NumFuncCalled] = Y;
					++NumFuncCalled;

				});

			EXPECT_EQ(NumFuncCalled, 2);
			EXPECT_EQ(XArray[0], 1000);
			EXPECT_EQ(YArray[0], -2000);

			EXPECT_EQ(XArray[1], 2000);
			EXPECT_EQ(YArray[1], -2000);
		}

		{
			int32 NumFuncCalled = 0;
			int32  XArray[3];
			int32  YArray[3];

			DoForSweepIntersectCellsImp(1.2928878353696973, 1.2928878353697257, -1013.1421764369597, 210.55865232178132, 712.84350045280678, -265.39563631071809, 1000.0, 0.001,
				[&](auto X, auto Y) {
					XArray[NumFuncCalled] = X;
					YArray[NumFuncCalled] = Y;
					++NumFuncCalled;

				});

			EXPECT_EQ(NumFuncCalled, 3);
			EXPECT_EQ(XArray[0], -2000);
			EXPECT_EQ(YArray[0], 0);

			EXPECT_EQ(XArray[1], -1000);
			EXPECT_EQ(YArray[1], 0);

			EXPECT_EQ(XArray[2], -1000);
			EXPECT_EQ(YArray[2], -1000);

		}
		{
			int32 NumFuncCalled = 0;
			DoForSweepIntersectCellsImp(4000, 4000, 0, 0, 7000 - 0.01, 3000 - 0.01, 1000.0, 0.001,
				[&](auto X, auto Y) {
					++NumFuncCalled;

				});

			// This was verified manually on a paper grid
			EXPECT_EQ(NumFuncCalled, 153);

		}

	}


	void AABBTreeTimesliceTest()
	{
		using TreeType = TAABBTree<int32, TAABBTreeLeafArray<int32>>;

		FImplicitObjectPtr Box;

		// If we are time slicing by a milliseconds budget, create a large tree so it takes time to process 
		auto Boxes = FAABBTimeSliceCVars::bUseTimeSliceMillisecondBudget ? BuildBoxes(Box, 50, FVec3(50.0f,50.0f,50.0f)) : BuildBoxes(Box) ;	

		// build AABB in one go
		TreeType SpatialBuildImmediate(
			MakeParticleView(Boxes.Get()) 
			, TreeType::DefaultMaxChildrenInLeaf
			, TreeType::DefaultMaxTreeDepth
			, TreeType::DefaultMaxPayloadBounds
			, 0); // build entire tree in one go, no timeslicing

		EXPECT_TRUE(SpatialBuildImmediate.IsAsyncTimeSlicingComplete());

		const double SlicedTreeGenerationStartTime = FPlatformTime::Seconds();
	
		// build AABB in time-sliced sections
		TreeType SpatialTimesliced(
			MakeParticleView(Boxes.Get())
			, TreeType::DefaultMaxChildrenInLeaf
			, TreeType::DefaultMaxTreeDepth
			, TreeType::DefaultMaxPayloadBounds
			, 20); // build in small iteration steps, 20 iterations per call to ProgressAsyncTimeSlicing

		EXPECT_FALSE(!FAABBTimeSliceCVars::bUseTimeSliceMillisecondBudget && SpatialTimesliced.IsAsyncTimeSlicingComplete());	

		// This is far from accurate, but give us some wiggle room to test it with default settings without needed to implement code to simulate a precise pause inside the Tree implementation.
		const float MaxSliceDurationWithErrorMargin = FAABBTimeSliceCVars::MaxProcessingTimePerSliceSeconds + 0.01f;
		double LargestSliceDuration = 0;

		int32 IterationNumber = 1;
		bool bSliceDoneWithinBudget = true;

		while (!SpatialTimesliced.IsAsyncTimeSlicingComplete())
		{
			const double SliceStartTime = FPlatformTime::Seconds();
	
			SpatialTimesliced.ProgressAsyncTimeSlicing(false);
			
			if (FAABBTimeSliceCVars::bUseTimeSliceMillisecondBudget)
			{
				const double ElapsedTime = FPlatformTime::Seconds() - SliceStartTime;

				bSliceDoneWithinBudget &= ElapsedTime < MaxSliceDurationWithErrorMargin;

				LargestSliceDuration = FMath::Max(LargestSliceDuration, ElapsedTime);
			}

			IterationNumber++;
		}

		EXPECT_TRUE(bSliceDoneWithinBudget);

		const FStringView TimeSliceMode = FAABBTimeSliceCVars::bUseTimeSliceMillisecondBudget ? TEXT("MillisecondsBudget") : TEXT("AmountOfWorkToDo");
		const double TotalGenerationTime = FPlatformTime::Seconds() - SlicedTreeGenerationStartTime;

		UE_LOGF(LogHeadlessChaos, Verbose, "Time Sliced Tree Generation took [%f] seconds | Using Mode [%ls] | In [%d] Iterations | LargestSliceDuration [%f] | EvaluatedMaxTimeSlicedDurarion [%f]", TotalGenerationTime, TimeSliceMode.GetData(), IterationNumber, LargestSliceDuration, MaxSliceDurationWithErrorMargin);

		// now check both AABBs have the same hierarchy
		// (indices will be different but walking tree should give same results)

		FAABB3 Tmp = FAABB3::ZeroAABB();

		TArray<FAABB3> AllBoundsBuildImmediate;
		SpatialBuildImmediate.GetAsBoundsArray(AllBoundsBuildImmediate, 0, -1, Tmp);

		TArray<FAABB3> AllBoundsTimesliced;
		SpatialTimesliced.GetAsBoundsArray(AllBoundsTimesliced, 0, -1, Tmp);

		EXPECT_EQ(AllBoundsBuildImmediate.Num(), AllBoundsTimesliced.Num());

		for (int i=0; i<AllBoundsBuildImmediate.Num(); i++)
		{
			EXPECT_EQ(AllBoundsBuildImmediate[i].Center(), AllBoundsTimesliced[i].Center());
			EXPECT_EQ(AllBoundsBuildImmediate[i].Extents(), AllBoundsTimesliced[i].Extents());
		}
	}

	// specific  sweep query seems to generate infinite loop
	// the values are outside of the function to avoid optimizing away the values
	// (the issue would only appear when optimization is enable)
	namespace EdgeCase
	{
		FVec3 QueryHalfExtents{ 5.51277924f, 4.77557945f, 4.96569443f };
		FVec3 StartPoint{ -40.7950134f, -4.77560043f, -11.2947388f };
		FVec3 Dir{ 0, 0, -1 };
		FReal CurrentLength = 146.779785f;
	}

	namespace LargeSweep
	{
		const FVec3 QueryHalfExtents{34, 34, 90};
		const FVec3 StartPoint{5000000000270, 11630, 187.15000295639038};
		const FVec3 Dir{-1.0000000172032004, 0, 0.00000000000040509186487903264};
		const FReal CurrentLength = 4999999913984;
	};
	
	void AABBTreeDirtyGridFunctionsWithEdgeCase()
	{
		constexpr FReal DirtyElementGridCellSize = 1000.0;
		constexpr FReal DirtyElementGridCellSizeInv = 1.0 / DirtyElementGridCellSize;
		constexpr int32 DirtyElementMaxGridCellQueryCount = 340;

		TArray<TVec2<FReal>> VisitedCells;
		// this should report only be as much as 4 hits as teh grid is 2D and the ray downward vertical and the halfextends are smaller than a cell size
		{
			DoForSweepIntersectCells(EdgeCase::QueryHalfExtents, EdgeCase::StartPoint, EdgeCase::Dir, EdgeCase::CurrentLength, DirtyElementGridCellSize, DirtyElementGridCellSizeInv,
				[&VisitedCells](FReal X, FReal Y)
				{
					VisitedCells.Add({ X,Y });
					EXPECT_TRUE(VisitedCells.Num() <= 4 );
				});
			EXPECT_TRUE(VisitedCells.Num() <= 4);
		}

		// 50 trillions unit long sweep test reporting that there's not too many cells
		{
			const bool bTooManyCell = TooManySweepQueryCells(LargeSweep::QueryHalfExtents, LargeSweep::StartPoint, LargeSweep::Dir, LargeSweep::CurrentLength, DirtyElementGridCellSizeInv, DirtyElementMaxGridCellQueryCount);
			EXPECT_TRUE(bTooManyCell);
		}
	}
	
	void BroadphaseCollectionTest()
	{
		using TreeType = TAABBTree<int32, TAABBTreeLeafArray<int32>>;
		{
			FImplicitObjectPtr Box;
			auto Boxes = BuildBoxes(Box);
			auto Spatial = MakeUnique<TreeType>(MakeParticleView(Boxes.Get()));

			while (!Spatial->IsAsyncTimeSlicingComplete())
			{
				Spatial->ProgressAsyncTimeSlicing(false);
			}

			TSpatialAccelerationCollection<TreeType> AccelerationCollection;
			AccelerationCollection.AddSubstructure(MoveTemp(Spatial), 0, 0);
			FSpatialAccelerationIdx SpatialIdx = { 0,0 };
			SpatialTestHelper(AccelerationCollection, Boxes.Get(), Box, SpatialIdx);
		}

		{
			using BVType = TBoundingVolume<int32>;
			FImplicitObjectPtr Box;
			auto Boxes0 = BuildBoxes(Box);
			auto Spatial0 = MakeUnique<TreeType>(MakeParticleView(Boxes0.Get()));
			while (!Spatial0->IsAsyncTimeSlicingComplete())
			{
				Spatial0->ProgressAsyncTimeSlicing(false);
			}

			FGeometryParticles EmptyBoxes;
			auto Spatial1 = MakeUnique<BVType>(MakeParticleView(&EmptyBoxes));
			while (!Spatial1->IsAsyncTimeSlicingComplete())
			{
				Spatial1->ProgressAsyncTimeSlicing(false);
			}

			TSpatialAccelerationCollection<TreeType, BVType> AccelerationCollection;
			AccelerationCollection.AddSubstructure(MoveTemp(Spatial0), 0, 0);
			AccelerationCollection.AddSubstructure(MoveTemp(Spatial1), 1, 0);

			FSpatialAccelerationIdx SpatialIdx = { 0,0 };
			SpatialTestHelper(AccelerationCollection, Boxes0.Get(), Box, SpatialIdx);
		}

		{
			using BVType = TBoundingVolume<int32>;
			FImplicitObjectPtr Box;
			auto Boxes1 = BuildBoxes(Box);
			FGeometryParticles EmptyBoxes;

			auto Spatial0 = MakeUnique<TreeType>(MakeParticleView(&EmptyBoxes));
			auto Spatial1 = MakeUnique<BVType>(MakeParticleView(Boxes1.Get()));

			TSpatialAccelerationCollection<TreeType, BVType> AccelerationCollection;
			AccelerationCollection.AddSubstructure(MoveTemp(Spatial0), 0, 0);
			AccelerationCollection.AddSubstructure(MoveTemp(Spatial1), 1, 0);

			FSpatialAccelerationIdx SpatialIdx = { 1,0 };
			SpatialTestHelper(AccelerationCollection, Boxes1.Get(), Box, SpatialIdx);
		}
	}

	// Verify we don't generate a NaN or invalid bounds if we build BoundingVolume with particles that have no bounds.
	void BoundingVolumeNoBoundsTest()
	{
		FImplicitObjectPtr Box( new TBox<FReal, 3>(FVec3(0, 0, 0), FVec3(100)));
		auto Boxes = MakeUnique<FGeometryParticles>();

		Boxes->AddParticles(1);

		// Construct a particle and set HasBounds to false.
		int32 Idx = 0;
		Boxes->SetX(Idx, FVec3(0));
		Boxes->SetR(Idx, FRotation3::Identity);
		Boxes->SetGeometry(Idx, Box);

		// Tell BV we have no bounds, this used to cause issues.
		Boxes->HasBounds(Idx) = false;

		// Make Bounding Volume with only particles that have no bounds.
		auto Spatial1 = MakeUnique<TBoundingVolume<int32>>(MakeParticleView(Boxes.Get()));

		EXPECT_EQ(Spatial1->GetBounds().Min().ContainsNaN(), false);
		EXPECT_EQ(Spatial1->GetBounds().Max().ContainsNaN(), false);
		EXPECT_EQ(Spatial1->GetBounds().Extents().ContainsNaN(), false);
	}


	void SpatialAccelerationDirtyAndGlobalQueryStrestTest()
	{
		using AABBTreeType = TAABBTree<FAccelerationStructureHandle, TAABBTreeLeafArray<FAccelerationStructureHandle>>;

		// Construct 100000 Particles
		const int32 NumRows = 100;
		const int32 NumCols = 100;
		const int32 NumHeight = 10;
		const int32 ParticleCount = NumRows * NumCols * NumHeight;
		const FReal BoxSize = 100;

		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		TArray<FPBDRigidParticleHandle*> ParticleHandles = Particles.CreateDynamicParticles(ParticleCount);
		for (auto& Handle : ParticleHandles)
		{
			Handle->GTGeometryParticle() = FGeometryParticle::CreateParticle().Release();
		}
		const auto& ParticlesView = Particles.GetAllParticlesView();

		// ensure these can't be filtered out.
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		Chaos::Filter::FShapeFilterBuilder Builder;
		Builder.SetBlockChannelMask(TNumericLimits<uint64>::Max());
		Builder.SetOverlapChannelMask(TNumericLimits<uint64>::Max());
		const Chaos::Filter::FShapeFilterData ShapeFilterData = Builder.Build();
		PRAGMA_ENABLE_INTERNAL_WARNINGS

		Chaos::FImplicitObjectPtr Box( new TBox<FReal, 3>(FVec3(0, 0, 0), FVec3(BoxSize, BoxSize, BoxSize)));

		int32 Idx = 0;
		for (int32 Height = 0; Height < NumHeight; ++Height)
		{
			for (int32 Row = 0; Row < NumRows; ++Row)
			{
				for (int32 Col = 0; Col < NumCols; ++Col)
				{
					FGeometryParticle* GTParticle = ParticleHandles[Idx]->GTGeometryParticle();
					FPBDRigidParticleHandle* Handle = ParticleHandles[Idx];

					Handle->SetX(FVec3(Col * BoxSize, Row * BoxSize, Height * BoxSize));
					GTParticle->SetX(FVec3(Col * BoxSize, Row * BoxSize, Height * BoxSize));
					Handle->SetR(FRotation3::Identity);
					GTParticle->SetR(FRotation3::Identity);
					Handle->SetGeometry(Box);
					Handle->ShapesArray()[0]->SetShapeFilterData(ShapeFilterData);
					GTParticle->SetGeometry(Box);
					GTParticle->ShapesArray()[0]->SetShapeFilterData(ShapeFilterData);
					Handle->SetUniqueIdx(FUniqueIdx(Idx));
					GTParticle->SetUniqueIdx(FUniqueIdx(Idx));
					++Idx;
				}
			}
		}

		int32 DirtyNum = 800;
		int32 Queries = 500;
		ensure(DirtyNum < ParticleCount);

		// Construct tree
		AABBTreeType Spatial(ParticlesView);

		// Update DirtyNum elements, so they are pulled out of leaves.
		for (int32 i = 0; i < DirtyNum; ++i)
		{
			FAccelerationStructureHandle Payload(ParticleHandles[i]->GTGeometryParticle());
			FAABB3 Bounds = ParticleHandles[i]->WorldSpaceInflatedBounds();
			Spatial.UpdateElement(Payload, Bounds, true);
		}

		// RAYCASTS
		{
			// Setup raycast params
			const FVec3 Start(500, 500, 500);
			const FVec3 Dir(1, 0, 0);
			const FReal Length = 1000;
			FStressTestVisitor Visitor;

			// Measure raycasts
			uint32 Cycles = 0.0;
			for (int32 Query = 0; Query < Queries; ++Query)
			{
				uint32 StartTime = FPlatformTime::Cycles();

				Spatial.Raycast(Start, Dir, Length, Visitor);

				Cycles += FPlatformTime::Cycles() - StartTime;
			}

			float Milliseconds = FPlatformTime::ToMilliseconds(Cycles);
			float AvgMicroseconds = (Milliseconds * 1000) / Queries;

			UE_LOGF(LogHeadlessChaos, Warning, "Raycast Test: Dirty Particles: %d, Queries: %d, Avg Query Time: %f(us), Total:%f(ms)", DirtyNum, Queries, AvgMicroseconds, Milliseconds);
		}

		// SWEEPS
		{
			// Setup Sweep params
			const FVec3 Start(500, 500, 500);
			const FVec3 Dir(1, 0, 0);
			const FReal Length = 1000;
			const FVec3 HalfExtents(50, 50, 50);
			FStressTestVisitor Visitor;

			// Measure raycasts
			uint32 Cycles = 0.0;
			for (int32 Query = 0; Query < Queries; ++Query)
			{
				uint32 StartTime = FPlatformTime::Cycles();

				Spatial.Sweep(Start, Dir, Length, HalfExtents, Visitor);

				Cycles += FPlatformTime::Cycles() - StartTime;
			}

			float Milliseconds = FPlatformTime::ToMilliseconds(Cycles);
			float AvgMicroseconds = (Milliseconds * 1000) / Queries;

			UE_LOGF(LogHeadlessChaos, Warning, "Sweep Test: Dirty Particles: %d, Queries: %d, Avg Query Time: %f(us), Total:%f(ms)", DirtyNum, Queries, AvgMicroseconds, Milliseconds);
		}

		// OVERLAPS
		{
			FStressTestVisitor Visitor;
			const FAABB3 QueryBounds(FVec3(-50, -50, -50), FVec3(50,50,50));

			// Measure raycasts
			uint32 Cycles = 0.0;
			for (int32 Query = 0; Query < Queries; ++Query)
			{
				uint32 StartTime = FPlatformTime::Cycles();

				Spatial.Overlap(QueryBounds, Visitor);

				Cycles += FPlatformTime::Cycles() - StartTime;
			}

			float Milliseconds = FPlatformTime::ToMilliseconds(Cycles);
			float AvgMicroseconds = (Milliseconds * 1000) / Queries;

			UE_LOGF(LogHeadlessChaos, Error, "Overlap Test: Dirty Particles: %d, Queries: %d, Avg Query Time: %f(us), Total:%f(ms)", DirtyNum, Queries, AvgMicroseconds, Milliseconds);
		}
	}

}
