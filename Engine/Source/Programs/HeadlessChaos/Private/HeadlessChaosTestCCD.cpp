// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Matrix.h"
#include "Chaos/Utilities.h"
#include "Chaos/AABB.h"
#include "Chaos/Core.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/Box.h"
#include "Chaos/Sphere.h"
#include "ChaosSolversModule.h"
#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Modules/ModuleManager.h"
#include "PBDRigidsSolver.h"

//PRAGMA_DISABLE_OPTIMIZATION

namespace ChaosTest
{
	using namespace Chaos;

	GTEST_TEST(CCDTests, CCDDisabled)
	{
		const FReal Dt = 1 / 30.0f;
		const FReal Fps = 1 / Dt;
		const FReal BoxHalfSize = 50; // cm
		const FReal InitialSpeed = BoxHalfSize * 5 * Fps; // More than enough to tunnel
		const FReal InitialPosition = BoxHalfSize * 2 + 30;
		using TEvolution = FPBDRigidsEvolutionGBF;
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		TEvolution Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		// Create particles
		TGeometryParticleHandle<FReal, 3>* Static = Evolution.CreateStaticParticles(1)[0];
		TPBDRigidParticleHandle<FReal, 3>* Dynamic = Evolution.CreateDynamicParticles(1)[0];

		// Set up physics material
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->SleepCounterThreshold = 1000; // Don't sleep
		PhysicsMaterial->Restitution = 1.0f;

		// Create box geometry 
		Chaos::FImplicitObjectPtr SmallBox(new TBox<FReal, 3>(FVec3(-BoxHalfSize, -BoxHalfSize, -BoxHalfSize), FVec3(BoxHalfSize, BoxHalfSize, BoxHalfSize)));
		Static->SetGeometry(SmallBox);
		AppendDynamicParticleConvexBox(*Dynamic, FVec3(BoxHalfSize), 0.0f);

		Evolution.SetPhysicsMaterial(Dynamic, MakeSerializable(PhysicsMaterial));

		const FRealSingle Mass = 100000.0f;;
		Dynamic->I() = TVec3<FRealSingle>(Mass, Mass, Mass);
		Dynamic->InvI() = TVec3<FRealSingle>(1.0f / Mass, 1.0f / Mass, 1.0f / Mass);

		// Positions and velocities
		Static->SetX(FVec3(0, 0, 0));
		Dynamic->SetX( FVec3(0, 0, InitialPosition)); // Start 30cm above the static box
		Dynamic->SetV(FVec3(0, 0, -InitialSpeed));

		// The position of the static has changed and statics don't automatically update bounds, so update explicitly
		Static->UpdateWorldSpaceState(TRigidTransform<FReal, 3>(Static->GetX(), Static->GetR()), FVec3(0));

		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		Dynamic->SetCCDEnabled(false);
		Dynamic->SetGravityEnabled(false);

		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);


		Dynamic->SetV(FVec3(0, 0, -InitialSpeed));

		for (int i = 0; i < 1; ++i)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		// Large error margin, we are testing CCD and not solver accuracy
		// The Box should pass right through the other one without interacting
		const FReal LargeErrorMargin = 10.0f;
		EXPECT_NEAR(Dynamic->GetX()[2], InitialPosition - InitialSpeed * Dt, LargeErrorMargin);
	}

	GTEST_TEST(CCDTests, ConvexConvex)
	{
		const FReal Dt = 1 / 30.0f;
		const FReal Fps = 1 / Dt;
		const FReal BoxHalfSize = 50; // cm
		const FReal InitialSpeed = (30.0f + BoxHalfSize * 5) * Fps; // More than enough to tunnel
		using TEvolution = FPBDRigidsEvolutionGBF;
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		TEvolution Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		// Create particles
		TGeometryParticleHandle<FReal, 3>* Static = Evolution.CreateStaticParticles(1)[0];
		TPBDRigidParticleHandle<FReal, 3>* Dynamic = Evolution.CreateDynamicParticles(1)[0];

		// Set up physics material
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->SleepCounterThreshold = 1000; // Don't sleep
		PhysicsMaterial->Restitution = 1.0f;

		// Create box geometry 
		Chaos::FImplicitObjectPtr SmallBox(new TBox<FReal, 3>(FVec3(-BoxHalfSize, -BoxHalfSize, -BoxHalfSize), FVec3(BoxHalfSize, BoxHalfSize, BoxHalfSize)));
		Static->SetGeometry(SmallBox);
		AppendDynamicParticleConvexBox(*Dynamic, FVec3(BoxHalfSize), 0.0f);

		Evolution.SetPhysicsMaterial(Dynamic, MakeSerializable(PhysicsMaterial));

		const FRealSingle Mass = 100000.0f;
		Dynamic->I() = TVec3<FRealSingle>(Mass, Mass, Mass);
		Dynamic->InvI() = TVec3<FRealSingle>(1.0f / Mass, 1.0f / Mass, 1.0f / Mass);

		// Positions and velocities
		Static->SetX(FVec3(0, 0, 0));
		Dynamic->SetX(FVec3(0, 0, BoxHalfSize * 2 + 30)); // Start 30cm above the static box
		Dynamic->SetV(FVec3(0, 0, -InitialSpeed));

		// The position of the static has changed and statics don't automatically update bounds, so update explicitly
		Static->UpdateWorldSpaceState(TRigidTransform<FReal, 3>(Static->GetX(), Static->GetR()), FVec3(0));

		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		Dynamic->SetCCDEnabled(true);
		Dynamic->SetGravityEnabled(false);


		Dynamic->SetV(FVec3(0, 0, -InitialSpeed));

		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		for (int i = 0; i < 1; ++i)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		// Large error margin, we are testing CCD and not solver accuracy
		const FReal LargeErrorMargin = 10.0f;
		EXPECT_GE(Dynamic->GetX()[2], BoxHalfSize * 2 - LargeErrorMargin);
	}

	// CCD not implemented for sphere sphere
	GTEST_TEST(CCDTests,DISABLED_SphereSphere)
	{
		const FReal Dt = 1 / 30.0f;
		const FReal Fps = 1 / Dt;
		const FReal SphereRadius = 100; // cm
		const FReal InitialSpeed = SphereRadius * 5 * Fps; // More than enough to tunnel
		using TEvolution = FPBDRigidsEvolutionGBF;
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		TEvolution Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		// Create particles
		TGeometryParticleHandle<FReal, 3>* Static = Evolution.CreateStaticParticles(1)[0];
		TPBDRigidParticleHandle<FReal, 3>* Dynamic = Evolution.CreateDynamicParticles(1)[0];

		// Set up physics material
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->SleepCounterThreshold = 1000; // Don't sleep
		PhysicsMaterial->Restitution = 1.0f;

		// Create Sphere geometry (Radius = 100)
		Chaos::FImplicitObjectPtr Sphere(new Chaos::FSphere(FVec3(0, 0, 0), SphereRadius));

		// Assign sphere geometry to both particles 
		Static->SetGeometry(Sphere);
		Dynamic->SetGeometry(Sphere);

		Evolution.SetPhysicsMaterial(Dynamic, MakeSerializable(PhysicsMaterial));

		const FRealSingle Mass = 100000.0f;
		Dynamic->I() = TVec3<FRealSingle>(Mass);
		Dynamic->InvI() = TVec3<FRealSingle>(1.0f / Mass);

		// Positions and velocities
		Static->SetX(FVec3(0, 0, 0));

		Dynamic->SetX(FVec3(0, 0, SphereRadius * 2 + 10));
		Dynamic->SetV(FVec3(0, 0, -InitialSpeed));
		
		// The position of the static has changed and statics don't automatically update bounds, so update explicitly
		Static->UpdateWorldSpaceState(TRigidTransform<FReal, 3>(Static->GetX(), Static->GetR()), FVec3(0));

		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		Dynamic->SetCCDEnabled(true);
		Dynamic->SetGravityEnabled(false);
		

		Dynamic->SetV(FVec3(0, 0, -InitialSpeed));

		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		for (int i = 0; i < 1; ++i)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		// Large error margin, we are testing CCD and not solver accuracy
		const FReal LargeErrorMargin = 10.0f;
		EXPECT_GE(Dynamic->GetX()[2], SphereRadius * 2 - LargeErrorMargin);
	}

	
	// Bounce a box within a box container without penetrating the container
	GTEST_TEST(CCDTests, BoxStayInsideBoxBoundaries)
	{
		const FReal Dt = 1 / 30.0f;
		const FReal Fps = 1 / Dt;
		const FReal SmallBoxHalfSize = 50; // cm
		const FReal ContainerBoxHalfSize = 250; // cm
		const FReal ContainerWallThickness = 10; // cm
		const int ContainerFaceCount = 6;

		const FVec3 InitialVelocity = FVec3(-ContainerBoxHalfSize * Fps * 10, 0 ,0);

		using TEvolution = FPBDRigidsEvolutionGBF;
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		TEvolution Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		// Create particles
		TArray <TGeometryParticleHandle<FReal, 3>*> ContainerFaces = Evolution.CreateStaticParticles(ContainerFaceCount); // 6 sides of box
		TPBDRigidParticleHandle<FReal, 3>* Dynamic = Evolution.CreateDynamicParticles(1)[0]; // The small box

		// Set up physics material
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->SleepCounterThreshold = 1000; // Don't sleep
		PhysicsMaterial->Restitution = 0.2f;// Bounce against the walls a few times

		// Create box geometry 
		//Chaos::FImplicitObjectPtr SmallBox(new TBox<FReal, 3>(FVec3(-SmallBoxHalfSize, -SmallBoxHalfSize, -SmallBoxHalfSize), FVec3(SmallBoxHalfSize, SmallBoxHalfSize, SmallBoxHalfSize)));
		AppendDynamicParticleConvexBox(*Dynamic, FVec3(SmallBoxHalfSize), 0.0f);

		// Just use 3 (x2) boxes for the walls of the container (avoid rotation transforms for this test)
		Chaos::FImplicitObjectPtr ContainerFaceX(new TBox<FReal, 3>(FVec3(-ContainerWallThickness / 2, -ContainerBoxHalfSize, -ContainerBoxHalfSize), FVec3(ContainerWallThickness / 2, ContainerBoxHalfSize, ContainerBoxHalfSize)));
		Chaos::FImplicitObjectPtr ContainerFaceY(new TBox<FReal, 3>(FVec3(-ContainerBoxHalfSize, -ContainerWallThickness / 2, -ContainerBoxHalfSize), FVec3(ContainerBoxHalfSize, ContainerWallThickness / 2, ContainerBoxHalfSize)));
		Chaos::FImplicitObjectPtr ContainerFaceZ(new TBox<FReal, 3>(FVec3(-ContainerBoxHalfSize, -ContainerBoxHalfSize, -ContainerWallThickness / 2), FVec3(ContainerBoxHalfSize, ContainerBoxHalfSize, ContainerWallThickness / 2)));		

		

		ContainerFaces[0]->SetGeometry(ContainerFaceX);
		ContainerFaces[1]->SetGeometry(ContainerFaceX);
		ContainerFaces[2]->SetGeometry(ContainerFaceY);
		ContainerFaces[3]->SetGeometry(ContainerFaceY);
		ContainerFaces[4]->SetGeometry(ContainerFaceZ);
		ContainerFaces[5]->SetGeometry(ContainerFaceZ);
		

		Evolution.SetPhysicsMaterial(Dynamic, MakeSerializable(PhysicsMaterial));

		const FRealSingle Mass = 100000.0f;
		Dynamic->I() = TVec3<FRealSingle>(Mass);
		Dynamic->InvI() = TVec3<FRealSingle>(1.0f / Mass);

		// Positions and velocities
		ContainerFaces[0]->SetX(FVec3(ContainerBoxHalfSize, 0, 0));
		ContainerFaces[1]->SetX(FVec3(-ContainerBoxHalfSize, 0, 0));
		ContainerFaces[2]->SetX(FVec3(0, ContainerBoxHalfSize, 0));
		ContainerFaces[3]->SetX(FVec3(0, -ContainerBoxHalfSize, 0));
		ContainerFaces[4]->SetX(FVec3(0, 0, ContainerBoxHalfSize));
		ContainerFaces[5]->SetX(FVec3(0, 0, -ContainerBoxHalfSize));

		Dynamic->SetX(FVec3(0, 0, 0));

		// The position of the static has changed and statics don't automatically update bounds, so update explicitly
		ContainerFaces[0]->UpdateWorldSpaceState(TRigidTransform<FReal, 3>(ContainerFaces[0]->GetX(), ContainerFaces[0]->GetR()), FVec3(0));
		ContainerFaces[1]->UpdateWorldSpaceState(TRigidTransform<FReal, 3>(ContainerFaces[1]->GetX(), ContainerFaces[1]->GetR()), FVec3(0));
		ContainerFaces[2]->UpdateWorldSpaceState(TRigidTransform<FReal, 3>(ContainerFaces[2]->GetX(), ContainerFaces[2]->GetR()), FVec3(0));
		ContainerFaces[3]->UpdateWorldSpaceState(TRigidTransform<FReal, 3>(ContainerFaces[3]->GetX(), ContainerFaces[3]->GetR()), FVec3(0));
		ContainerFaces[4]->UpdateWorldSpaceState(TRigidTransform<FReal, 3>(ContainerFaces[4]->GetX(), ContainerFaces[4]->GetR()), FVec3(0));
		ContainerFaces[5]->UpdateWorldSpaceState(TRigidTransform<FReal, 3>(ContainerFaces[5]->GetX(), ContainerFaces[5]->GetR()), FVec3(0));

		::ChaosTest::SetParticleSimDataToCollide({ Dynamic });
		::ChaosTest::SetParticleSimDataToCollide({ ContainerFaces });

		Dynamic->SetCCDEnabled(true);
		Dynamic->SetGravityEnabled(false);

		// IMPORTANT : this is required to make sure the particles internal representation will reflect the sim data
		Evolution.EnableParticle(ContainerFaces[0]);
		Evolution.EnableParticle(ContainerFaces[1]);
		Evolution.EnableParticle(ContainerFaces[2]);
		Evolution.EnableParticle(ContainerFaces[3]);
		Evolution.EnableParticle(ContainerFaces[4]);
		Evolution.EnableParticle(ContainerFaces[5]);
		Evolution.EnableParticle(Dynamic);

		Dynamic->SetV(InitialVelocity);
		///////////////////////////////////
		// Test 1: bouncing from two opposite walls
		for (int i = 0; i < 10; ++i)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		// Large error margin, we are testing CCD and not solver accuracy
		const FReal LargeErrorMargin = 10.0f;
		// Check that we did not escape the box!
		for (int axis = 0; axis < 3; axis++)
		{
			// If this failed, the dynamic cube escaped the air tight static container
			const  FReal MaxCoordinates = ContainerBoxHalfSize - ContainerWallThickness / 2 - SmallBoxHalfSize;
			EXPECT_LT(FMath::Abs(Dynamic->GetX()[axis]), MaxCoordinates + LargeErrorMargin);
		}
		/////////////////////////////////////////////
		// Test2: Now launch to cube to a corner
		Dynamic->SetVf(FVec3f(static_cast<FRealSingle>(-ContainerBoxHalfSize * Fps * 10)));
		Dynamic->SetWf(FVec3f(0));
		Dynamic->SetX(FVec3(0));
		Dynamic->SetP(FVec3(0));
		Dynamic->SetRf(TRotation<FRealSingle, 3>::FromIdentity());
		Dynamic->SetQf(TRotation<FRealSingle, 3>::FromIdentity());

		for (int i = 0; i < 10; ++i)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		// Check that we did not escape the box!
		for (int axis = 0; axis < 3; axis++)
		{
			// If this failed, the dynamic cube escaped the air tight static container
			const  FReal MaxCoordinates = ContainerBoxHalfSize - ContainerWallThickness / 2 - SmallBoxHalfSize;
			EXPECT_LT(FMath::Abs(Dynamic->GetX()[axis]), MaxCoordinates + LargeErrorMargin);
		}

		/////////////////////////////////////////////////////////////////////
		// Test 3: Now we give it something impossible to solve with PBD solver (restitution of 1, high velocities causes final position to be outside of box). 
		// Make sure it still stays inside the box (albeit with a very reduced velocity) 
		Dynamic->SetV(InitialVelocity);
		Dynamic->SetW(FVec3(0));
		Dynamic->SetX(FVec3(0));
		Dynamic->SetP(FVec3(0));
		Dynamic->SetR(TRotation<FReal, 3>::FromIdentity());
		Dynamic->SetQ(TRotation<FReal, 3>::FromIdentity());
		PhysicsMaterial->Restitution = 0.9f;

		for (int i = 0; i < 10; ++i) // To fix this unit test
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		// Check that we did not escape the box!
		for (int axis = 0; axis < 3; axis++)
		{
			// If this failed, the dynamic cube escaped the air tight static container
			const  FReal MaxCoordinates = ContainerBoxHalfSize - ContainerWallThickness / 2 - SmallBoxHalfSize;
			EXPECT_LT(FMath::Abs(Dynamic->GetX()[axis]), MaxCoordinates + LargeErrorMargin);
		}
	}
}

