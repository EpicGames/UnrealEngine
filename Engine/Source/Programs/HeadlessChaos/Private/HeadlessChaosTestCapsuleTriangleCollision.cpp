// Copyright Epic Games, Inc. All Rights Reserved.
#include "HeadlessChaos.h"
#include "Chaos/Capsule.h"
#include "Chaos/Collision/CapsuleTriangleContactPoint.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Triangle.h"

namespace ChaosTest
{
	using namespace Chaos;

	bool ManifoldContainsPoint(const FContactPointManifold& ContactPoints, const int32 BodyIndex, const FVec3& Point, const FReal Tolerance)
	{
		for (FContactPoint ContactPoint : ContactPoints)
		{
			FVec3 Delta = ContactPoint.ShapeContactPoints[BodyIndex] - Point;
			if (Delta.Size() <= Tolerance)
			{
				return true;
			}
		}
		return false;
	}

	bool IsPointOnCapsuleSurface(const FImplicitCapsule3& Capsule, const FVec3& Point, const FReal Tolerance)
	{
		const FReal SegmentT = Utilities::ClosestTimeOnLineSegment(Point, FVec3(Capsule.GetX1f()), FVec3(Capsule.GetX2f()));
		const FVec3 SegmentPoint = FMath::Lerp(Capsule.GetX1f(), Capsule.GetX2f(), SegmentT);
		const FVec3 Delta = Point - SegmentPoint;
		const FReal Distance = Delta.Size();
		return FMath::IsNearlyEqual(Distance, Capsule.GetRadiusf(), Tolerance);
	}

	// Double face contact: both capsule points within the edge planes.
	// Capsule Radius 20cm, at 10cm above triangle.
	// Should have 2 contacts points, one at each segement end point.
	//
	//  ________
	//  \       |
	//   \ +==+ |
	//    \     |
	//     \    |
	//      \   |
	//       \  |
	//        \ |
	//         \|
	//
	GTEST_TEST(CapsuleTriangleTests, TestFace)
	{
		const FReal Tolerance = UE_KINDA_SMALL_NUMBER;
		const FReal CullDistance = FReal(3);

		FTriangle Triangle(FVec3(0,0,0), FVec3(100, -100, 0), FVec3(100, 0, 0));
		FImplicitCapsule3 Capsule(FVec3(30, -10, 10), FVec3(90, -10, 10), FReal(20));

		FContactPointManifold ContactPoints;
		ConstructCapsuleTriangleOneShotManifold2(Capsule, Triangle, CullDistance, ContactPoints);

		EXPECT_EQ(ContactPoints.Num(), 2);
		if (ContactPoints.Num() == 2)
		{
			EXPECT_TRUE(ManifoldContainsPoint(ContactPoints, 0, Capsule.GetX1f() - Capsule.GetRadiusf() * Triangle.GetNormal(), Tolerance));
			EXPECT_TRUE(ManifoldContainsPoint(ContactPoints, 0, Capsule.GetX2f() - Capsule.GetRadiusf() * Triangle.GetNormal(), Tolerance));
		}
	}

	// Same as TestFace except capsule is just below the triangle.
	// Should not get any contacts.
	//
	GTEST_TEST(CapsuleTriangleTests, TestFaceBelow)
	{
		const FReal Tolerance = UE_KINDA_SMALL_NUMBER;
		const FReal CullDistance = FReal(3);

		FTriangle Triangle(FVec3(0, 0, 0), FVec3(100, -100, 0), FVec3(100, 0, 0));
		FImplicitCapsule3 Capsule(FVec3(30, -10, -1), FVec3(90, -10, -1), FReal(20));

		FContactPointManifold ContactPoints;
		ConstructCapsuleTriangleOneShotManifold2(Capsule, Triangle, CullDistance, ContactPoints);

		EXPECT_EQ(ContactPoints.Num(), 0);
	}

	// Same as TestFace except capsule is exactly in the plane of the triangle
	// Should be same result as TestFace
	//
	GTEST_TEST(CapsuleTriangleTests, TestFaceAligned)
	{
		const FReal Tolerance = UE_KINDA_SMALL_NUMBER;
		const FReal CullDistance = FReal(3);

		FTriangle Triangle(FVec3(0, 0, 0), FVec3(100, -100, 0), FVec3(100, 0, 0));
		FImplicitCapsule3 Capsule(FVec3(30, -10, 0), FVec3(90, -10, 0), FReal(20));

		FContactPointManifold ContactPoints;
		ConstructCapsuleTriangleOneShotManifold2(Capsule, Triangle, CullDistance, ContactPoints);

		EXPECT_EQ(ContactPoints.Num(), 2);
		if (ContactPoints.Num() == 2)
		{
			EXPECT_TRUE(ManifoldContainsPoint(ContactPoints, 0, Capsule.GetX1f() - Capsule.GetRadiusf() * Triangle.GetNormal(), Tolerance));
			EXPECT_TRUE(ManifoldContainsPoint(ContactPoints, 0, Capsule.GetX2f() - Capsule.GetRadiusf() * Triangle.GetNormal(), Tolerance));
		}
	}

	// Face + Edge contact:
	// Capsule Radius 20cm, at 10cm above triangle.
	// Should have 2 contacts points
	//
	//  ________
	//  \       |
	//   \   +==+==
	//    \     |
	//     \    |
	//      \   |
	//       \  |
	//        \ |
	//         \|
	//
	GTEST_TEST(CapsuleTriangleTests, TestFaceEdge)
	{
		const FReal Tolerance = UE_KINDA_SMALL_NUMBER;
		const FReal CullDistance = FReal(3);

		FTriangle Triangle(FVec3(0, 0, 0), FVec3(100, -100, 0), FVec3(100, 0, 0));
		FImplicitCapsule3 Capsule(FVec3(50, -10, 10), FVec3(120, -10, 10), FReal(20));

		FContactPointManifold ContactPoints;
		ConstructCapsuleTriangleOneShotManifold2(Capsule, Triangle, CullDistance, ContactPoints);

		EXPECT_EQ(ContactPoints.Num(), 2);
		if (ContactPoints.Num() == 2)
		{
			EXPECT_TRUE(ManifoldContainsPoint(ContactPoints, 0, Capsule.GetX1f() - Capsule.GetRadiusf() * Triangle.GetNormal(), Tolerance));
			EXPECT_TRUE(ManifoldContainsPoint(ContactPoints, 0, FVec3(100, -10, 10) - Capsule.GetRadiusf() * Triangle.GetNormal(), Tolerance));
		}
	}

	// Face + Edge contact with capsule tilted by 10deg wrt triangle surface.
	// Capsule Radius 20cm, at 10cm above triangle.
	// Should have 2 face contacts points (10deg is below threshold)
	//
	//  ________
	//  \       |
	//   \ (+)==+==)
	//    \     |
	//     \    |
	//      \   |
	//       \  |
	//        \ |
	//         \|
	//
	GTEST_TEST(CapsuleTriangleTests, TestFaceEdgeAngleInThreshold)
	{
		const FReal PositionTolerance = UE_KINDA_SMALL_NUMBER;
		const FReal NormalTolerance = UE_KINDA_SMALL_NUMBER;
		const FReal CullDistance = FReal(3);
		const FReal Sin10 = FMath::Sin(FMath::DegreesToRadians(10));
		const FReal Cos10 = FMath::Cos(FMath::DegreesToRadians(10));

		const FTriangle Triangle(FVec3(0, 0, 0), FVec3(100, -100, 0), FVec3(100, 0, 0));
		const FImplicitCapsule3 Capsule(FVec3(50, -10, 10 + 50 * Sin10), FVec3(150, -10, 10 - 50 * Sin10), FReal(20));

		FContactPointManifold ContactPoints;
		ConstructCapsuleTriangleOneShotManifold2(Capsule, Triangle, CullDistance, ContactPoints);

		EXPECT_EQ(ContactPoints.Num(), 2);
		if (ContactPoints.Num() == 2)
		{
			// We should have a contact on the cylinder below the first segment
			// And the other will be with the edge. 
			// @todo(chaos): check this
			
			// The normal should be vertical because we are within the face angle threshold
			EXPECT_NEAR(ContactPoints[1].ShapeContactNormal.Z, FReal(1), NormalTolerance);

			// Make sure both points are on the capsule surface
			EXPECT_TRUE(IsPointOnCapsuleSurface(Capsule, ContactPoints[0].ShapeContactPoints[0], PositionTolerance));
			EXPECT_TRUE(IsPointOnCapsuleSurface(Capsule, ContactPoints[1].ShapeContactPoints[0], PositionTolerance));
		}
	}

	// Capsule at a 45 degree angle to the triangle, and the capsule axis passes right through an edge.
	// We should get a single contact pointing out of the triangle at 45 degrees.
	// 
	//     \ \
	//      \ \
	//  -----\+\
	//        \ \
	//
	GTEST_TEST(CapsuleTriangleTests, TestEdgeZeroSeparation)
	{
		const FReal PositionTolerance = UE_KINDA_SMALL_NUMBER;
		const FReal NormalTolerance = UE_KINDA_SMALL_NUMBER;
		const FReal CullDistance = FReal(3);

		FTriangle Triangle(FVec3(0, 0, 0), FVec3(100, -100, 0), FVec3(100, 0, 0));
		FImplicitCapsule3 Capsule(FVec3(50, -10, 50), FVec3(150, -10, -50), FReal(10));

		FContactPointManifold ContactPoints;
		ConstructCapsuleTriangleOneShotManifold2(Capsule, Triangle, CullDistance, ContactPoints);

		EXPECT_EQ(ContactPoints.Num(), 1);
		if (ContactPoints.Num() == 1)
		{
			EXPECT_NEAR(ContactPoints[0].Phi, -Capsule.GetRadiusf(), PositionTolerance);
			EXPECT_NEAR(FVec3::DotProduct(ContactPoints[0].ShapeContactNormal, FVec3(0,0,1)), FMath::Cos(FMath::DegreesToRadians(45)), NormalTolerance);
		}
	}

	// Capsule at a 45 degree angle to the triangle, and the capsule axis passes right through an edge.
	// This time we have an end cap within the edge planes, so we should get a face contact at that location.
	// The edge contact will be ignored because it would generate an inward-facing normal, and we are
	// above the face angle threshold.
	// 
	//         / /
	//        / /
	//  ---+-/-/
	//      / /
	//     / /
	//
	GTEST_TEST(CapsuleTriangleTests, TestEdgeZeroSeparation2)
	{
		const FReal PositionTolerance = UE_KINDA_SMALL_NUMBER;
		const FReal NormalTolerance = UE_KINDA_SMALL_NUMBER;
		const FReal CullDistance = FReal(3);

		FTriangle Triangle(FVec3(0, 0, 0), FVec3(100, -100, 0), FVec3(100, 0, 0));
		FImplicitCapsule3 Capsule(FVec3(50, -10, -50), FVec3(150, -10, 50), FReal(10));

		FContactPointManifold ContactPoints;
		ConstructCapsuleTriangleOneShotManifold2(Capsule, Triangle, CullDistance, ContactPoints);

		EXPECT_EQ(ContactPoints.Num(), 1);
		if (ContactPoints.Num() == 1)
		{
			EXPECT_NEAR(ContactPoints[0].ShapeContactPoints[0].X, Capsule.GetX1f().X, PositionTolerance);
			EXPECT_NEAR(ContactPoints[0].ShapeContactPoints[0].Y, Capsule.GetX1f().Y, PositionTolerance);
			EXPECT_NEAR(ContactPoints[0].ShapeContactPoints[0].Z, Capsule.GetX1f().Z - Capsule.GetRadiusf(), PositionTolerance);

			EXPECT_NEAR(ContactPoints[0].ShapeContactNormal.Z, FReal(1), NormalTolerance);
		}
	}
}
