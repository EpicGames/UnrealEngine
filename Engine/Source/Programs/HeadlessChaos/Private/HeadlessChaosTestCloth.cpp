// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestCloth.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDEvolution.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/PBDAxialSpringConstraints.h"
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/Utilities.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "Modules/ModuleManager.h"

namespace ChaosTest {

	using namespace Chaos;
	DEFINE_LOG_CATEGORY_STATIC(LogChaosTestCloth, Verbose, All);

	TUniquePtr<Softs::FPBDEvolution> InitPBDEvolution(
		const int32 NumIterations=1,
		const Softs::FSolverReal CollisionThickness=KINDA_SMALL_NUMBER,
		const Softs::FSolverReal SelfCollisionThickness=KINDA_SMALL_NUMBER,
		const Softs::FSolverReal Friction=0.0,
		const Softs::FSolverReal Damping=0.04)
	{
		Chaos::Softs::FSolverParticles Particles;
		Chaos::Softs::FSolverCollisionParticles RigidParticles;
		TUniquePtr<Softs::FPBDEvolution> Evolution(
			new Softs::FPBDEvolution(
				MoveTemp(Particles),
				MoveTemp(RigidParticles),
				{},
				NumIterations,
				CollisionThickness,
				SelfCollisionThickness,
				Friction,
				Damping));
		return Evolution;
	}

	template <class TEvolutionPtr>
	void InitSingleParticle(
		TEvolutionPtr& Evolution,
		const Softs::FSolverVec3& Position = Softs::FSolverVec3(0),
		const Softs::FSolverVec3& Velocity = Softs::FSolverVec3(0),
		const Softs::FSolverReal Mass = 1.0)
	{
		auto& Particles = Evolution->Particles();
		const uint32 Idx = Particles.Size();
		Particles.AddParticles(1);
		Particles.SetX(Idx, Position);
		Particles.V(Idx) = Velocity;
		Particles.M(Idx) = Mass;
		Particles.InvM(Idx) = 1.0 / Mass;
	}

	template <class TEvolutionPtr>
	void InitTriMesh_EquilateralTri(
		FTriangleMesh& TriMesh,
		TEvolutionPtr& Evolution, 
		const Softs::FSolverVec3& XOffset=Softs::FSolverVec3(0))
	{
		auto& Particles = Evolution->Particles();
		const uint32 InitialNumParticles = Particles.Size();

		FTriangleMesh::InitEquilateralTriangleYZ(TriMesh, Particles);

		// Initialize particles.  Use 1/3 area of connected triangles for particle mass.
		for (uint32 i = InitialNumParticles; i < Particles.Size(); i++)
		{
			Particles.SetX(i, Particles.GetX(i) + XOffset);
			Particles.V(i) = Chaos::Softs::FSolverVec3(0);
			Particles.M(i) = 0;
		}
		for (const Chaos::TVec3<int32>& Tri : TriMesh.GetElements())
		{
			const Softs::FSolverReal TriArea = 0.5 * Chaos::Softs::FSolverVec3::CrossProduct(
				Particles.GetX(Tri[1]) - Particles.GetX(Tri[0]),
				Particles.GetX(Tri[2]) - Particles.GetX(Tri[0])).Size();
			for (int32 i = 0; i < 3; i++)
			{
				Particles.M(Tri[i]) += TriArea / 3;
			}
		}
		for (uint32 i = InitialNumParticles; i < Particles.Size(); i++)
		{
			check(Particles.M(i) > SMALL_NUMBER);
			Particles.InvM(i) = 1.0 / Particles.M(i);
		}
	}

	template <class TEvolutionPtr>
	void AddEdgeLengthConstraint(
		TEvolutionPtr& Evolution,
		const TArray<Chaos::TVec3<int32>>& Topology,
		const Softs::FSolverReal Stiffness)
	{
		check(Stiffness >= 0. && Stiffness <= 1.);
		// TODO: Use Add AddConstraintRuleRange
		//Evolution->AddPBDConstraintFunction(
		//	[SpringConstraints = Chaos::FPBDSpringConstraints(
		//		Evolution->Particles(), Topology, Stiffness)](
		//			TPBDParticles<float, 3>& InParticles, const float Dt) 
		//	{
		//		SpringConstraints.Apply(InParticles, Dt);
		//	});
	}
	
	template <class TEvolutionPtr>
	void AddAxialConstraint(
		TEvolutionPtr& Evolution,
		TArray<Chaos::TVec3<int32>>&& Topology,
		const Softs::FSolverReal Stiffness)
	{
		check(Stiffness >= 0. && Stiffness <= 1.);
		// TODO: Use Add AddConstraintRuleRange
		//Evolution->AddPBDConstraintFunction(
		//	[SpringConstraints = Chaos::FPBDAxialSpringConstraints(
		//		Evolution->Particles(), MoveTemp(Topology), Stiffness)](
		//			TPBDParticles<float, 3>& InParticles, const float Dt) 
		//	{
		//		SpringConstraints.Apply(InParticles, Dt);
		//	});
	}

	template <class TEvolutionPtr>
	void AdvanceTime(
		TEvolutionPtr& Evolution,
		const uint32 NumFrames=1,
		const uint32 NumTimeStepsPerFrame=1,
		const uint32 FPS=24)
	{
		check(NumTimeStepsPerFrame > 0);
		Evolution->SetIterations(NumTimeStepsPerFrame);

		check(FPS > 0);
		const Softs::FSolverReal Dt = 1.0 / FPS;
		for (uint32 i = 0; i < NumFrames; i++)
		{
			Evolution->AdvanceOneTimeStep(Dt);
		}
	}

	template <class TParticleContainer>
	TArray<Chaos::Softs::FSolverVec3> CopyPoints(const TParticleContainer& Particles)
	{
		TArray<Chaos::Softs::FSolverVec3> Points;
		Points.SetNum(Particles.Size());

		for (uint32 i = 0; i < Particles.Size(); i++)
			Points[i] = Particles.GetX(i);

		return Points;
	}

	template <class TParticleContainer>
	void Reset(TParticleContainer& Particles, const TArray<Chaos::Softs::FSolverVec3>& Points)
	{
		for (uint32 i = 0; i < Particles.Size(); i++)
		{
			Particles.SetX(i, Points[i]);
			Particles.V(i) = Chaos::Softs::FSolverVec3(0);
		}
	}

	TArray<Softs::FSolverVec3> GetDifference(const TArray<Softs::FSolverVec3>& A, const TArray<Softs::FSolverVec3>& B)
	{
		TArray<Softs::FSolverVec3> C;
		C.SetNum(A.Num());
		for (int32 i = 0; i < A.Num(); i++)
			C[i] = A[i] - B[i];
		return C;
	}

	TArray<Softs::FSolverReal> GetMagnitude(const TArray<Chaos::Softs::FSolverVec3>& V)
	{
		TArray<Softs::FSolverReal> M;
		M.SetNum(V.Num());
		for (int32 i = 0; i < V.Num(); i++)
			M[i] = V[i].Size();
		return M;
	}

	template <class T>
	bool AllSame(const TArray<T>& V, int32& Idx, const T Tolerance=KINDA_SMALL_NUMBER)
	{
		if (!V.Num())
			return true;
		const T& V0 = V[0];
		for (int32 i = 1; i < V.Num(); i++)
		{
			if (!FMath::IsNearlyEqual(V0, V[i], Tolerance))
			{
				Idx = i;
				return false;
			}
		}
		return true;
	}

	template <class TV>
	bool RunDropTest(
		TUniquePtr<Softs::FPBDEvolution>& Evolution,
		const Softs::FSolverReal GravMag,
		const TV& GravDir,
		const TArray<TV>& InitialPoints,
		const int32 SubFrameSteps,
		const Softs::FSolverReal DistTolerance,
		const char* TestID)
	{
		const Softs::FSolverReal PreTime = Evolution->GetTime();
		AdvanceTime(Evolution, 24, SubFrameSteps); // 1 second
		const Softs::FSolverReal PostTime = Evolution->GetTime();
		EXPECT_NEAR(PostTime-PreTime, 1.0, KINDA_SMALL_NUMBER)
			<< TestID
			<< "Evolution advanced time by " << (PostTime - PreTime)
			<< " seconds, expected 1.0 seconds.";

		const TArray<Softs::FSolverVec3> PostPoints = CopyPoints(Evolution->Particles());
		const TArray<Softs::FSolverVec3> Diff = GetDifference(PostPoints, InitialPoints);
		const TArray<Softs::FSolverReal> ScalarDiff = GetMagnitude(Diff);

		// All points did the same thing
		int32 Idx = 0;
		EXPECT_TRUE(AllSame(ScalarDiff, Idx, (Softs::FSolverReal)0.1))
			<< TestID
			<< "Points fell different distances - Index 0: "
			<< ScalarDiff[0] << " != Index " 
			<< Idx << ": " 
			<< ScalarDiff[Idx] << " +/- 0.1.";

		// Fell the right amount
		EXPECT_NEAR(ScalarDiff[0], (Softs::FSolverReal)0.5 * GravMag, DistTolerance)
			<< TestID
			<< "Points fell by " << ScalarDiff[0] 
			<< ", expected " << ((Softs::FSolverReal)0.5 * GravMag)
			<< " +/- " << DistTolerance << "."; 

		// Fell the right direction
		const Softs::FSolverReal DirDot = Chaos::Softs::FSolverVec3::DotProduct(GravDir, Diff[0].GetSafeNormal());
		EXPECT_NEAR(DirDot, (Softs::FSolverReal)1.0, (Softs::FSolverReal)KINDA_SMALL_NUMBER)
			<< TestID
			<< "Points fell in different directions.";

		return true;
	}

	void DeformableGravity()
	{
		const Softs::FSolverReal DistTol = 0.0002f;

		//
		// Initialize solver and gravity
		//

		TUniquePtr<Softs::FPBDEvolution> Evolution = InitPBDEvolution();

		const Softs::FSolverVec3 GravDir(0, 0, -1);
		const Softs::FSolverReal GravMag = 980.665f;

		//
		// Drop a single particle
		//

		InitSingleParticle(Evolution);
		TArray<Softs::FSolverVec3> InitialPoints = CopyPoints(Evolution->Particles());

		RunDropTest(Evolution, GravMag, GravDir, InitialPoints, 1, DistTol, "Single point falling under gravity, iters: 1 - ");
		Reset(Evolution->Particles(), InitialPoints);
		RunDropTest(Evolution, GravMag, GravDir, InitialPoints, 100, DistTol, "Single point falling under gravity, iters: 100 - ");
		Reset(Evolution->Particles(), InitialPoints);

		//
		// Add a triangle mesh
		//

		Chaos::FTriangleMesh TriMesh;
		InitTriMesh_EquilateralTri(TriMesh, Evolution);
		InitialPoints = CopyPoints(Evolution->Particles());

		// 
		// Points falling under gravity
		//

		RunDropTest(Evolution, GravMag, GravDir, InitialPoints, 1, DistTol, "Points falling under gravity, iters: 1 - ");
		Reset(Evolution->Particles(), InitialPoints);
		RunDropTest(Evolution, GravMag, GravDir, InitialPoints, 100, DistTol, "Points falling under gravity, iters: 100 - ");
		Reset(Evolution->Particles(), InitialPoints);

		// 
		// Points falling under gravity with edge length constraint
		//

		AddEdgeLengthConstraint(Evolution, TriMesh.GetSurfaceElements(), 1.0);

		RunDropTest(Evolution, GravMag, GravDir, InitialPoints, 1, DistTol, "Points falling under gravity & edge cnstr, iters: 1 - ");
		Reset(Evolution->Particles(), InitialPoints);
		RunDropTest(Evolution, GravMag, GravDir, InitialPoints, 100, DistTol, "Points falling under gravity & edge cnstr, iters: 100 - ");
		Reset(Evolution->Particles(), InitialPoints);
	}

	void EdgeConstraints()
	{
		TUniquePtr<Softs::FPBDEvolution> Evolution = InitPBDEvolution();
		FTriangleMesh TriMesh;
		auto& Particles = Evolution->Particles();
		Particles.AddParticles(2145);
		TArray<TVec3<int32>> Triangles;
		Triangles.SetNum(2048);
		// 32 n, 32 m
		// 6 + 4*(n-1) + (m - 1)(3 + 2*(n-1)) = 2*n*m
		for (int32 TriIndex = 0; TriIndex < 2048; TriIndex++)
		{
			int32 i = ((float)rand()) / ((float)RAND_MAX) * 2144;
			int32 j = ((float)rand()) / ((float)RAND_MAX) * 2144;
			int32 k = ((float)rand()) / ((float)RAND_MAX) * 2144;
			Triangles[TriIndex] = TVec3<int32>(i, j, k);
		}
		AddEdgeLengthConstraint(Evolution, Triangles, 1.0);
		AddAxialConstraint(Evolution, MoveTemp(Triangles), 1.0);
	}

	void ClothCollection()
	{
		FManagedArrayCollection ManagedArrayCollection;

		const FName DataGroup("Data");
		TManagedArray<int32> Value;
		ManagedArrayCollection.AddExternalAttribute<int32>("Value", DataGroup, Value);

		const FName ViewGroup("View");
		FManagedArrayCollection::FConstructionParameters DataDependency(DataGroup);
		TManagedArray<int32> ValueStart;
		TManagedArray<int32> ValueEnd;
		TManagedArray<int32> Id;
		ManagedArrayCollection.AddExternalAttribute<int32>("ValueStart", ViewGroup, ValueStart, DataDependency);
		ManagedArrayCollection.AddExternalAttribute<int32>("ValueEnd", ViewGroup, ValueEnd, DataDependency);
		ManagedArrayCollection.AddExternalAttribute<int32>("Id", ViewGroup, Id);

		auto InsertView = [&ManagedArrayCollection, &Value, &ValueStart, &ValueEnd, &Id, &DataGroup, &ViewGroup](int32 ViewPosition, int32 NumValues)
			{
				check(ViewPosition <= ManagedArrayCollection.NumElements(ViewGroup));
				EXPECT_TRUE(ManagedArrayCollection.InsertElements(1, ViewPosition, ViewGroup) == ViewPosition);

				static int32 ViewId = 0;
				Id[ViewPosition] = ViewId++;

				if (NumValues)
				{
					int32 PrevValuePosition = INDEX_NONE;
					for (int32 Index = ViewPosition - 1; Index >= 0; --Index)
					{
						if (ValueEnd[Index] != INDEX_NONE)
						{
							PrevValuePosition = ValueEnd[Index];
							break;
						}
					}
					const int32 ValuePosition = PrevValuePosition + 1;

					EXPECT_TRUE(ManagedArrayCollection.InsertElements(NumValues, ValuePosition, DataGroup) == ValuePosition);
			
					ValueStart[ViewPosition] = ValuePosition;
					ValueEnd[ViewPosition] = ValuePosition + NumValues - 1;

					for (int32 ValueIndex = ValueStart[ViewPosition]; ValueIndex <= ValueEnd[ViewPosition]; ++ValueIndex)
					{
						Value[ValueIndex] = Id[ViewPosition];
					}
				}
				else
				{
					ValueStart[ViewPosition] = INDEX_NONE;
					ValueEnd[ViewPosition] = INDEX_NONE;
				}
			};

		auto RemoveViews = [&ManagedArrayCollection, &ValueStart, &ValueEnd, &DataGroup, &ViewGroup](int32 ViewPosition, int32 NumViews)
			{
				const int32 ViewStart = ViewPosition;
				const int32 ViewEnd = ViewPosition + NumViews - 1;

				for (int32 ViewIndex = ViewStart; ViewIndex <= ViewEnd; ++ViewIndex)
				{
					// Remove data
					const int32 Position = ValueStart[ViewIndex];
					if (Position != INDEX_NONE)
					{
						const int32 NumValues = ValueEnd[ViewIndex] - ValueStart[ViewIndex] + 1;
						ManagedArrayCollection.RemoveElements(DataGroup, NumValues, Position);
					}
				}
				// Remove views
				ManagedArrayCollection.RemoveElements(ViewGroup, NumViews, ViewPosition);
			};

		auto SetViewSize = [&ManagedArrayCollection, &Value, &ValueStart, &ValueEnd, &Id, &DataGroup, &ViewGroup](int32 ViewIndex, int32 InNumValues)
			{
				check(InNumValues >= 0);

				int32& Start = ValueStart[ViewIndex];
				int32& End = ValueEnd[ViewIndex];
				check(Start != INDEX_NONE || End == INDEX_NONE);

				const int32 NumValues = (Start == INDEX_NONE) ? 0 : End - Start + 1;

				if (const int32 Delta = InNumValues - NumValues)
				{
					if (Delta > 0)
					{
						// Find a previous valid index range to insert after when the range is empty
						auto ComputeEnd = [&ValueEnd](int32 ViewIndex)->int32
						{
							for (int32 Index = ViewIndex; Index >= 0; --Index)
							{
								if (ValueEnd[Index] != INDEX_NONE)
								{
									return ValueEnd[Index];
								}
							}
							return INDEX_NONE;
						};

						// Grow the array
						const int32 Position = ComputeEnd(ViewIndex) + 1;
						ManagedArrayCollection.InsertElements(Delta, Position, DataGroup);

						// Update Start/End
						if (!NumValues)
						{
							Start = Position;
						}
						End = Start + InNumValues - 1;

						// Fill the test values with the id check
						for (int32 ValueIndex = Start; ValueIndex <= End; ++ValueIndex)
						{
							Value[ValueIndex] = Id[ViewIndex];
						}

					}
					else
					{
						// Shrink the array
						const int32 Position = Start + InNumValues;
						ManagedArrayCollection.RemoveElements(DataGroup, -Delta, Position);

						// Update Start/End
						if (InNumValues)
						{
							End = Position - 1;
						}
						else
						{
							End = Start = INDEX_NONE;
						}
					}
				}
				const int32 Size = ValueEnd[ViewIndex] - ValueStart[ViewIndex] + 1;
				check(
					(InNumValues == 0 && ValueStart[ViewIndex] == INDEX_NONE && ValueEnd[ViewIndex] == INDEX_NONE) ||
					(InNumValues == Size && ValueStart[ViewIndex] != INDEX_NONE && ValueEnd[ViewIndex] != INDEX_NONE));
		};

		auto HasKeptIntegrity = [&Value , &ValueStart, &ValueEnd, &Id]()->bool
			{
				EXPECT_TRUE(ValueEnd.Num() == ValueStart.Num() && Id.Num() == ValueStart.Num());
				for (int32 ViewIndex = 0; ViewIndex < ValueStart.Num(); ++ViewIndex)
				{
					if (ValueStart[ViewIndex] == INDEX_NONE || ValueEnd[ViewIndex] == INDEX_NONE)
					{
						if (ValueStart[ViewIndex] != ValueEnd[ViewIndex])
						{
							return false;
						}
						continue;
					}
					for (int32 ValueIndex = ValueStart[ViewIndex]; ValueIndex <= ValueEnd[ViewIndex]; ++ValueIndex)
					{
						if (Value[ValueIndex] != Id[ViewIndex])
						{
							return false;
						}
					}
				}
				return true;
			};

		auto HasKeptOrder = [&Value, &ValueStart, &ValueEnd, &Id]()->bool
		{
			int32 PrevValueEnd = -1;
			for (int32 ViewIndex = 0; ViewIndex < ValueStart.Num(); ++ViewIndex)
			{
				if (ValueStart[ViewIndex] == INDEX_NONE)
				{
					check(ValueEnd[ViewIndex] == INDEX_NONE);
					continue;
				}
				if (ValueStart[ViewIndex] != PrevValueEnd + 1)
				{
					return false;
				}
				PrevValueEnd = ValueEnd[ViewIndex];
			}
			return PrevValueEnd == Value.Num() - 1;
		};

		// Insert 5 views from the start of the array
		for (int32 Index = 0; Index < 5; ++Index)
		{
			InsertView(0, (Index + 1) * 100);
		}
		// Insert 2 more views for testing consecutive empty views
		for (int32 Index = 0; Index < 2; ++Index)
		{
			InsertView(0, 0);
		}
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Add 1 view in the middle of the array
		InsertView(ManagedArrayCollection.NumElements(ViewGroup) / 2, 500);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Add 1 view at the end of the array
		InsertView(ManagedArrayCollection.NumElements(ViewGroup), 1000);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Remove 1 view from the start of the array
		RemoveViews(0, 1);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Remove 1 view from the end of the array
		RemoveViews(ManagedArrayCollection.NumElements(ViewGroup) - 1, 1);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Remove 2 views from the middle
		RemoveViews(ManagedArrayCollection.NumElements(ViewGroup) / 2, 2);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Remove all remaining 5 views
		RemoveViews(0, 5);
		EXPECT_TRUE(ManagedArrayCollection.NumElements(ViewGroup) == 0);
		EXPECT_TRUE(ManagedArrayCollection.NumElements(DataGroup) == 0);

		// Insert 5 empty views
		for (int32 Index = 0; Index < 5; ++Index)
		{
			InsertView(0, 0);
		}
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Resize view 1
		SetViewSize(1, 11);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Resize view 2
		SetViewSize(2, 22);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Resize view 3
		SetViewSize(3, 33);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Remove view 2
		RemoveViews(2, 1);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());
		
		// Resize view 1 to 0
		SetViewSize(1, 0);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Resize view 3 to 33
		SetViewSize(3, 33);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Resize view 3 to 0
		SetViewSize(3, 0);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());

		// Resize view 0
		SetViewSize(0, 1);
		EXPECT_TRUE(HasKeptIntegrity());
		EXPECT_TRUE(HasKeptOrder());
	}
} // namespace ChaosTest