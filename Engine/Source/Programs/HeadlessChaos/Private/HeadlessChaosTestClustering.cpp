// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestClustering.h"

#include "HeadlessChaos.h"
#include "Modules/ModuleManager.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/Box.h"
#include "Chaos/Plane.h"
#include "Chaos/Sphere.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Levelset.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Utilities.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"

namespace ChaosTest {

	using namespace Chaos;

	void ImplicitCluster()
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolution Evolution(Particles, PhysicalMaterials);
		FPBDRigidClusteredParticles& ClusteredParticles = Particles.GetClusteredParticles();
		
		uint32 FirstId = ClusteredParticles.Size();
		FPBDRigidParticleHandle* Box1 = AppendClusteredParticleBox(Particles, FVec3((FReal)100, (FReal)100, (FReal)100));
		uint32 BoxId = FirstId++;
		FPBDRigidParticleHandle* Box2 = AppendClusteredParticleBox(Particles, FVec3((FReal)100, (FReal)100, (FReal)100));
		uint32 Box2Id = FirstId++;

		Box2->SetX(FVec3((FReal)100, (FReal)0, (FReal)0));
		Box2->SetP(Box2->GetX());

		Evolution.EnableParticle(Box1);
		Evolution.EnableParticle(Box2);

		Evolution.AdvanceOneTimeStep(0);	//hack to initialize islands
		FClusterCreationParameters ClusterParams;
		
		TArray<Chaos::FPBDRigidParticleHandle*> ClusterChildren;
		ClusterChildren.Add(Box1);
		ClusterChildren.Add(Box2);

		Evolution.GetRigidClustering().CreateClusterParticle(0, MoveTemp(ClusterChildren), ClusterParams, FImplicitObjectPtr(nullptr));
		EXPECT_EQ(ClusteredParticles.Size(), 3);

		FVec3 ClusterX = ClusteredParticles.GetX(2);
		FRotation3 ClusterRot = ClusteredParticles.GetR(2);

		EXPECT_TRUE(ClusterX.Equals(FVec3 {(FReal)50, 0, 0}));
		EXPECT_TRUE(ClusterRot.Equals(FRotation3::Identity));
		EXPECT_TRUE(ClusterX.Equals(ClusteredParticles.GetP(2)));
		EXPECT_TRUE(ClusterRot.Equals(ClusteredParticles.GetQ(2)));

		FRigidTransform3 ClusterTM(ClusterX, ClusterRot);
		FVec3 LocalPos = ClusterTM.InverseTransformPositionNoScale(FVec3 {(FReal)200, (FReal)0, (FReal)0});
		FVec3 Normal;
		FReal Phi = ClusteredParticles.GetGeometry(2)->PhiWithNormal(LocalPos, Normal);
		EXPECT_TRUE(FMath::IsNearlyEqual(Phi, (FReal)50));
		EXPECT_TRUE(Normal.Equals(FVec3{(FReal)1, (FReal)0, (FReal)0}));

		//EXPECT_TRUE(Evolution.GetParticles().Geometry(2)->IsConvex());  we don't actually guarantee this
	}

	void FractureCluster()
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolution Evolution(Particles, PhysicalMaterials);
		auto& ClusteredParticles = Particles.GetClusteredParticles();

		//create a long row of boxes - the depth 0 cluster is the entire row, the depth 1 clusters 4 boxes each, the depth 2 clusters are 1 box each

		constexpr int32 NumBoxes = 32;
		TArray<FPBDRigidParticleHandle*> Boxes;
		TArray<uint32> BoxIDs;
		for (int i = 0; i < NumBoxes; ++i)
		{
			BoxIDs.Add(ClusteredParticles.Size());
			FPBDRigidParticleHandle* Box = AppendClusteredParticleBox(Particles, FVec3((FReal)100, (FReal)100, (FReal)100));
			Box->SetX(FVec3((FReal)i * (FReal)100, (FReal)0, (FReal)0));
			Box->SetP(Box->GetX());
			Boxes.Add(Box);

			Evolution.EnableParticle(Box);
		}

		Evolution.AdvanceOneTimeStep(0);	//hack to generate islands

		TArray<Chaos::FPBDRigidParticleHandle* > ClusterHandles;

		for (int i = 0; i < NumBoxes / 4; ++i)
		{
			FClusterCreationParameters ClusterParams;

			TArray<Chaos::FPBDRigidParticleHandle*> ClusterChildren;
			ClusterChildren.Add(Boxes[i * 4]);
			ClusterChildren.Add(Boxes[i * 4+1]);
			ClusterChildren.Add(Boxes[i * 4+2]);
			ClusterChildren.Add(Boxes[i * 4+3]);
			ClusterHandles.Add(Evolution.GetRigidClustering().CreateClusterParticle(0, MoveTemp(ClusterChildren), ClusterParams, FImplicitObjectPtr(nullptr)));
		}

		FClusterCreationParameters ClusterParams;
		TArray<Chaos::FPBDRigidParticleHandle* > ClusterHandlesCopy = ClusterHandles;
		Chaos::FPBDRigidParticleHandle* RootClusterHandle = Evolution.GetRigidClustering().CreateClusterParticle(0, MoveTemp(ClusterHandlesCopy), ClusterParams, FImplicitObjectPtr(nullptr));
		FVec3 InitialVelocity((FReal)50, (FReal)20, (FReal)100);

		RootClusterHandle->SetV(InitialVelocity);
		
		constexpr int NumParticles = NumBoxes + NumBoxes / 4 + 1;
		EXPECT_EQ(ClusteredParticles.Size(), NumParticles);

		for (int i = 0; i < NumParticles-1; ++i)
		{
			EXPECT_TRUE(ClusteredParticles.Disabled(i));
		}

		EXPECT_TRUE(RootClusterHandle->Disabled() == false);
		EXPECT_EQ(Particles.GetNonDisabledView().Num(), 1);
		EXPECT_EQ(Particles.GetNonDisabledView().Begin()->Handle(), RootClusterHandle);

		const FReal Dt = 0;	//don't want to integrate gravity, just want to test fracture
		Evolution.AdvanceOneTimeStep(Dt);
		EXPECT_TRUE(RootClusterHandle->Disabled());	//not a cluster anymore, so disabled
		for (auto& Particle : Particles.GetNonDisabledView())
		{
			EXPECT_NE(Particle.Handle(), RootClusterHandle);	//view no longer contains root
		}

		EXPECT_EQ(Particles.GetNonDisabledView().Num(), NumBoxes / 4);

		//children are still in a cluster, so disabled
		for (uint32 BoxID : BoxIDs)
		{
			EXPECT_TRUE(ClusteredParticles.Disabled(BoxID));
			for (auto& Particle : Particles.GetNonDisabledView())
			{
				EXPECT_NE(Particle.Handle(), ClusteredParticles.Handle(BoxID));	//make sure boxes are not in non disabled array
			}

			EXPECT_FALSE(ClusteredParticles.Handle(BoxID)->IsInConstraintGraph());
		}

		for (Chaos::FPBDRigidParticleHandle* ClusterHandle : ClusterHandles)
		{
			EXPECT_TRUE(ClusterHandle->Disabled() == false);	//not a cluster anymore, so disabled
			bool bFoundInNonDisabled = false;
			for (auto& Particle : Particles.GetNonDisabledView())
			{
				bFoundInNonDisabled |= Particle.Handle() == ClusterHandle;
			}

			EXPECT_TRUE(bFoundInNonDisabled);	//clusters are enabled and in non disabled array
			EXPECT_TRUE(ClusterHandle->GetV().Equals(InitialVelocity));
		}

		Evolution.AdvanceOneTimeStep(Dt);
		//second fracture, all clusters are now disabled
		for (Chaos::FPBDRigidParticleHandle* ClusterHandle : ClusterHandles)
		{
			EXPECT_TRUE(ClusterHandle->Disabled() == true);	//not a cluster anymore, so disabled
			for (auto& Particle : Particles.GetNonDisabledView())
			{
				EXPECT_NE(Particle.Handle(), ClusterHandle);	//make sure boxes are not in non disabled array
			}

			EXPECT_FALSE(ClusterHandle->IsInConstraintGraph());
		}

		EXPECT_EQ(Particles.GetNonDisabledView().Num(), NumBoxes);
		
		for (FPBDRigidParticleHandle* BoxHandle : Boxes)
		{
			EXPECT_TRUE(BoxHandle->Disabled() == false);
			bool bFoundInNonDisabled = false;
			for (auto& Particle : Particles.GetNonDisabledView())
			{
				bFoundInNonDisabled |= Particle.Handle() == BoxHandle;
			}
			EXPECT_TRUE(bFoundInNonDisabled);
			EXPECT_TRUE(BoxHandle->GetV().Equals(InitialVelocity));
		}
	}

	void PartialFractureCluster()
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolution Evolution(Particles, PhysicalMaterials);
		auto& ClusteredParticles = Particles.GetClusteredParticles();

		//create a long row of boxes - the depth 0 cluster is the entire row, the depth 1 clusters 4 boxes each, the depth 2 clusters are 1 box each

		constexpr int32 NumBoxes = 32;
		TArray<FPBDRigidParticleHandle*> Boxes;
		TArray<uint32> BoxIDs;
		for (int i = 0; i < NumBoxes; ++i)
		{
			BoxIDs.Add(ClusteredParticles.Size());
			FPBDRigidParticleHandle* Box = AppendClusteredParticleBox(Particles, FVec3((FReal)100, (FReal)100, (FReal)100));
			Box->SetX(FVec3((FReal)i * (FReal)100, (FReal)0, (FReal)0));
			Box->SetP(Box->GetX());
			Boxes.Add(Box);

			Evolution.EnableParticle(Box);
		}

		Evolution.AdvanceOneTimeStep(0);	//hack to generate islands

		TArray<Chaos::FPBDRigidParticleHandle* > ClusterHandles;

		for (int i = 0; i < NumBoxes / 4; ++i)
		{
			FClusterCreationParameters ClusterParams;

			TArray<Chaos::FPBDRigidParticleHandle*> ClusterChildren;
			ClusterChildren.Add(Boxes[i * 4]);
			ClusterChildren.Add(Boxes[i * 4 + 1]);
			ClusterChildren.Add(Boxes[i * 4 + 2]);
			ClusterChildren.Add(Boxes[i * 4 + 3]);
			ClusterHandles.Add(Evolution.GetRigidClustering().CreateClusterParticle(0, MoveTemp(ClusterChildren), ClusterParams, FImplicitObjectPtr(nullptr)));
		}

		TArray<Chaos::FPBDRigidParticleHandle* > ClusterHandlesDup = ClusterHandles;

		FClusterCreationParameters ClusterParams;
		Chaos::FPBDRigidParticleHandle* RootClusterHandle = Evolution.GetRigidClustering().CreateClusterParticle(0, MoveTemp(ClusterHandles), ClusterParams, FImplicitObjectPtr(nullptr));
		FVec3 InitialVelocity((FReal)50, (FReal)20, (FReal)100);

		RootClusterHandle->SetV(InitialVelocity);

		TUniquePtr<FChaosPhysicsMaterial> PhysicalMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicalMaterial->Friction = 0;
		PhysicalMaterial->Restitution = 0;
		PhysicalMaterial->SleepingLinearThreshold = 0;
		PhysicalMaterial->SleepingAngularThreshold = 0;
		PhysicalMaterial->DisabledLinearThreshold = 0;
		PhysicalMaterial->DisabledAngularThreshold = 0;

		Chaos::TArrayCollectionArray<FRealSingle>& SolverStrainArray = Evolution.GetRigidClustering().GetStrainArray();

		for (int i = 0; i < NumBoxes + NumBoxes / 4 + 1; ++i)
		{
			SolverStrainArray[i] = (FReal)1;
			Evolution.SetPhysicsMaterial(ClusteredParticles.Handle(i), MakeSerializable(PhysicalMaterial));
		}

		Evolution.AdvanceOneTimeStep((FReal)1 / (FReal)60);
		EXPECT_TRUE(RootClusterHandle->Disabled() == false);	//strain > 0 so no fracture yet

		// todo: is this the correct replacement for strain?
		//fracture the third cluster, this should leave us with three pieces (0, 1), (2), (3,4,5,6,7)
		Evolution.GetRigidClustering().SetInternalStrain(static_cast<Chaos::FPBDRigidClusteredParticleHandle*>(ClusterHandlesDup[2]), 0.0);

		Evolution.AdvanceOneTimeStep((FReal)1 / (FReal)60);
		//EXPECT_TRUE(Evolution.GetParticles().Disabled(RootClusterHandle) == false);	//one of the connected pieces should re-use this
		EXPECT_TRUE(ClusterHandlesDup[2]->Disabled() == false);	//this cluster is on its own and should be enabled 
		
		
		EXPECT_EQ(Evolution.GetRigidClustering().GetTopLevelClusterParents().Num(), 3);	//there should only be 3 pieces
		for (uint32 BoxID : BoxIDs)
		{
			EXPECT_TRUE(ClusteredParticles.Disabled(BoxID));	//no boxes should be active yet
			EXPECT_TRUE(Evolution.GetRigidClustering().GetTopLevelClusterParents().Contains(ClusteredParticles.Handle(BoxID)) == false);
			EXPECT_FALSE(ClusteredParticles.Handle(BoxID)->IsInConstraintGraph());
		}

		SolverStrainArray[NumBoxes + NumBoxes / 4 + 1] = (FReal)1;
		Evolution.SetPhysicsMaterial(ClusteredParticles.Handle(NumBoxes + NumBoxes / 4 + 1), MakeSerializable(PhysicalMaterial));
		SolverStrainArray[NumBoxes + NumBoxes / 4 + 2] = (FReal)1;
		Evolution.SetPhysicsMaterial(ClusteredParticles.Handle(NumBoxes + NumBoxes / 4 + 2), MakeSerializable(PhysicalMaterial));

		Evolution.AdvanceOneTimeStep((FReal)1 / (FReal)60);	//next frame nothing should fracture
		//EXPECT_TRUE(Evolution.GetParticles().Disabled(RootClusterHandle) == false);	//one of the connected pieces should re-use this
		EXPECT_TRUE(ClusterHandlesDup[2]->Disabled() == false);	//this cluster is on its own and should be enabled 

		EXPECT_EQ(Evolution.GetRigidClustering().GetTopLevelClusterParents().Num(), 3);	//there should only be 3 pieces
		for (uint32 BoxID : BoxIDs)
		{
			EXPECT_TRUE(ClusteredParticles.Disabled(BoxID));	//no boxes should be active yet
			EXPECT_TRUE(Evolution.GetRigidClustering().GetTopLevelClusterParents().Contains(ClusteredParticles.Handle(BoxID)) == false);
			EXPECT_FALSE(ClusteredParticles.Handle(BoxID)->IsInConstraintGraph());
		}
		
	}
}

