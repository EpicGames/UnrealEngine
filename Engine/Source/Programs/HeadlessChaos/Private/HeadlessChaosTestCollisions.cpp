// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestCollisions.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosCollisionConstraints.h"
#include "Chaos/GJK.h"
#include "Chaos/Pair.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/CollisionOneShotManifoldsMiscShapes.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"
#include "Modules/ModuleManager.h"

#define SMALL_THRESHOLD 1e-4

#define RESET_PQ(Particle) Particle->SetP(Particle->GetX()); Particle->SetQ(Particle->GetR());
#define INVARIANT_XR_START(Particle) FVec3 InvariantPreX_##Particle = Particle->GetX(); FRotation3 InvariantPreR_##Particle = Particle->GetR()
#define INVARIANT_XR_END(Particle) EXPECT_TRUE(InvariantPreX_##Particle.Equals(Particle->GetX())); EXPECT_TRUE(InvariantPreR_##Particle.Equals(Particle->GetR()))
#define INVARIANT_VW_START(Particle) FVec3 InvariantPreV_##Particle = Particle->GetV(); FVec3 InvariantPreW_##Particle = Particle->GetW()
#define INVARIANT_VW_END(Particle) EXPECT_TRUE(InvariantPreV_##Particle.Equals(Particle->GetV())); EXPECT_TRUE(InvariantPreW_##Particle.Equals(Particle->GetW()))

namespace ChaosTest {

	using namespace Chaos;

	DEFINE_LOG_CATEGORY_STATIC(LogHChaosCollisions, Verbose, All);

	void LevelsetConstraint()
	{
		TArrayCollectionArray<bool> Collided;
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->Friction = (FReal)0;
		PhysicsMaterial->Restitution = (FReal)0;
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;
		
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		Particles.GetParticleHandles().AddArray(&Collided);
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);

		auto Box1 = AppendDynamicParticleBox(Particles);
		Box1->SetX(FVec3(1.f));
		Box1->SetR(FRotation3(FQuat::Identity));
		Box1->SetP(Box1->GetX());
		Box1->SetQ(Box1->GetR());
		Box1->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);
		Box1->UpdateWorldSpaceState(FRigidTransform3(Box1->GetX(), Box1->GetR()), FVec3(0));

		auto Box2 = AppendDynamicParticleBox(Particles);
		Box2->SetX(FVec3(0.5f, 0.5f, 1.9f));
		Box2->SetR(FRotation3(FQuat::Identity));
		Box2->SetP(Box2->GetX());
		Box2->SetQ(Box2->GetR());
		Box2->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);
		Box2->UpdateWorldSpaceState(FRigidTransform3(Box2->GetX(), Box2->GetR()), FVec3(0));

		FPBDCollisionConstraintAccessor Collisions(Particles, Collided, PhysicsMaterials, PerParticlePhysicsMaterials, 1, 1);
		Collisions.ComputeConstraints(0.f);
		EXPECT_EQ(Collisions.NumConstraints(), 1);

		FPBDCollisionConstraint& Constraint = Collisions.GetConstraint(0);
		if (auto PBDRigid = Constraint.GetParticle0()->CastToRigidParticle())
		{
			//Question: non dynamics don't have collision particles, seems wrong if the levelset is dynamic and the static is something like a box
			PBDRigid->CollisionParticles()->UpdateAccelerationStructures();
		}
		Collisions.UpdateLevelsetConstraint(Constraint);

		EXPECT_EQ(Constraint.GetParticle0(), Box2);
		EXPECT_EQ(Constraint.GetParticle1(), Box1);
		EXPECT_TRUE(Constraint.CalculateWorldContactNormal().operator==(FVec3(0, 0, 1)));

		// The contact point is the average of the surface contact points, so it should be half of Phi inside particle0
		EXPECT_NEAR(Constraint.GetPhi(), -0.1f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Constraint.CalculateWorldContactLocation().Z, 1.45f, KINDA_SMALL_NUMBER);
	}

	void LevelsetConstraintGJK()
	{
		TArrayCollectionArray<bool> Collided;
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->Friction = (FReal)0;
		PhysicsMaterial->Restitution = (FReal)0;
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;

		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		Particles.GetParticleHandles().AddArray(&Collided);
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);

		auto Box1 = AppendDynamicParticleConvexBox(Particles, FVec3(1.f) );
		Box1->SetX(FVec3(0.f));
		Box1->SetR(FRotation3(FQuat::Identity));
		Box1->SetP(Box1->GetX());
		Box1->SetQ(Box1->GetR());
		Box1->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);
		Box1->UpdateWorldSpaceState(FRigidTransform3(Box1->GetX(), Box1->GetR()), FVec3(0));

		auto Box2 = AppendDynamicParticleBox(Particles, FVec3(1.f) );
		Box2->SetX(FVec3(1.25f, 0.f, 0.f));
		Box2->SetR(FRotation3(FQuat::Identity));
		Box2->SetP(Box2->GetX());
		Box2->SetQ(Box2->GetR());
		Box2->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);
		Box2->UpdateWorldSpaceState(FRigidTransform3(Box2->GetX(), Box2->GetR()), FVec3(0));

		FPBDCollisionConstraintAccessor Collisions(Particles, Collided, PhysicsMaterials, PerParticlePhysicsMaterials, 1, 1);
		Collisions.ComputeConstraints(0.f);
		EXPECT_EQ(Collisions.NumConstraints(), 1);

		FPBDCollisionConstraint& Constraint = Collisions.GetConstraint(0);
		Collisions.UpdateLevelsetConstraint(Constraint);
		
		EXPECT_EQ(Constraint.GetParticle0(), Box2);
		EXPECT_EQ(Constraint.GetParticle1(), Box1);
		EXPECT_TRUE(Constraint.CalculateWorldContactNormal().operator==(FVec3(0, 0, 1)));

		// The contact point is the average of the surface contact points, so it should be half of Phi inside particle0
		Chaos::FReal Distance = ChaosTest::SignedDistance(*Constraint.GetParticle0(), Constraint.CalculateWorldContactLocation());
		EXPECT_NEAR(Distance, 0.5f * Constraint.GetPhi(), KINDA_SMALL_NUMBER);
	}
	
	void CollisionBoxPlane()
	{
		// test a box and plane in a colliding state
		TArrayCollectionArray<bool> Collided;
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->Friction = (FReal)0;
		PhysicsMaterial->Restitution = (FReal)1;
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;

		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		Particles.GetParticleHandles().AddArray(&Collided);
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);

		const FReal Dt = FReal(1) / FReal(24.);

		auto Floor = AppendStaticAnalyticFloor(Particles);
		auto Box = AppendDynamicParticleBox(Particles);
		Box->SetP(FVec3(0, 1, 0));
		Box->SetQ(FRotation3(FQuat::Identity));
		Box->SetV(FVec3(0, 0, -1));
		Box->SetPreV(Box->GetV());
		Box->SetX(Box->GetP() - Box->GetV() * Dt);
		Box->SetR(Box->GetQ());
		Box->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);
		Box->UpdateWorldSpaceState(FRigidTransform3(Box->GetP(), Box->GetQ()), FVec3(0));

		const FReal InitialBoxZ = Box->GetX().Z;

		FPBDCollisionConstraintAccessor Collisions(Particles, Collided, PhysicsMaterials, PerParticlePhysicsMaterials, 2, 5);

		Collisions.ComputeConstraints(Dt);
		EXPECT_EQ(Collisions.NumConstraints(), 1);

		FPBDCollisionConstraint& Constraint = Collisions.GetConstraint(0);
		EXPECT_TRUE(Constraint.GetParticle0() != nullptr);
		EXPECT_TRUE(Constraint.GetParticle1() != nullptr);

		if (auto PBDRigid = Constraint.GetParticle0()->CastToRigidParticle())
		{
			PBDRigid->CollisionParticles()->UpdateAccelerationStructures();
		}

		Collisions.GatherInput(Dt);

		Collisions.Update(Constraint);

		EXPECT_EQ(Constraint.GetParticle0(), Box);
		EXPECT_EQ(Constraint.GetParticle1(), Floor);
		EXPECT_TRUE(Constraint.CalculateWorldContactNormal().operator==(FVec3(0, 0, 1)));
		EXPECT_TRUE(FMath::Abs(Constraint.GetPhi() - FReal(-0.5) ) < SMALL_THRESHOLD );

		{
			INVARIANT_XR_START(Box);
			const int32 NumIts = 8;
			Collisions.Apply(Dt, NumIts);
			INVARIANT_XR_END(Box);
		}

		{
			RESET_PQ(Box);
			{
				INVARIANT_XR_START(Box);
				INVARIANT_VW_START(Box);
				const int32 NumIts = 1;
				Collisions.ApplyPushOut(Dt, NumIts);
				INVARIANT_XR_END(Box);
				INVARIANT_VW_END(Box);
			}
		}

		Collisions.ScatterOutput(Dt);

		// Box will not move because the default depentration velocity is zero
		const FReal ExpectedBoxZ = InitialBoxZ + Collisions.CollisionConstraints.GetSolverSettings().DepenetrationVelocity * Dt;
		EXPECT_NEAR(Box->GetP().Z, ExpectedBoxZ, 1.e-2f);

		// Velocity is below the restitution threshold, so expecting 0 velocity despite the fact that restitution is 1
		EXPECT_TRUE(Box->GetV().Equals(FVec3(0)));
		EXPECT_TRUE(Box->GetW().Equals(FVec3(0)));
	}

	void CollisionConvexConvex()
	{

		// test a box and plane in a colliding state
		TArrayCollectionArray<bool> Collided;
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->Friction = (FReal)0;
		PhysicsMaterial->Restitution = (FReal)0;
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;

		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		Particles.GetParticleHandles().AddArray(&Collided);
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);

		auto Floor = AppendStaticConvexFloor(Particles);
		auto Box = AppendDynamicParticleConvexBox( Particles, FVec3(50) );
		Box->SetX(FVec3(0, 0, 49));
		Box->SetR(FRotation3(FQuat::Identity));
		Box->SetV(FVec3(0, 0, -1));
		Box->SetPreV(Box->GetV());
		Box->SetP(Box->GetX());
		Box->SetQ(Box->GetR());
		Box->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);
		Box->UpdateWorldSpaceState(FRigidTransform3(Box->GetP(), Box->GetQ()), FVec3(0));

		const FReal Dt = FReal(1) / FReal(24.);

		FPBDCollisionConstraintAccessor Collisions(Particles, Collided, PhysicsMaterials, PerParticlePhysicsMaterials, 2, 5);

		Collisions.ComputeConstraints(Dt);
		EXPECT_EQ(Collisions.NumConstraints(), 1);

		FPBDCollisionConstraint* Constraint = &Collisions.GetConstraint(0);
		EXPECT_TRUE(Constraint != nullptr);

		Collisions.GatherInput(Dt);

		Collisions.Update(*Constraint);

		EXPECT_EQ(Constraint->GetParticle0(), Box);
		EXPECT_EQ(Constraint->GetParticle1(), Floor);
		EXPECT_TRUE(Constraint->CalculateWorldContactNormal().operator==(FVec3(0, 0, 1)));
		EXPECT_TRUE(FMath::Abs(Constraint->GetPhi() - FReal(-1.0)) < SMALL_THRESHOLD);

		{
			INVARIANT_XR_START(Box);
			Collisions.Apply(Dt, 1);
			INVARIANT_XR_END(Box);
		}

		{
			RESET_PQ(Box);
			{
				INVARIANT_XR_START(Box);
				INVARIANT_VW_START(Box);
				const int32 NumIts = 10;
				Collisions.ApplyPushOut(Dt, NumIts);
				INVARIANT_XR_END(Box);
				INVARIANT_VW_END(Box);
			}
		}

		Collisions.ScatterOutput(Dt);

		//EXPECT_TRUE(Box->P().Equals(FVector(0.f, 0.f, 50.f)));
		//EXPECT_TRUE(Box->Q().Equals(FQuat::Identity));

	}

	void CollisionBoxPlaneZeroResitution()
	{
		// test a box and plane in a colliding state
		TArrayCollectionArray<bool> Collided;
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->Friction = (FReal)0;
		PhysicsMaterial->Restitution = (FReal)0;
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;
		
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		Particles.GetParticleHandles().AddArray(&Collided);
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);

		const FReal Dt = FReal(1) / FReal(24.);

		auto Floor = AppendStaticAnalyticFloor(Particles);
		auto Box = AppendDynamicParticleBox(Particles);
		Box->SetP(FVec3(0, 1, 0));
		Box->SetQ(FRotation3(FQuat::Identity));
		Box->SetV(FVec3(0, 0, -1));
		Box->SetPreV(Box->GetV());
		Box->SetX(Box->GetP() - Box->GetV() * Dt);
		Box->SetR(Box->GetR());
		Box->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);
		Box->UpdateWorldSpaceState(FRigidTransform3(Box->GetP(), Box->GetQ()), FVec3(0));

		FReal InitialBoxZ = Box->GetX().Z;

		FPBDCollisionConstraintAccessor Collisions(Particles, Collided, PhysicsMaterials, PerParticlePhysicsMaterials, 2, 5);

		Collisions.ComputeConstraints(Dt);
		EXPECT_EQ(Collisions.NumConstraints(), 1);

		Collisions.GatherInput(Dt);

		FPBDCollisionConstraint& Constraint = Collisions.GetConstraint(0);
		EXPECT_EQ(Constraint.GetParticle0(), Box);
		EXPECT_EQ(Constraint.GetParticle1(), Floor);
		EXPECT_TRUE(Constraint.CalculateWorldContactNormal().operator==(FVec3(0, 0, 1)));
		EXPECT_TRUE(FMath::Abs(Constraint.GetPhi() - FReal(-0.5)) < SMALL_THRESHOLD);

		{
			INVARIANT_XR_START(Box);
			Collisions.Apply(Dt, 1);
			INVARIANT_XR_END(Box);
		}

		{
			RESET_PQ(Box);
			{
				INVARIANT_XR_START(Box);
				INVARIANT_VW_START(Box);
				const int32 NumIts = 10;
				Collisions.ApplyPushOut(Dt, NumIts);
				INVARIANT_XR_END(Box);
				INVARIANT_VW_END(Box);
			}
		}

		Collisions.ScatterOutput(Dt);

		// 0 restitution so expecting 0 velocity
		EXPECT_TRUE(Box->GetV().Equals(FVec3(0)));
		EXPECT_TRUE(Box->GetW().Equals(FVec3(0)));

		// Box will not move because the default depentration velocity is zero
		const FReal ExpectedBoxZ = InitialBoxZ + Collisions.CollisionConstraints.GetSolverSettings().DepenetrationVelocity * Dt;
		EXPECT_TRUE(FMath::IsNearlyEqual(Box->GetP().Z, ExpectedBoxZ, 1.e-2));
	}

	void CollisionBoxPlaneRestitution()
	{
		TArrayCollectionArray<bool> Collided;
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->Friction = (FReal)0;
		PhysicsMaterial->Restitution = (FReal)1;
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		Particles.GetParticleHandles().AddArray(&Collided);
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);

		const FReal Dt = FReal(1) / FReal(24.);

		auto Floor = AppendStaticAnalyticFloor(Particles);
		auto Box = AppendDynamicParticleBox(Particles);
		Box->SetP(FVec3(0, 0, 0));
		Box->SetQ(FRotation3(FQuat::Identity));
		Box->SetV(FVec3(0, 0, -100));
		Box->SetPreV(Box->GetV());
		Box->SetX(Box->GetP() - Box->GetV() * Dt);
		Box->SetR(Box->GetQ());
		Box->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);
		Box->UpdateWorldSpaceState(FRigidTransform3(Box->GetP(), Box->GetQ()), FVec3(0));

		FPBDCollisionConstraintAccessor Collisions(Particles, Collided, PhysicsMaterials, PerParticlePhysicsMaterials, 2, 5);

		Collisions.ComputeConstraints(Dt);
		EXPECT_EQ(Collisions.NumConstraints(), 1);

		FPBDCollisionConstraint& Constraint = Collisions.GetConstraint(0);
		if (auto PBDRigid = Constraint.GetParticle0()->CastToRigidParticle())
		{
			PBDRigid->CollisionParticles()->UpdateAccelerationStructures();
		}

		Collisions.GatherInput(Dt);

		Collisions.UpdateLevelsetConstraint(Constraint);
		EXPECT_EQ(Constraint.GetParticle0(), Box);
		EXPECT_EQ(Constraint.GetParticle1(), Floor);
		EXPECT_TRUE(Constraint.CalculateWorldContactNormal().operator==(FVec3(0, 0, 1)));
		EXPECT_TRUE(FMath::Abs(Constraint.GetPhi() - FReal(-0.5)) < SMALL_THRESHOLD);

		{
			INVARIANT_XR_START(Box);
			Collisions.Apply(Dt, 1);
			INVARIANT_XR_END(Box);
		}

		{
			RESET_PQ(Box);
			{
				INVARIANT_XR_START(Box);
				const int32 NumIts = 10;
				Collisions.ApplyPushOut(Dt, NumIts);
				INVARIANT_XR_END(Box);
			}
		}

		Collisions.ScatterOutput(Dt);

		// full restitution, so expecting negative velocity
		EXPECT_TRUE(Box->GetV().Equals(FVec3(0.f, 0.f, 100.f)));
		EXPECT_TRUE(Box->GetW().Equals(FVec3(0)));

		// should end up outside the plane
		EXPECT_GE(Box->GetP().Z, -Box->GetGeometry()->BoundingBox().Min().Z);
		EXPECT_TRUE(Box->GetQ().Equals(FQuat::Identity));
	}

	// This test will make sure that a dynamic cube colliding with a static floor will have the correct bounce velocity
	// for a restitution of 0.5
	void CollisionCubeCubeRestitution()
	{
		TArrayCollectionArray<bool> Collided;
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->Friction = (FReal)0;
		PhysicsMaterial->Restitution = (FReal)0.5;
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		Particles.GetParticleHandles().AddArray(&Collided);
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);

		const FReal Dt = FReal(1) / FReal(24.);

		FGeometryParticleHandle* StaticCube = AppendStaticParticleBox(Particles, FVec3(100.0f));
		StaticCube->SetX(FVec3(0, 0, -50.0f));
		StaticCube->UpdateWorldSpaceState(FRigidTransform3(StaticCube->GetX(), StaticCube->GetR()), FVec3(0));

		FPBDRigidParticleHandle* DynamicCube = AppendDynamicParticleBox(Particles, FVec3(100.0f));
		DynamicCube->SetX(FVec3(0, 0, 50));
		DynamicCube->SetR(FRotation3::FromIdentity());
		DynamicCube->SetV(FVec3(0, 0, -100));
		DynamicCube->SetPreV(DynamicCube->GetV());
		DynamicCube->SetP(DynamicCube->GetX() + DynamicCube->GetV() * Dt);
		DynamicCube->SetQ(DynamicCube->GetR());
		DynamicCube->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);
		DynamicCube->UpdateWorldSpaceState(FRigidTransform3(DynamicCube->GetP(), DynamicCube->GetQ()), FVec3(0));

		FPBDCollisionConstraintAccessor Collisions(Particles, Collided, PhysicsMaterials, PerParticlePhysicsMaterials, 2, 5);

		Collisions.ComputeConstraints(Dt);
		EXPECT_EQ(Collisions.NumConstraints(), 1);

		if (Collisions.NumConstraints() <= 0)
		{
			return;
		}

		FPBDCollisionConstraint& Constraint = Collisions.GetConstraint(0);
		if (FPBDRigidParticleHandle* PBDRigid = Constraint.GetParticle0()->CastToRigidParticle())
		{
			PBDRigid->CollisionParticles()->UpdateAccelerationStructures();
		}

		Collisions.GatherInput(Dt);

		EXPECT_EQ(Constraint.GetParticle0(), DynamicCube);
		EXPECT_EQ(Constraint.GetParticle1(), StaticCube);
		EXPECT_NEAR(Constraint.CalculateWorldContactNormal().Z, FReal(1), KINDA_SMALL_NUMBER);
		{
			INVARIANT_XR_START(DynamicCube);
			Collisions.Apply(Dt, 1);
			INVARIANT_XR_END(DynamicCube);
		}

		// If we need accurate restitution we need more velocity iterations
		Collisions.ApplyPushOut(Dt, 4);

		Collisions.ScatterOutput(Dt);

		// This test's tolerances are set to be very crude as to not be over sensitive (for now)
		EXPECT_NEAR(DynamicCube->GetV().Z, 50.0f, 5.0f);  // restitution not too low
		EXPECT_TRUE(FMath::Abs(DynamicCube->GetV().X) < 1.0f);
		EXPECT_TRUE(FMath::Abs(DynamicCube->GetV().Y) < 1.0f);
	}

	void CollisionBoxToStaticBox()
	{
		TArrayCollectionArray<bool> Collided;
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->Friction = (FReal)0;
		PhysicsMaterial->Restitution = (FReal)0;
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		Particles.GetParticleHandles().AddArray(&Collided);
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);

		auto StaticBox = AppendStaticParticleBox(Particles);
		StaticBox->SetX(FVec3(-0.05f, -0.05f, -0.1f));
		StaticBox->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);
		StaticBox->UpdateWorldSpaceState(FRigidTransform3(StaticBox->GetX(), StaticBox->GetR()), FVec3(0));

		FReal Dt = FReal(1) / FReal(24.);

		auto Box2 = AppendDynamicParticleBox(Particles);
		FVec3 StartingPoint(0.5f);
		Box2->SetP(StartingPoint);
		Box2->SetQ(Box2->GetR());
		Box2->SetV(FVec3(0, 0, -1));
		Box2->SetPreV(Box2->GetV());
		Box2->SetX(Box2->GetP() - Box2->GetV() * Dt);
		Box2->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);
		Box2->UpdateWorldSpaceState(FRigidTransform3(Box2->GetP(), Box2->GetQ()), FVec3(0));

		FBox Region(FVector(FReal(.2)), FVector(FReal(.5)));

		FPBDCollisionConstraintAccessor Collisions(Particles, Collided, PhysicsMaterials, PerParticlePhysicsMaterials, 1, 1);
		Collisions.ComputeConstraints(Dt);
		EXPECT_EQ(Collisions.NumConstraints(), 1);

		Collisions.GatherInput(Dt);

		FPBDCollisionConstraint& Constraint = Collisions.GetConstraint(0);

		if (auto PBDRigid = Constraint.GetParticle0()->CastToRigidParticle())
		{
			PBDRigid->CollisionParticles()->UpdateAccelerationStructures();
		}

		EXPECT_EQ(Constraint.GetParticle0(), Box2);
		EXPECT_EQ(Constraint.GetParticle1(), StaticBox);
		EXPECT_TRUE(Constraint.CalculateWorldContactNormal().Equals(FVector(0.0, 0.0, 1.0f)));
		EXPECT_TRUE(FMath::Abs(Constraint.GetPhi() - FReal(-0.4)) < SMALL_THRESHOLD);


		EXPECT_TRUE(FMath::Abs(Box2->GetV().Size() - 1.f)<SMALL_THRESHOLD ); // no velocity change yet  

		{
			INVARIANT_XR_START(Box2);
			INVARIANT_XR_START(StaticBox);
			Collisions.Apply(Dt, 1);
			INVARIANT_XR_END(Box2);
			INVARIANT_XR_END(StaticBox);
		}

		RESET_PQ(Box2);
		{
			//INVARIANT_XR_START(Box2);
			//INVARIANT_XR_START(StaticBox);
			//INVARIANT_VW_START(Box2);
			const int32 NumIts = 10;
			Collisions.ApplyPushOut(Dt, NumIts);
			//INVARIANT_XR_END(Box2);
			//INVARIANT_XR_END(StaticBox);
			//INVARIANT_VW_END(Box2);
		}

		Collisions.ScatterOutput(Dt);

		EXPECT_TRUE(Box2->GetV().Size() < FVector(0, -1, 0).Size()); // slowed down  
		EXPECT_TRUE(Box2->GetW().Size() > 0); // now has rotation 

		EXPECT_FALSE(Box2->GetP().Equals(StartingPoint)); // moved
		EXPECT_FALSE(Box2->GetQ().Equals(FQuat::Identity)); // and rotated
	}

	void CollisionCapsuleCapsule()
	{
		const FReal CullDistance = FReal(3);
		{
			FCapsule Capsule1(FVec3(-20, -10, 10), FVec3(20, -10, 10), FReal(20));
			FCapsule Capsule2(FVec3(-20, -10, 10), FVec3(20, -10, 10), FReal(20));

			FImplicitObjectPtr Implicit1 = MakeImplicitObjectPtr<FCapsule>(Capsule1);
			FImplicitObjectPtr Implicit2 = MakeImplicitObjectPtr<FCapsule>(Capsule2);

			FRigidTransform3 Transform1(FRigidTransform3::Identity);
			FRigidTransform3 Transform2(FVec3(0.0, 0.0, 18), FRotation3(FQuat::Identity));
			FPBDCollisionConstraint Constraint;

			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs RigidsSOA(UniqueIndices);
			TArray<FPBDRigidParticleHandle*> Particles = RigidsSOA.CreateDynamicParticles(2);

			FGeometryParticleHandle* Particle1 = Particles[0];
			FGeometryParticleHandle* Particle2 = Particles[1];

			Particle1->SetGeometry(Implicit1);
			Particle2->SetGeometry(Implicit2);

			const FShapeInstancePtr& Shape1 = Particle1->ShapeInstances()[0];
			const FShapeInstancePtr& Shape2 = Particle2->ShapeInstances()[0];

			FPBDCollisionConstraint::Make(Particle1,
				Implicit1,
				Shape1.Get(),
				nullptr,
				Transform1,
				Particle1,
				Implicit2,
				Shape2.Get(),
				nullptr,
				Transform2,
				CullDistance,
				true,
				EContactShapesType::CapsuleCapsule,
				Constraint);

			Collisions::UpdateConstraint(Constraint, Transform1, Transform2, 0.1);

			const TArrayView<FManifoldPoint>& ManifoldPoints = Constraint.GetManifoldPoints();

			EXPECT_EQ(ManifoldPoints.Num(), 2);

			EXPECT_EQ(ManifoldPoints[0].ContactPoint.Phi, -22.0f); // Two radius 40 minus 18 of translation -> -22.0f
			EXPECT_EQ(ManifoldPoints[0].ContactPoint.ShapeContactPoints[0], FVec3f(20, -10, 30)); // z = 10 + 20 radius -> 30.0f
			EXPECT_EQ(ManifoldPoints[0].ContactPoint.ShapeContactPoints[1], FVec3f(20, -10, -10));

			EXPECT_EQ(ManifoldPoints[1].ContactPoint.Phi, -18.0f);
			EXPECT_EQ(ManifoldPoints[1].ContactPoint.ShapeContactPoints[0], FVec3f(-20, -10, 30));
			EXPECT_EQ(ManifoldPoints[1].ContactPoint.ShapeContactPoints[1], FVec3f(-20, -10, 30));

			{
				Transform1.SetRotation(FRotation3::FromAxisAngle(FVec3(1.0, 0.0, 0.0), 0.1));
				Collisions::UpdateConstraint(Constraint, Transform1, Transform2, 0.1);
				const TArrayView<FManifoldPoint>& ManifoldPoints2 = Constraint.GetManifoldPoints();
				EXPECT_EQ(ManifoldPoints2.Num(), 2);
				EXPECT_LT(ManifoldPoints2[0].ContactPoint.Phi, -20.0f);
				EXPECT_GT(ManifoldPoints2[0].ContactPoint.Phi, -21.0f);
				EXPECT_LT(ManifoldPoints2[1].ContactPoint.Phi, -19.0f);
				EXPECT_GT(ManifoldPoints2[1].ContactPoint.Phi, -20.0f);

			}
		}
		{
			FCapsule Capsule1(FVec3(-20, -10, 10), FVec3(20, -10, 10), FReal(18));
			FCapsule Capsule2(FVec3(-20, -10, 10), FVec3(20, -10, 10), FReal(20));

			FImplicitObjectPtr Implicit1 = MakeImplicitObjectPtr<FCapsule>(Capsule1);
			FImplicitObjectPtr Implicit2 = MakeImplicitObjectPtr<FCapsule>(Capsule2);

			FRigidTransform3 Transform1(FRigidTransform3::Identity);
			FRigidTransform3 Transform2(FVec3(0.0, 0.0, 16), FRotation3(FQuat::Identity));
			FPBDCollisionConstraint Constraint;

			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs RigidsSOA(UniqueIndices);
			TArray<FPBDRigidParticleHandle*> Particles = RigidsSOA.CreateDynamicParticles(2);

			FGeometryParticleHandle* Particle1 = Particles[0];
			FGeometryParticleHandle* Particle2 = Particles[1];

			Particle1->SetGeometry(Implicit1);
			Particle2->SetGeometry(Implicit2);

			const FShapeInstancePtr& Shape1 = Particle1->ShapeInstances()[0];
			const FShapeInstancePtr& Shape2 = Particle2->ShapeInstances()[0];

			FPBDCollisionConstraint::Make(Particle1,
				Implicit1,
				Shape1.Get(),
				nullptr,
				Transform1,
				Particle1,
				Implicit2,
				Shape2.Get(),
				nullptr,
				Transform2,
				CullDistance,
				true,
				EContactShapesType::CapsuleCapsule,
				Constraint);

			Collisions::UpdateConstraint(Constraint, Transform1, Transform2, 0.1);

			const TArrayView<FManifoldPoint>& ManifoldPoints = Constraint.GetManifoldPoints();

			EXPECT_EQ(ManifoldPoints.Num(), 2);

			EXPECT_EQ(ManifoldPoints[0].ContactPoint.Phi, -22.0f);
			EXPECT_EQ(ManifoldPoints[0].ContactPoint.ShapeContactPoints[0], FVec3f(20, -10, 28));
			EXPECT_EQ(ManifoldPoints[0].ContactPoint.ShapeContactPoints[1], FVec3f(20, -10, -10));

			EXPECT_EQ(ManifoldPoints[1].ContactPoint.Phi, -18.0f);
			EXPECT_EQ(ManifoldPoints[1].ContactPoint.ShapeContactPoints[0], FVec3f(-20, -10, 28));
			EXPECT_EQ(ManifoldPoints[1].ContactPoint.ShapeContactPoints[1], FVec3f(-20, -10, 30));
		}

		{
			FCapsule Capsule1(FVec3(-20, -10, 10), FVec3(20, -10, 10), FReal(20));
			FCapsule Capsule2(FVec3(-20, -10, 10), FVec3(20, -10, 10), FReal(18));

			FImplicitObjectPtr Implicit1 = MakeImplicitObjectPtr<FCapsule>(Capsule1);
			FImplicitObjectPtr Implicit2 = MakeImplicitObjectPtr<FCapsule>(Capsule2);

			FRigidTransform3 Transform1(FRigidTransform3::Identity);
			FRigidTransform3 Transform2(FVec3(0.0, 0.0, 16), FRotation3(FQuat::Identity));
			FPBDCollisionConstraint Constraint;

			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs RigidsSOA(UniqueIndices);
			TArray<FPBDRigidParticleHandle*> Particles = RigidsSOA.CreateDynamicParticles(2);

			FGeometryParticleHandle* Particle1 = Particles[0];
			FGeometryParticleHandle* Particle2 = Particles[1];

			Particle1->SetGeometry(Implicit1);
			Particle2->SetGeometry(Implicit2);

			const FShapeInstancePtr& Shape1 = Particle1->ShapeInstances()[0];
			const FShapeInstancePtr& Shape2 = Particle2->ShapeInstances()[0];

			FPBDCollisionConstraint::Make(Particle1,
				Implicit1,
				Shape1.Get(),
				nullptr,
				Transform1,
				Particle1,
				Implicit2,
				Shape2.Get(),
				nullptr,
				Transform2,
				CullDistance,
				true,
				EContactShapesType::CapsuleCapsule,
				Constraint);

			Collisions::UpdateConstraint(Constraint, Transform1, Transform2, 0.1);

			const TArrayView<FManifoldPoint>& ManifoldPoints = Constraint.GetManifoldPoints();

			EXPECT_EQ(ManifoldPoints.Num(), 2);

			EXPECT_EQ(ManifoldPoints[0].ContactPoint.Phi, -22.0f);
			EXPECT_EQ(ManifoldPoints[0].ContactPoint.ShapeContactPoints[0], FVec3f(20, -10, 30));
			EXPECT_EQ(ManifoldPoints[0].ContactPoint.ShapeContactPoints[1], FVec3f(20, -10, -8));

			EXPECT_EQ(ManifoldPoints[1].ContactPoint.Phi, -18.0f);
			EXPECT_EQ(ManifoldPoints[1].ContactPoint.ShapeContactPoints[0], FVec3f(-20, -10, -10));
			EXPECT_EQ(ManifoldPoints[1].ContactPoint.ShapeContactPoints[1], FVec3f(-20, -10, -8));
		}
		
	}

}

