// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/PBDPositionConstraints.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/PBDPositionConstraints.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/PBDRigidSpringConstraints.h"
#include "Chaos/PBDRigidDynamicSpringConstraints.h"

namespace ChaosTest {

	using namespace Chaos;

	/**
	 * Check that we can access and remove constraints using handles
	 */
	template<typename TConstraints>
	void CheckConstraintHandles(TConstraints& Constraints, TArray<FPBDRigidParticleHandle*> ParticleHandles, TArray<typename TConstraints::FConstraintContainerHandle*> ConstraintsHandles)
	{
		// Constraints are created in valid state
		EXPECT_EQ(Constraints.NumConstraints(), 4);
		EXPECT_TRUE(ConstraintsHandles[0]->IsValid());
		EXPECT_TRUE(ConstraintsHandles[1]->IsValid());
		EXPECT_TRUE(ConstraintsHandles[2]->IsValid());
		EXPECT_TRUE(ConstraintsHandles[3]->IsValid());

		// Can access constraints' particles
		EXPECT_EQ(ConstraintsHandles[0]->GetConstrainedParticles()[0], ParticleHandles[0]);
		EXPECT_EQ(ConstraintsHandles[1]->GetConstrainedParticles()[0], ParticleHandles[1]);
		EXPECT_EQ(ConstraintsHandles[2]->GetConstrainedParticles()[0], ParticleHandles[2]);
		EXPECT_EQ(ConstraintsHandles[3]->GetConstrainedParticles()[0], ParticleHandles[3]);

		// Some constraints are single-particle, so can't do this check for all TConstraint
		//EXPECT_EQ(ConstraintsHandles[0]->GetConstrainedParticles()[1], ParticleHandles[1]);
		//EXPECT_EQ(ConstraintsHandles[1]->GetConstrainedParticles()[1], ParticleHandles[2]);
		//EXPECT_EQ(ConstraintsHandles[2]->GetConstrainedParticles()[1], ParticleHandles[3]);
		//EXPECT_EQ(ConstraintsHandles[3]->GetConstrainedParticles()[1], ParticleHandles[4]);

		// Array is packed as expected
		EXPECT_EQ(ConstraintsHandles[0]->GetConstraintIndex(), 0);
		EXPECT_EQ(ConstraintsHandles[1]->GetConstraintIndex(), 1);
		EXPECT_EQ(ConstraintsHandles[2]->GetConstraintIndex(), 2);
		EXPECT_EQ(ConstraintsHandles[3]->GetConstraintIndex(), 3);

		// Can remove constraints (from the middle of the constraint array)
		ConstraintsHandles[1]->RemoveConstraint();
		ConstraintsHandles[1] = nullptr;
		EXPECT_EQ(Constraints.NumConstraints(), 3);

		// Array is still packed as expected
		EXPECT_EQ(ConstraintsHandles[0]->GetConstraintIndex(), 0);
		EXPECT_EQ(ConstraintsHandles[2]->GetConstraintIndex(), 2);
		EXPECT_EQ(ConstraintsHandles[3]->GetConstraintIndex(), 1);

		// Can remove constraints (from the end of the constraint array)
		ConstraintsHandles[3]->RemoveConstraint();
		ConstraintsHandles[3] = nullptr;
		EXPECT_EQ(Constraints.NumConstraints(), 2);

		// Array is still packed as expected
		EXPECT_EQ(ConstraintsHandles[0]->GetConstraintIndex(), 0);
		EXPECT_EQ(ConstraintsHandles[2]->GetConstraintIndex(), 1);

		// Can remove constraints (from beginning of the constraint array)
		ConstraintsHandles[0]->RemoveConstraint();
		ConstraintsHandles[0] = nullptr;
		EXPECT_EQ(Constraints.NumConstraints(), 1);
		EXPECT_EQ(ConstraintsHandles[2]->GetConstraintIndex(), 0);

		// Can remove last constraint
		ConstraintsHandles[2]->RemoveConstraint();
		ConstraintsHandles[2] = nullptr;
		EXPECT_EQ(Constraints.NumConstraints(), 0);
	}

	template<typename TEvolution>
	void CollisionConstraintHandles()
	{
#if CHAOS_CONSTRAINTHANDLE_TODO
		// @todo(ccaulfield): Collision Constraints Container can't be used without collision detection loop.
		FPBDRigidsSOAs Particles;
		TPBDCollisionConstraints<FReal, 3> Constraints;
		TEvolution Evolution(Particles);

		TArray<FPBDRigidParticleHandle*> ParticleHandles = Evolution.CreateDynamicParticles(5);

		TArray<TPBDCollisionConstraintHandle<FReal, 3>*> ConstraintsHandles =
		{
			Constraints.AddConstraint({ ParticleHandles[0], ParticleHandles[1] }, ...),
			Constraints.AddConstraint({ ParticleHandles[1], ParticleHandles[2] }, ...),
			Constraints.AddConstraint({ ParticleHandles[2], ParticleHandles[3] }, ...),
			Constraints.AddConstraint({ ParticleHandles[3], ParticleHandles[4] }, ...),
		};

		//CheckConstraintHandles(Constraints, ParticleHandles, ConstraintsHandles);
#endif
	}

	template<typename TEvolution>
	void JointConstraintHandles()
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		TEvolution Evolution(Particles, PhysicalMaterials);

		TArray<FPBDRigidParticleHandle*> ParticleHandles = Evolution.CreateDynamicParticles(5);
		for (FPBDRigidParticleHandle* ParticleHandle : ParticleHandles)
		{
			Evolution.EnableParticle(ParticleHandle);
		}

		FPBDJointConstraints Constraints;
		TArray<FPBDJointConstraintHandle*> ConstraintsHandles =
		{
			Constraints.AddConstraint({ ParticleHandles[0], ParticleHandles[1] }, { FRigidTransform3(), FRigidTransform3() }),
			Constraints.AddConstraint({ ParticleHandles[1], ParticleHandles[2] }, { FRigidTransform3(), FRigidTransform3() }),
			Constraints.AddConstraint({ ParticleHandles[2], ParticleHandles[3] }, { FRigidTransform3(), FRigidTransform3() }),
			Constraints.AddConstraint({ ParticleHandles[3], ParticleHandles[4] }, { FRigidTransform3(), FRigidTransform3() }),
		};

		CheckConstraintHandles(Constraints, ParticleHandles, ConstraintsHandles);
	}

	template<typename TEvolution>
	void PositionConstraintHandles()
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		TEvolution Evolution(Particles, PhysicalMaterials);
		TArray<FPBDRigidParticleHandle*> ParticleHandles = Evolution.CreateDynamicParticles(5);
		for (FPBDRigidParticleHandle* ParticleHandle : ParticleHandles)
		{
			Evolution.EnableParticle(ParticleHandle);
		}

		FPBDPositionConstraints Constraints;
		TArray<FPBDPositionConstraintHandle*> ConstraintsHandles =
		{
			Constraints.AddConstraint(ParticleHandles[0], { 0, 0, 0 }),
			Constraints.AddConstraint(ParticleHandles[1], { 0, 0, 0 }),
			Constraints.AddConstraint(ParticleHandles[2], { 0, 0, 0 }),
			Constraints.AddConstraint(ParticleHandles[3], { 0, 0, 0 }),
		};

		CheckConstraintHandles(Constraints, ParticleHandles, ConstraintsHandles);
	}

	template<typename TEvolution>
	void RigidSpringConstraintHandles()
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		TEvolution Evolution(Particles, PhysicalMaterials);
		TArray<FPBDRigidParticleHandle*> ParticleHandles = Evolution.CreateDynamicParticles(5);
		for (FPBDRigidParticleHandle* ParticleHandle : ParticleHandles)
		{
			Evolution.EnableParticle(ParticleHandle);
		}

		FPBDRigidSpringConstraints Constraints;
		TArray<FPBDRigidSpringConstraintHandle*> ConstraintsHandles =
		{
			Constraints.AddConstraint({ ParticleHandles[0], ParticleHandles[1] }, { { 0, 0, 0 }, { 0, 0, 0 } }, 1.0f, 0.0f, 0.0f),
			Constraints.AddConstraint({ ParticleHandles[1], ParticleHandles[2] }, { { 0, 0, 0 }, { 0, 0, 0 } }, 1.0f, 0.0f, 0.0f),
			Constraints.AddConstraint({ ParticleHandles[2], ParticleHandles[3] }, { { 0, 0, 0 }, { 0, 0, 0 } }, 1.0f, 0.0f, 0.0f),
			Constraints.AddConstraint({ ParticleHandles[3], ParticleHandles[4] }, { { 0, 0, 0 }, { 0, 0, 0 } }, 1.0f, 0.0f, 0.0f),
		};

		CheckConstraintHandles(Constraints, ParticleHandles, ConstraintsHandles);
	}


	template<typename TEvolution>
	void RigidDynamicSpringConstraintHandles()
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		TEvolution Evolution(Particles, PhysicalMaterials);
		TArray<FPBDRigidParticleHandle*> ParticleHandles = Evolution.CreateDynamicParticles(5);
		for (FPBDRigidParticleHandle* ParticleHandle : ParticleHandles)
		{
			Evolution.EnableParticle(ParticleHandle);
		}

		FPBDRigidDynamicSpringConstraints Constraints;
		TArray<FPBDRigidDynamicSpringConstraintHandle*> ConstraintsHandles =
		{
			Constraints.AddConstraint({ ParticleHandles[0], ParticleHandles[1] }),
			Constraints.AddConstraint({ ParticleHandles[1], ParticleHandles[2] }),
			Constraints.AddConstraint({ ParticleHandles[2], ParticleHandles[3] }),
			Constraints.AddConstraint({ ParticleHandles[3], ParticleHandles[4] }),
		};

		CheckConstraintHandles(Constraints, ParticleHandles, ConstraintsHandles);
	}

	GTEST_TEST(AllEvolutions, DISABLED_ConstraintHandleTests_CollisionConstraintHandle)
	{
		CollisionConstraintHandles<FPBDRigidsEvolutionGBF>();

		SUCCEED();
	}

	GTEST_TEST(AllEvolutions, ConstraintHandleTests_JointConstraintHandle)
	{
		JointConstraintHandles<FPBDRigidsEvolutionGBF>();

		SUCCEED();
	}

	GTEST_TEST(AllEvolutions, ConstraintHandleTests_PositionConstraintHandles)
	{
		PositionConstraintHandles<FPBDRigidsEvolutionGBF>();

		SUCCEED();
	}

	GTEST_TEST(AllEvolutions, ConstraintHandleTests_RigidSpringConstraintHandles)
	{
		RigidSpringConstraintHandles<FPBDRigidsEvolutionGBF>();

		SUCCEED();
	}

	GTEST_TEST(AllEvolutions, ConstraintHandleTests_RigidDynamicSpringConstraintHandles)
	{
		RigidDynamicSpringConstraintHandles<FPBDRigidsEvolutionGBF>();

		SUCCEED();
	}
}