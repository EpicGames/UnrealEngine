// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Evolution/SolverBody.h"
#include "Chaos/GJK.h"
#include "Chaos/Pair.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"

namespace ChaosTest
{
	using namespace Chaos;

	class FCollisionSolverTest : public ::testing::Test
	{
	protected:
		FCollisionSolverTest()
		{

		}

		~FCollisionSolverTest()
		{
		}
	};

#if 0

	// We can create and initialze a constraint with simple parameters and it does something sensible
	// Create 1 static body and 1 dynamic body, separated along Z
	// Create a contact between them with some overlap
	// Check that the penetration is resolved correctly
	// Single contact point, no rotations, etc
	TEST_F(FCollisionSolverTest, TestBasicContact)
	{
		const FReal Dt;
		FSolverBody Bodies[2];
		FPBDCollisionSolver Collision;

		const FReal Radius = 50.0f;
		const FReal Overlap = 10.0f;

		const FVec3 X0 = FVec3(0);
		const FRotation3 R0 = FRotation3::FromIdentity();
		const FVec3 V0 = FVec3(0);
		const FVec3 W0 = FVec3(0);

		const FVec3 X1 = FVec3(0,0,2 * Radius - Overlap);
		const FRotation3 R1 = FRotation3::FromIdentity();
		const FVec3 V1 = FVec3(0);
		const FVec3 W1 = FVec3(0);
		const FReal M1 = 100.0f;
		const FReal I1 = 100.0f * 1000.0f;	// Approx box size 100

		// Static body
		Bodies[0].SetX(X0 - V0 * Dt));
		Bodies[0].SetP(X0);
		Bodies[0].SetR(FRotation3::IntegrateRotationWithAngularVelocity(R0, -W0, Dt));
		Bodies[0].SetQ(R0);
		Bodies[0].SetV(V0);
		Bodies[0].SetW(W0);

		// Dynamic body
		Bodies[1].SetX(X1 - V1 * Dt));
		Bodies[1].SetP(X1);
		Bodies[1].SetR(FRotation3::IntegrateRotationWithAngularVelocity(R1, -W1, Dt));
		Bodies[1].SetQ(R1);
		Bodies[1].SetV(V1);
		Bodies[1].SetW(W1);
		Bodies[1].SetInvM(FReal(1) / M1);
		Bodies[1].SetInvI(FReal(1) / I1);


		Collision.SetSolverBodies(&Bodies[0], &Bodies[1]);
		Collision.SetFriction(0, 0, 0, 0);
		Collision.SetNumManifoldPoints(1);
		Collision.InitContact(0, FVec3(0,0,Radius), FVec3(0,0,-Radius), FVec3(0,0,1));
		Collision.InitMaterial(0, 0, false, 0);

		Collision.SolvePosition(Dt, false);

		// Bodies should be separated and not rotated
		EXPECT_NEAR(Bodies[1].P().X, FReal(0), KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Bodies[1].P().Y, FReal(0), KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Bodies[1].P().Z, FReal(100), KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Bodies[1].Q().X, FReal(0), KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Bodies[1].Q().Y, FReal(0), KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Bodies[1].Q().Z, FReal(0), KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Bodies[1].Q().W, FReal(1), KINDA_SMALL_NUMBER);

		Bodies[1].SetImplicitVelocity(Dt);

		// Dynamic body should have gained Z velocity because it started penetrating
		EXPECT_NEAR(Bodies[1].V().X, FReal(0), KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Bodies[1].V().Y, FReal(0)), KINDA_SMALL_NUMBER;
		EXPECT_NEAR(Bodies[1].V().Z, Overlap * Dt, KINDA_SMALL_NUMBER);

		Collision.SolveVelocity(Dt, false);

		// Dynamic body should have 0 Z velocity because we resolved Restitution constraint
		EXPECT_NEAR(Bodies[1].V().X, FReal(0), KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Bodies[1].V().Y, FReal(0)), KINDA_SMALL_NUMBER;
		EXPECT_NEAR(Bodies[1].V().Z, FReal(0), KINDA_SMALL_NUMBER);
	}

#endif
}
