// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestConstraints.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Modules/ModuleManager.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/PBDPositionConstraints.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDSpringConstraints.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/PBDPositionConstraints.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/PBDSuspensionConstraints.h"

namespace ChaosTest {

	using namespace Chaos;

	/**
	 * Position constraint test
	 */
	template<typename TEvolution, bool bUseSimd>
	void Position()
	{
		{
			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs Particles(UniqueIndices);
			THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
			TEvolution Evolution(Particles, PhysicalMaterials);
			TArray<FPBDRigidParticleHandle*> Dynamics = Evolution.CreateDynamicParticles(1);
			Evolution.EnableParticle(Dynamics[0]);

			TArray<FVec3> Positions = { FVec3(0) };
			FPBDPositionConstraints PositionConstraints(MoveTemp(Positions), MoveTemp(Dynamics), 1.f);
			InitEvolutionSettings(Evolution);
			Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

			Evolution.AddConstraintContainer(PositionConstraints);
			Evolution.AdvanceOneTimeStep(0.1);
			Evolution.EndFrame(0.1);
			EXPECT_LT(Evolution.GetParticleHandles().Handle(0)->GetX().SizeSquared(), SMALL_NUMBER);
		}
		{
			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs Particles(UniqueIndices);
			THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
			TEvolution Evolution(Particles, PhysicalMaterials);
			InitEvolutionSettings(Evolution);
			TArray<FPBDRigidParticleHandle*> Dynamics = Evolution.CreateDynamicParticles(1);
			Dynamics[0]->SetGravityEnabled(false);
			Evolution.EnableParticle(Dynamics[0]);

			TArray<FVec3> Positions = { FVec3(1) };
			FPBDPositionConstraints PositionConstraints(MoveTemp(Positions), MoveTemp(Dynamics), 0.5f);
			Evolution.AddConstraintContainer(PositionConstraints);
			Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

			// The effect of stiffness parameter (which is set to 0.5 above) is iteration depeendent
			Evolution.SetNumPositionIterations(1);
			Evolution.SetNumVelocityIterations(1);

			Evolution.AdvanceOneTimeStep(0.1);
			Evolution.EndFrame(0.1);
			auto& Handle = Evolution.GetParticleHandles().Handle(0);
			EXPECT_LT(FMath::Abs(Handle->GetX()[0] - 0.5), (FReal)SMALL_NUMBER);
			EXPECT_LT(FMath::Abs(Handle->GetX()[1] - 0.5), (FReal)SMALL_NUMBER);
			EXPECT_LT(FMath::Abs(Handle->GetX()[2] - 0.5), (FReal)SMALL_NUMBER);

			Evolution.AdvanceOneTimeStep(0.1);
			Evolution.EndFrame(0.1);

			EXPECT_LT(FMath::Abs(Handle->GetX()[0] - 1), (FReal)SMALL_NUMBER);
			EXPECT_LT(FMath::Abs(Handle->GetX()[1] - 1), (FReal)SMALL_NUMBER);
			EXPECT_LT(FMath::Abs(Handle->GetX()[2] - 1), (FReal)SMALL_NUMBER);
		}
	}


	/**
	 * Joint constraints test with the fixed body held in place with a position constraint.
	 * Joint body swings under the fixed body at fixed distance.
	 */
	template<typename TEvolution, bool bUseSimd = false>
	void PositionAndJoint()
	{
		const int32 Iterations = 10;
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		TEvolution Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);
		TArray<FPBDRigidParticleHandle*> Dynamics = Evolution.CreateDynamicParticles(2);
		TArray<FVec3> PositionConstraintPositions = { FVec3(0, 0, 0) };

		Evolution.SetNumPositionIterations(Iterations);

		Dynamics[1]->SetX(FVec3(500, 0, 0));
		FVec3 JointConstraintPosition = FVec3(0, 0, 0);

		TArray<FPBDRigidParticleHandle*> PositionParticles = { Dynamics[0] };
		FPBDPositionConstraints PositionConstraints(MoveTemp(PositionConstraintPositions), MoveTemp(PositionParticles), 1.f);
		Evolution.AddConstraintContainer(PositionConstraints);

		TVec2<TGeometryParticleHandle<FReal, 3>*> JointParticles = { Dynamics[0], Dynamics[1] };
		FPBDJointConstraints JointConstraints;
		JointConstraints.SetUseSimd(bUseSimd);
		JointConstraints.AddConstraint(JointParticles, FRigidTransform3(JointConstraintPosition, FRotation3::FromIdentity()));
		Evolution.AddConstraintContainer(JointConstraints);

		Evolution.EnableParticle(Dynamics[0]);
		Evolution.EnableParticle(Dynamics[1]);

		FReal Dt = 0.1f;
		for (int32 TimeIndex = 0; TimeIndex < 100; ++TimeIndex)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);

			const auto& Pos0 = Dynamics[0]->GetX();
			const auto& Pos1 = Dynamics[1]->GetX();
			const float Delta0 = (Pos0 - FVec3(0, 0, 0)).Size();
			const float Separation = (Pos1 - Pos0).Size();

			EXPECT_LT(Delta0, 5);
			EXPECT_GT(Separation, 495);
			EXPECT_LT(Separation, 505);
		}
	}


	/**
	 * Position constraint test
	 */
	template<typename TEvolution>
	void SuspensionConstraintHardstop()
	{
		// Suspension setup
		FPBDSuspensionSettings SuspensionSettings;
		SuspensionSettings.Enabled = true;
		SuspensionSettings.MinLength = 2.0f;			// hard-stop
		SuspensionSettings.MaxLength = 5.0f;
		SuspensionSettings.HardstopStiffness = 1.0f;	// all about the hard-stop..
		SuspensionSettings.SpringStiffness = 0.0f;		// the spring has no effect
		SuspensionSettings.SpringDamping = 0.0f;
		SuspensionSettings.Axis = FVec3(0.0f, 0.0f, 1.0f);
		SuspensionSettings.Normal = FVec3(0.0f, 0.0f, 1.0f);

		{
			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs Particles(UniqueIndices);
			THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
			TEvolution Evolution(Particles, PhysicalMaterials);
			InitEvolutionSettings(Evolution);

			// disable gravity
			Evolution.GetGravityForces().SetAcceleration(FVec3(0, 0, 0), 0);

			// chassis particle
			TPBDRigidParticleHandle<FReal, 3>* DynamicParticle = Evolution.CreateDynamicParticles(1)[0];
			DynamicParticle->SetX(FVec3(0, 10, 10));
			//	DynamicParticle->R() = FRotation3::FromAxisAngle(FVec3(0,1,0), PI); // upside down
			DynamicParticle->I() = TVec3<FRealSingle>(100.0f);
			DynamicParticle->InvI() = TVec3<FRealSingle>(1.0f / 100.0f);

			TGeometryParticleHandle<FReal, 3>* GroundParticle = Evolution.CreateStaticParticles(1)[0];
			GroundParticle->SetX(FVec3(0, 10, 10));

			FPBDSuspensionConstraints SuspensionConstraints;
			FVec3 SuspensionLocalLocationA(FVec3(0, 0, 0));

			SuspensionSettings.Target = FVec3(0, 0, 9);

			// hard-stop will activate because 9 breaks the min suspension limit, anything greater than 8 will do this
			SuspensionConstraints.AddConstraint(DynamicParticle, SuspensionLocalLocationA, SuspensionSettings);
			SuspensionConstraints.SetGroundParticle(0, GroundParticle);

			Evolution.AddConstraintContainer(SuspensionConstraints);
			Evolution.EnableParticle(DynamicParticle);

			Evolution.AdvanceOneTimeStep(0.1);
			Evolution.EndFrame(0.1);

			const FVec3& Pos = Evolution.GetParticleHandles().Handle(0)->GetX();
			const FRotation3& Rot = Evolution.GetParticleHandles().Handle(0)->GetR();

			//UE_LOGF(LogChaos, Warning, "Pos %ls", *Pos.ToString());
			//UE_LOGF(LogChaos, Warning, "Rot %ls", *Rot.ToString());

			EXPECT_LT(FMath::Abs(Pos.X), SMALL_NUMBER);
			EXPECT_LT(FMath::Abs(Pos.Y - 10.f), SMALL_NUMBER);
			EXPECT_LT(FMath::Abs(Pos.Z - 11.f), SMALL_NUMBER);
			EXPECT_LT(Rot.X, SMALL_NUMBER);
			EXPECT_LT(Rot.Y, SMALL_NUMBER);
			EXPECT_LT(Rot.Z, SMALL_NUMBER);
		}

		{

			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs Particles(UniqueIndices);
			THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
			TEvolution Evolution(Particles, PhysicalMaterials);
			InitEvolutionSettings(Evolution);

			// disable gravity
			Evolution.GetGravityForces().SetAcceleration(FVec3(0, 0, 0), 0);

			// chassis particle
			auto* DynamicParticle = Evolution.CreateDynamicParticles(1)[0];
			DynamicParticle->SetX(FVec3(50, 10, 10));

			// minimize rotation using high inertia
			DynamicParticle->I() = TVec3<FRealSingle>(100000.0f);
			DynamicParticle->InvI() = TVec3<FRealSingle>(1.0f / 100000.0f);

			TGeometryParticleHandle<FReal, 3>* GroundParticle = Evolution.CreateStaticParticles(1)[0];
			GroundParticle->SetX(FVec3(0, 10, 10));

			FPBDSuspensionConstraints SuspensionConstraints;

			// local offset from particle origin
			FVec3 SuspensionLocalLocationA(FVector(5, 0, -2));
			FVec3 SuspensionLocalLocationB(FVector(-5, 0, -2));

			// hard-stop will activate because 9 breaks the min suspension limit, anything greater than 8 will do this
			SuspensionSettings.Target = FVec3(55, 10, 9);
			SuspensionConstraints.AddConstraint(DynamicParticle, SuspensionLocalLocationA, SuspensionSettings);
			SuspensionSettings.Target = FVec3(45, 10, 9);
			SuspensionConstraints.AddConstraint(DynamicParticle, SuspensionLocalLocationB, SuspensionSettings);
			SuspensionConstraints.SetGroundParticle(0, GroundParticle);
			SuspensionConstraints.SetGroundParticle(1, GroundParticle);

			Evolution.AddConstraintContainer(SuspensionConstraints);
			Evolution.EnableParticle(DynamicParticle);

			const FVec3& Pos = Evolution.GetParticleHandles().Handle(0)->GetX();
			const FRotation3& Rot = Evolution.GetParticleHandles().Handle(0)->GetR();

			Evolution.AdvanceOneTimeStep(0.1);
			Evolution.EndFrame(0.1);

			// rotation component from first and second hits means the positional accuracy isn't as good as the test where 
			// the single spring constraint is applied directly though the COM
			float Tolerance = 0.01f;

			//UE_LOGF(LogChaos, Warning, "Pos %ls", *Pos.ToString());
			//UE_LOGF(LogChaos, Warning, "Rot %ls", *Rot.ToString());

			EXPECT_LT(FMath::Abs(Pos.X - 50.0f), Tolerance);
			EXPECT_LT(FMath::Abs(Pos.Y - 10.f), Tolerance);
			EXPECT_LT(FMath::Abs(Pos.Z - 13.f), Tolerance); // = 9 + 2(MinLength) + 2(local offset effect)
			EXPECT_LT(Rot.X, Tolerance);
			EXPECT_LT(Rot.Y, Tolerance);
			EXPECT_LT(Rot.Z, Tolerance);
		}

	}

	template<typename TEvolution>
	void SuspensionConstraintSpring()
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		TEvolution Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);
		Evolution.SetNumPositionIterations(1);
		Evolution.SetNumVelocityIterations(1);

		// disable gravity
		Evolution.GetGravityForces().SetAcceleration(FVec3(0, 0, -980.f), 0);

		float Mass = 1.0f;

		// chassis particle
		auto* DynamicParticle = Evolution.CreateDynamicParticles(1)[0];
		DynamicParticle->SetLinearEtherDrag(0.f);
		DynamicParticle->SetX(FVec3(0, 0, 10));
		DynamicParticle->M() = Mass;
		DynamicParticle->InvM() = 1.0f / Mass;
		DynamicParticle->I() = TVec3<FRealSingle>(100000.0f);
		DynamicParticle->InvI() = TVec3<FRealSingle>(1.0f / 100000.0f);

		TGeometryParticleHandle<FReal, 3>* GroundParticle = Evolution.CreateStaticParticles(1)[0];
		GroundParticle->SetX(FVec3(0, 10, 10));

		// Suspension setup
		FPBDSuspensionSettings SuspensionSettings;
		SuspensionSettings.Enabled = true;
		SuspensionSettings.MinLength = 2.0f;	// hard-stop
		SuspensionSettings.MaxLength = 5.0f;
		SuspensionSettings.HardstopStiffness = 1.0f;
		SuspensionSettings.HardstopVelocityCompensation = 0.05f;
		SuspensionSettings.SpringStiffness = 50.0f;
		SuspensionSettings.SpringDamping = 0.5f;
		SuspensionSettings.Target = FVec3(0,0,9);
		SuspensionSettings.Axis = FVec3(0.0f, 0.0f, 1.0f);
		SuspensionSettings.Normal = FVec3(0.0f, 0.0f, 1.0f);

		TArray<FVec3> SusLocalOffset;
		SusLocalOffset.Push(FVector(0, 0, -1));

		FPBDSuspensionConstraints SuspensionConstraints;

		for (int SusIndex=0; SusIndex < SusLocalOffset.Num(); SusIndex++)
		{
			FVec3 WorldTarget = FVec3(0, 0, 9);
			SuspensionConstraints.AddConstraint(DynamicParticle, SusLocalOffset[SusIndex], SuspensionSettings);
		}

		Evolution.AddConstraintContainer(SuspensionConstraints);
		SuspensionConstraints.SetGroundParticle(0, GroundParticle);
		Evolution.EnableParticle(DynamicParticle);

		const FVec3& Pos = Evolution.GetParticleHandles().Handle(0)->GetX();
		const FRotation3& Rot = Evolution.GetParticleHandles().Handle(0)->GetR();
		const float DeltaTime = 1.0f / 30.0f;

		const float PositionTolerance = KINDA_SMALL_NUMBER;
		const float RotationTolerance = SMALL_NUMBER;

		for (int Iteration = 0; Iteration < 100; Iteration++)
		{
			Evolution.AdvanceOneTimeStep(DeltaTime);
			Evolution.EndFrame(DeltaTime);

			//UE_LOGF(LogChaos, Warning, "Pos %ls", *Pos.ToString());
 			EXPECT_GT(Pos.Z, 12.f - PositionTolerance); // should never go past hard-stop
		}

		EXPECT_GT(Pos.Z, 12.f - PositionTolerance);	// suspension min limit
		EXPECT_LT(Pos.Z, 15.f + PositionTolerance);	// suspension max limit
		EXPECT_LT(Rot.X, RotationTolerance);
		EXPECT_LT(Rot.Y, RotationTolerance);
		EXPECT_LT(Rot.Z, RotationTolerance);

	}

	GTEST_TEST(AllEvolutions, Constraints)
	{
		ChaosTest::Position<FPBDRigidsEvolutionGBF, false>();
		ChaosTest::Position<FPBDRigidsEvolutionGBF, true>();
		ChaosTest::PositionAndJoint<FPBDRigidsEvolutionGBF, false>();
		ChaosTest::PositionAndJoint<FPBDRigidsEvolutionGBF, true>();
		ChaosTest::SuspensionConstraintHardstop<FPBDRigidsEvolutionGBF>();
		ChaosTest::SuspensionConstraintSpring<FPBDRigidsEvolutionGBF>();

		SUCCEED();
	}

}
