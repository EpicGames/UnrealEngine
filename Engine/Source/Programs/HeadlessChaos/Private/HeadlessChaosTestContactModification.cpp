// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ContactModification.h"
#include "Chaos/CCDModification.h"
#include "ChaosSolversModule.h"
#include "HeadlessChaosTestUtility.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

namespace ChaosTest
{
	using namespace Chaos;

	class FContactModificationTestCallback : public Chaos::TSimCallbackObject<
		Chaos::FSimCallbackNoInput,
		Chaos::FSimCallbackNoOutput,
		Chaos::ESimCallbackOptions::Presimulate | Chaos::ESimCallbackOptions::ContactModification | Chaos::ESimCallbackOptions::CCDModification>
	{
	public:
		TUniqueFunction<void(Chaos::FCollisionContactModifier&)> TestLambda;
		TUniqueFunction<void(Chaos::FCCDModifierAccessor&)> TestCCDLambda = nullptr;

	private:
		virtual void OnPreSimulate_Internal() override {}
		virtual void OnContactModification_Internal(Chaos::FCollisionContactModifier& Modifier) override;
		virtual void OnCCDModification_Internal(Chaos::FCCDModifierAccessor& Accessor) override;
	};

	void FContactModificationTestCallback::OnContactModification_Internal(Chaos::FCollisionContactModifier& Modifier)
	{
		TestLambda(Modifier);
	}

	void FContactModificationTestCallback::OnCCDModification_Internal(Chaos::FCCDModifierAccessor& Accessor)
	{
		if (TestCCDLambda)
		{
			TestCCDLambda(Accessor);
		}
	}


	GTEST_TEST(AllTraits, ContactModification_Disable)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// create a static floor and two boxes falling onto it.
		// One box has contacts disabled and should fall through, one should collide.

		// simulated cube with downward velocity,should collide with floor and not fall through.
		FSingleParticlePhysicsProxy* CollidingCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& CollidingCubeParticle = CollidingCubeProxy->GetGameThreadAPI();
		auto CollidingCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		CollidingCubeParticle.SetGeometry(CollidingCubeGeom);
		Solver->RegisterObject(CollidingCubeProxy);
		CollidingCubeParticle.SetGravityEnabled(false);
		CollidingCubeParticle.SetV(FVec3(0,0,-100));
		CollidingCubeParticle.SetX(FVec3(200,0,500));
		SetCubeInertiaTensor(CollidingCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({CollidingCubeProxy->GetParticle_LowLevel()});

		// Simulated cube with downawrd velocity, contact modification disables collision with floor, should fall through.
		FSingleParticlePhysicsProxy* ModifiedCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& ModifiedCubeParticle = ModifiedCubeProxy->GetGameThreadAPI();
		auto ModifiedCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		ModifiedCubeParticle.SetGeometry(ModifiedCubeGeom);
		Solver->RegisterObject(ModifiedCubeProxy);
		ModifiedCubeParticle.SetGravityEnabled(false);
		ModifiedCubeParticle.SetV(FVec3(0, 0, -100));
		ModifiedCubeParticle.SetX(FVec3(-200, 0, 500));
		SetCubeInertiaTensor(ModifiedCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ ModifiedCubeProxy->GetParticle_LowLevel() });


		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0,0,0));
		ChaosTest::SetParticleSimDataToCollide({FloorProxy->GetParticle_LowLevel()});


		// Save Unique indices of floor and modified cube to disable in contact mod.
		TVec2<FUniqueIdx> UniqueIndices({ModifiedCubeParticle.UniqueIdx(), FloorParticle.UniqueIdx()});

		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [UniqueIndices](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();
				FUniqueIdx Idx0 = Particles[0]->UniqueIdx();
				FUniqueIdx Idx1 = Particles[1]->UniqueIdx();

				// If unique indices match disable the pair.
				if( (UniqueIndices[0] == Idx0 && UniqueIndices[1] == Idx1) ||
					(UniqueIndices[0] == Idx1 && UniqueIndices[1] == Idx0))
				{
					PairModifier.Disable();
				}
			}
		};

	
		const float Dt = 1.0f;
		const int32 Steps = 10;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		// Modified cube should be below floor because we disabled collision.
		EXPECT_LT(ModifiedCubeParticle.X().Z, FloorParticle.X().Z);

		// Colliding cube should be above floor due to collision.
		EXPECT_GT(CollidingCubeParticle.X().Z, FloorParticle.X().Z);

		// Floor should be at origin.
		EXPECT_EQ(FloorParticle.X().Z, 0);


		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(CollidingCubeProxy);
		Solver->UnregisterObject(ModifiedCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, ContactModification_Probe)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// create a static floor and two boxes falling onto it.
		// Both boxes should turn all contacts to probes, one of them has CCD enabled
		// and the other one doesn't.

		auto CubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));

		// Fall through the floor, no ccd
		FSingleParticlePhysicsProxy* CubeProxyA = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& CubeParticleA = CubeProxyA->GetGameThreadAPI();
		CubeParticleA.SetGeometry(CubeGeom);
		Solver->RegisterObject(CubeProxyA);
		CubeParticleA.SetGravityEnabled(false);
		CubeParticleA.SetCCDEnabled(false);
		CubeParticleA.SetV(FVec3(0, 0, -100));
		CubeParticleA.SetX(FVec3(200, 0, 500));
		SetCubeInertiaTensor(CubeParticleA, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ CubeProxyA->GetParticle_LowLevel() });

		// Fall through the floor, with ccd
		FSingleParticlePhysicsProxy* CubeProxyB = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& CubeParticleB = CubeProxyB->GetGameThreadAPI();
		CubeParticleB.SetGeometry(CubeGeom);
		Solver->RegisterObject(CubeProxyB);
		CubeParticleB.SetGravityEnabled(false);
		CubeParticleB.SetCCDEnabled(true);
		CubeParticleB.SetV(FVec3(0, 0, -1000));
		CubeParticleB.SetX(FVec3(-200, 0, 500));
		SetCubeInertiaTensor(CubeParticleB, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ CubeProxyB->GetParticle_LowLevel() });


		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });

		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				PairModifier.ConvertToProbe();
			}
		};
		
		Callback->TestCCDLambda = [CubeProxyB](Chaos::FCCDModifierAccessor& Accessor)
		{
			if (Chaos::FGeometryParticleHandle* ParticleHandle = CubeProxyB->GetHandle_LowLevel())
			{
				for (FCCDModifier& CCDModifier : Accessor.GetModifiers(ParticleHandle))
				{
					CCDModifier.ConvertToProbe();
				}
			}
		};


		const float Dt = 1.0f;
		const int32 Steps = 10;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		// TODO: Check for hit callbacks?

		// Both cubes should be below the floor
		EXPECT_LT(CubeParticleA.X().Z, FloorParticle.X().Z);
		EXPECT_LT(CubeParticleB.X().Z, FloorParticle.X().Z);

		// Floor should be at origin.
		EXPECT_EQ(FloorParticle.X().Z, 0);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(CubeProxyA);
		Solver->UnregisterObject(CubeProxyB);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	// Disabling due to: UE-216793; Objects fall through the floor with ConvertToNonProbe modification.
	GTEST_TEST(AllTraits, DISABLED_ContactModification_NonProbe)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// Create a static floor and two cubes falling onto it.
		// Both cubes start off as probes and should apply a modification to turn all contacts to non-probes, 
		// CubeB has CCD enabled and CubeA does not.

		auto CubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));

		// CubeA - Probe & No CCD
		FSingleParticlePhysicsProxy* CubeProxyA = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& CubeParticleA = CubeProxyA->GetGameThreadAPI();
		CubeParticleA.SetGeometry(CubeGeom);
		Solver->RegisterObject(CubeProxyA);
		CubeParticleA.SetGravityEnabled(false);
		CubeParticleA.SetCCDEnabled(false);
		CubeParticleA.SetV(FVec3(0, 0, -100));
		CubeParticleA.SetX(FVec3(200, 0, 500));
		CubeParticleA.ShapesArray()[0]->SetIsProbe(true);
		SetCubeInertiaTensor(CubeParticleA, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ CubeProxyA->GetParticle_LowLevel() });

		
		// CubeB - Probe & With CCD
		FSingleParticlePhysicsProxy* CubeProxyB = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& CubeParticleB = CubeProxyB->GetGameThreadAPI();
		CubeParticleB.SetGeometry(CubeGeom);
		Solver->RegisterObject(CubeProxyB);
		CubeParticleB.SetGravityEnabled(false);
		CubeParticleB.SetCCDEnabled(true);
		CubeParticleB.SetV(FVec3(0, 0, -1000));
		CubeParticleB.SetX(FVec3(-200, 0, 500));
		CubeParticleB.ShapesArray()[0]->SetIsProbe(true);
		SetCubeInertiaTensor(CubeParticleB, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ CubeProxyB->GetParticle_LowLevel() });
		

		// Static floor at origin, occupying Z = [-100,0], XY = [-500, 500]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });

		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [](Chaos::FCollisionContactModifier& Modifier)
			{
				for (FContactPairModifier& PairModifier : Modifier)
				{
					PairModifier.ConvertToNonProbe();
				}
			};

		const float Dt = 1.0f;
		const int32 Steps = 10;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		// Both cubes should be above the floor
		EXPECT_GT(CubeParticleA.X().Z, FloorParticle.X().Z);
		EXPECT_GT(CubeParticleB.X().Z, FloorParticle.X().Z);

		// Floor should still be at origin.
		EXPECT_EQ(FloorParticle.X().Z, 0);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(CubeProxyA);
		Solver->UnregisterObject(CubeProxyB);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, ContactModification_ModifySeparation)
	{
		// The amount to pad the saparation by in the collision callback. Currently this must
		// be less than the CullDistance specified in the settings (3.0)
		// @todo(chaos): allow the user to pad bounds to support position modification
		const FReal SeparationPadding = 2.0f;

		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// create a static floor and two boxes falling onto it.
		// One box has separation modified to float 5 units above floor.
		// Other box is not modified and should rest on top of floor.

		// simulated cube with downward velocity, should rest directly on floor.
		FSingleParticlePhysicsProxy* RegularCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& RegularCubeParticle = RegularCubeProxy->GetGameThreadAPI();
		auto RegularCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		RegularCubeParticle.SetGeometry(RegularCubeGeom);
		Solver->RegisterObject(RegularCubeProxy);
		RegularCubeParticle.SetGravityEnabled(true);
		RegularCubeParticle.SetX(FVec3(200, 0, 110));
		SetCubeInertiaTensor(RegularCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ RegularCubeProxy->GetParticle_LowLevel() });

		// Simulated cube with downawrd velocity, contact modification subtracts SeparationPadding from separation,  causing cube to rest above floor.
		FSingleParticlePhysicsProxy* ModifiedCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& ModifiedCubeParticle = ModifiedCubeProxy->GetGameThreadAPI();
		auto ModifiedCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		ModifiedCubeParticle.SetGeometry(ModifiedCubeGeom);
		Solver->RegisterObject(ModifiedCubeProxy);
		ModifiedCubeParticle.SetGravityEnabled(true);
		ModifiedCubeParticle.SetX(FVec3(-200, 0, 110));
		SetCubeInertiaTensor(ModifiedCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ ModifiedCubeProxy->GetParticle_LowLevel() });


		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });


		// Save Unique indices of floor and modified cube to disable in contact mod.
		TVec2<FUniqueIdx> UniqueIndices({ ModifiedCubeParticle.UniqueIdx(), FloorParticle.UniqueIdx() });

		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [UniqueIndices, SeparationPadding](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();
				FUniqueIdx Idx0 = Particles[0]->UniqueIdx();
				FUniqueIdx Idx1 = Particles[1]->UniqueIdx();

				// If unique indices match disable the pair.
				if ((UniqueIndices[0] == Idx0 && UniqueIndices[1] == Idx1) ||
					(UniqueIndices[0] == Idx1 && UniqueIndices[1] == Idx0))
				{
					int32 NumContacts = PairModifier.GetNumContacts();
					for (int32 PointIdx = 0; PointIdx < NumContacts; ++PointIdx)
					{
						PairModifier.ModifyTargetSeparation(SeparationPadding, PointIdx);
					}
				}
			}
		};


		const float Dt = 0.1f;
		const int32 Steps = 30;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		const float PositionTolerance = 1.e-2f;

		// Modified cube should be resting SeparationPadding above floor, as we added that penetration through contact mod.
		EXPECT_NEAR(ModifiedCubeParticle.X().Z, 100.f + SeparationPadding, PositionTolerance);

		// Colliding cube should be resting on floor.
		EXPECT_NEAR(RegularCubeParticle.X().Z, 100.f, PositionTolerance);

		// Floor should be at origin.
		EXPECT_EQ(FloorParticle.X().Z, 0);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(RegularCubeProxy);
		Solver->UnregisterObject(ModifiedCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, ContactModification_ModifyNormal)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// create a static floor and two boxes falling onto it.
		// One box has normal modified to be parallel to floor, should fall through floor due to non-upward normal.
		// Other box is not modified and should rest on top of floor.

		// simulated cube with downward velocity, should rest directly on floor.
		FSingleParticlePhysicsProxy* RegularCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& RegularCubeParticle = RegularCubeProxy->GetGameThreadAPI();
		auto RegularCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		RegularCubeParticle.SetGeometry(RegularCubeGeom);
		Solver->RegisterObject(RegularCubeProxy);
		RegularCubeParticle.SetGravityEnabled(false);
		RegularCubeParticle.SetV(FVec3(0, 0, -100));
		RegularCubeParticle.SetX(FVec3(200, 0, 500));
		SetCubeInertiaTensor(RegularCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ RegularCubeProxy->GetParticle_LowLevel() });

		FSingleParticlePhysicsProxy* ModifiedCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& ModifiedCubeParticle = ModifiedCubeProxy->GetGameThreadAPI();
		auto ModifiedCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		ModifiedCubeParticle.SetGeometry(ModifiedCubeGeom);
		Solver->RegisterObject(ModifiedCubeProxy);
		ModifiedCubeParticle.SetGravityEnabled(false);
		ModifiedCubeParticle.SetV(FVec3(0, 0, -100));
		ModifiedCubeParticle.SetX(FVec3(-200, 0, 500));
		SetCubeInertiaTensor(ModifiedCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ ModifiedCubeProxy->GetParticle_LowLevel() });


		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });


		// Save Unique indices of floor and modified cube to disable in contact mod.
		TVec2<FUniqueIdx> UniqueIndices({ ModifiedCubeParticle.UniqueIdx(), FloorParticle.UniqueIdx() });

		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [UniqueIndices](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();
				FUniqueIdx Idx0 = Particles[0]->UniqueIdx();
				FUniqueIdx Idx1 = Particles[1]->UniqueIdx();

				// If unique indices match disable the pair.
				if ((UniqueIndices[0] == Idx0 && UniqueIndices[1] == Idx1) ||
					(UniqueIndices[0] == Idx1 && UniqueIndices[1] == Idx0))
				{
					int32 NumContacts = PairModifier.GetNumContacts();
					for (int32 PointIdx = 0; PointIdx < NumContacts; ++PointIdx)
					{
						FVec3 NewNormal(-1, 0, 0);
						PairModifier.ModifyWorldNormal(NewNormal, PointIdx);
					}
				}
			}
		};


		const float Dt = 1.0f;
		const int32 Steps = 10;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		// Normal was modified to be parallel to floor, should fall through and not collide.	
		EXPECT_LT(ModifiedCubeParticle.X().Z, 0.f);
		EXPECT_LT(ModifiedCubeParticle.V().Z, 0.f);

		// non-modified cube should be resting on floor.
		EXPECT_NEAR(RegularCubeParticle.X().Z, 100.f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(RegularCubeParticle.V().Z, 0.f, KINDA_SMALL_NUMBER);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(RegularCubeProxy);
		Solver->UnregisterObject(ModifiedCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, ContactModification_ModifyLocationWorldSpace)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// create a static floor and two boxes colliding on edge of floor each with center of mass over the edge.
		// One cube should rotate off side, the other has contact point locations moved under center of mass,
		// so cube does not rotate and fall, instead remains on floor.

		// simulated cube falling onto floor with center of mass over hanging past edge, should fall under floor,
		FSingleParticlePhysicsProxy* FallingCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& FallingCubeParticle = FallingCubeProxy->GetGameThreadAPI();
		auto FallingCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		FallingCubeParticle.SetGeometry(FallingCubeGeom);
		Solver->RegisterObject(FallingCubeProxy);
		FallingCubeParticle.SetGravityEnabled(true);
		FallingCubeParticle.SetX(FVec3(550, 0, 110));
		SetCubeInertiaTensor(FallingCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ FallingCubeProxy->GetParticle_LowLevel() });

		// cube with CoM hanging past edge of floor, contact mod moves contact under CoM so it will not tip off edge.
		FSingleParticlePhysicsProxy* ModifiedCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& ModifiedCubeParticle = ModifiedCubeProxy->GetGameThreadAPI();
		auto ModifiedCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		ModifiedCubeParticle.SetGeometry(ModifiedCubeGeom);
		Solver->RegisterObject(ModifiedCubeProxy);
		ModifiedCubeParticle.SetGravityEnabled(true);
		ModifiedCubeParticle.SetX(FVec3(-550, 0, 110));
		SetCubeInertiaTensor(ModifiedCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ ModifiedCubeProxy->GetParticle_LowLevel() });


		// static floor at origin, X/Y spanning [-500, 500] and Z spanning [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });


		// Save Unique indices of floor and modified cube to disable in contact mod.
		TVec2<FUniqueIdx> UniqueIndices({ ModifiedCubeParticle.UniqueIdx(), FloorParticle.UniqueIdx() });

		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [UniqueIndices](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();
				FUniqueIdx Idx0 = Particles[0]->UniqueIdx();
				FUniqueIdx Idx1 = Particles[1]->UniqueIdx();

				// If unique indices match disable the pair.
				if ((UniqueIndices[0] == Idx0 && UniqueIndices[1] == Idx1) ||
					(UniqueIndices[0] == Idx1 && UniqueIndices[1] == Idx0))
				{
					int32 NumContacts = PairModifier.GetNumContacts();
					for (int32 PointIdx = 0; PointIdx < NumContacts; ++PointIdx)
					{
						// Move contact locations below center of mass
						FVec3 WorldPos0;
						FVec3 WorldPos1;
						PairModifier.GetWorldContactLocations(PointIdx, WorldPos0, WorldPos1);

						int32 DynamicIdx = (UniqueIndices[0] == Idx0) ? 0 : 1;
						const FVec3 CoM = FParticleUtilities::GetCoMWorldPosition(FConstGenericParticleHandle(Particles[DynamicIdx]));

						// Move point0 under center of mass and move second point under CoM but keep the same distance between bodies
						FVec3 PointUnderCoM0(CoM.X, CoM.Y, WorldPos0.Z);
						FVec3 PointUnderCoM1(CoM.X, CoM.Y, WorldPos1.Z);
						PairModifier.ModifyWorldContactLocations(PointUnderCoM0, PointUnderCoM1, PointIdx);
					}
				}
			}
		};


		const float Dt = 0.1f;
		const int32 Steps = 10;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		// Modified contact points to be below CoM, cube should not tip off edge of floor, but rest on it instead.
		EXPECT_NEAR(ModifiedCubeParticle.X().Z, 100.f, KINDA_SMALL_NUMBER);

		// Expected to tip off edge as CoM hangs off of floor.
		EXPECT_LT(FallingCubeParticle.X().Z, 0.f);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(FallingCubeProxy);
		Solver->UnregisterObject(ModifiedCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, ContactModification_ModifyRestitution_NoBounce)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// Collide cube with static floor with restitution modified to 0

		// Simulated cube with downawrd velocity, should not bounce, and end up with zero velocity.
		FSingleParticlePhysicsProxy* NoBounceCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& NoBounceCubeParticle = NoBounceCubeProxy->GetGameThreadAPI();
		auto NoBounceCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		NoBounceCubeParticle.SetGeometry(NoBounceCubeGeom);
		Solver->RegisterObject(NoBounceCubeProxy);
		NoBounceCubeParticle.SetGravityEnabled(false);
		NoBounceCubeParticle.SetX(FVec3(-200, 0, 200));
		SetCubeInertiaTensor(NoBounceCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ NoBounceCubeProxy->GetParticle_LowLevel() });
		NoBounceCubeParticle.SetV(FVec3(0, 0, -100));

		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });

		// Save Unique indices of floor and modified cube to disable in contact mod.
		TVec2<FUniqueIdx> NoBounceUniqueIndices({ NoBounceCubeParticle.UniqueIdx(), FloorParticle.UniqueIdx() });

		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [NoBounceUniqueIndices](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();
				FUniqueIdx Idx0 = Particles[0]->UniqueIdx();
				FUniqueIdx Idx1 = Particles[1]->UniqueIdx();

				if ((NoBounceUniqueIndices[0] == Idx0 && NoBounceUniqueIndices[1] == Idx1) ||
					(NoBounceUniqueIndices[0] == Idx1 && NoBounceUniqueIndices[1] == Idx0))
				{
					int32 NumContacts = PairModifier.GetNumContacts();
					for (int32 PointIdx = 0; PointIdx < NumContacts; ++PointIdx)
					{
						// Make sure that values are not held between frames
						// @todo(chaos): this test actually has zero restitution
						//EXPECT_GT(PairModifier.GetRestitution(), 0);

						// Remove restitution
						PairModifier.ModifyRestitution(0);
					}
				}
			}
		};

		// If we rely on good restitution, we need more velocity iterations
		Solver->GetEvolution()->SetNumVelocityIterations(4);

		const float Dt = 0.1f;
		const int32 Steps = 30;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		
		EXPECT_NEAR(NoBounceCubeParticle.V().Z, 0.f, 0.1);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(NoBounceCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}	


	GTEST_TEST(AllTraits, ContactModification_ModifyRestitution_Bounce)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// Collide cubes with static floor  with restitution modified to 1

		// simulated cube with downward velocity, should bounce on floor and end up with upward velocity.
		FSingleParticlePhysicsProxy* BounceCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& BounceCubeParticle = BounceCubeProxy->GetGameThreadAPI();
		auto BounceCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		BounceCubeParticle.SetGeometry(BounceCubeGeom);
		Solver->RegisterObject(BounceCubeProxy);
		BounceCubeParticle.SetGravityEnabled(false);
		BounceCubeParticle.SetX(FVec3(200, 0, 200));
		SetCubeInertiaTensor(BounceCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ BounceCubeProxy->GetParticle_LowLevel() });
		BounceCubeParticle.SetV(FVec3(0, 0, -100));

		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });

		// Save Unique indices of floor and modified cube to disable in contact mod.
		TVec2<FUniqueIdx> BounceUniqueIndices({ BounceCubeParticle.UniqueIdx(), FloorParticle.UniqueIdx() });

		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [BounceUniqueIndices](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();
				FUniqueIdx Idx0 = Particles[0]->UniqueIdx();
				FUniqueIdx Idx1 = Particles[1]->UniqueIdx();

				if ((BounceUniqueIndices[0] == Idx0 && BounceUniqueIndices[1] == Idx1) ||
					(BounceUniqueIndices[0] == Idx1 && BounceUniqueIndices[1] == Idx0))
				{
					int32 NumContacts = PairModifier.GetNumContacts();
					for (int32 PointIdx = 0; PointIdx < NumContacts; ++PointIdx)
					{
						// set restitution
						PairModifier.ModifyRestitution(1);

						// Our object is slow enough restitution will not be applied with default threshold.
						PairModifier.ModifyRestitutionThreshold(99);
					}
				}
			}
		};

		// If we rely on good restitution, we need more velocity iterations
		Solver->GetEvolution()->SetNumVelocityIterations(4);

		const float Dt = 0.1f;
		const int32 Steps = 30;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		EXPECT_NEAR(BounceCubeParticle.V().Z, 100.f, 0.1);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(BounceCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, ContactModification_ModifyFriction)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// two cubes fall on tilted floor, one is expecting to slide off, one has friction modified to keep it on floor.

		FSingleParticlePhysicsProxy* SlidingCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& SlidingCubeParticle = SlidingCubeProxy->GetGameThreadAPI();
		auto SlidingCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		SlidingCubeParticle.SetGeometry(SlidingCubeGeom);
		Solver->RegisterObject(SlidingCubeProxy);
		SlidingCubeParticle.SetGravityEnabled(true);
		SlidingCubeParticle.SetX(FVec3(200, -200, 50));
		SlidingCubeParticle.SetR(FQuat::MakeFromEuler(FVec3(0, 20, 0)));
		SetCubeInertiaTensor(SlidingCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ SlidingCubeProxy->GetParticle_LowLevel() });

		FSingleParticlePhysicsProxy* ModifiedCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& ModifiedCubeParticle = ModifiedCubeProxy->GetGameThreadAPI();
		auto ModifiedCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		ModifiedCubeParticle.SetGeometry(ModifiedCubeGeom);
		Solver->RegisterObject(ModifiedCubeProxy);
		ModifiedCubeParticle.SetGravityEnabled(true);
		ModifiedCubeParticle.SetX(FVec3(200, 200, 50));
		ModifiedCubeParticle.SetR(FQuat::MakeFromEuler(FVec3(0, 20, 0)));
		SetCubeInertiaTensor(ModifiedCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ ModifiedCubeProxy->GetParticle_LowLevel() });


		// static floor rotated 30 degrees
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		FloorParticle.SetR(FQuat::MakeFromEuler(FVec3(0, 20, 0)));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });


		// Save Unique indices of floor and modified cube to disable in contact mod.
		TVec2<FUniqueIdx> UniqueIndices({ ModifiedCubeParticle.UniqueIdx(), FloorParticle.UniqueIdx() });

		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [UniqueIndices](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();
				FUniqueIdx Idx0 = Particles[0]->UniqueIdx();
				FUniqueIdx Idx1 = Particles[1]->UniqueIdx();

				// If unique indices match disable the pair.
				if ((UniqueIndices[0] == Idx0 && UniqueIndices[1] == Idx1) ||
					(UniqueIndices[0] == Idx1 && UniqueIndices[1] == Idx0))
				{
					// Make sure that values are not held between frames
					EXPECT_LT(PairModifier.GetDynamicFriction(), 1);
					EXPECT_LT(PairModifier.GetStaticFriction(), 1);

					PairModifier.ModifyDynamicFriction(1);
					PairModifier.ModifyStaticFriction(1);
				}
			}
		};


		const float Dt = 0.1f;
		const int32 Steps = 50;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		// Verify modified cube with increased friction sticks to floor.
		EXPECT_NEAR(ModifiedCubeParticle.V().Z, 0.f, KINDA_SMALL_NUMBER);

		// This cube should have slid off floor.
		EXPECT_LT(SlidingCubeParticle.V().Z, 0.f);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(SlidingCubeProxy);
		Solver->UnregisterObject(ModifiedCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, ContactModification_ModifyParticleVelocity)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);


		// simulated cube with downward velocity, on contact modification set an upward velocity so it should move away from floor,.
		FSingleParticlePhysicsProxy* ModifiedCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& ModifiedCubeParticle = ModifiedCubeProxy->GetGameThreadAPI();
		auto ModifiedCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		ModifiedCubeParticle.SetGeometry(ModifiedCubeGeom);
		Solver->RegisterObject(ModifiedCubeProxy);
		ModifiedCubeParticle.SetGravityEnabled(false);
		ModifiedCubeParticle.SetX(FVec3(200, 0, 500));
		SetCubeInertiaTensor(ModifiedCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ ModifiedCubeProxy->GetParticle_LowLevel() });
		ModifiedCubeParticle.SetV(FVec3(0, 0, -100));


		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });

		FVec3 NewVelocity(100,0,100);
		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [NewVelocity](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();
				int32 DynamicParticleIdx = (Particles[0]->ObjectState() == EObjectStateType::Dynamic) ? 0 : 1;
				PairModifier.ModifyParticleVelocity(NewVelocity, DynamicParticleIdx);
			}
		};


		const float Dt = 1.0f;
		const int32 Steps = 10;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		EXPECT_NEAR(ModifiedCubeParticle.V().X, NewVelocity.X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ModifiedCubeParticle.V().Y, NewVelocity.Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ModifiedCubeParticle.V().Z, NewVelocity.Z, KINDA_SMALL_NUMBER);

		// Make sure we didn't somehow go through floor, once close to floor should have been moving parallel to floor.
		EXPECT_GT(ModifiedCubeParticle.X().Z, 0.f);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(ModifiedCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, ContactModification_ModifyParticleAngularVelocity)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);


		// simulated cube falling on floor, on contact modification set angular velocity to make it spin
		FSingleParticlePhysicsProxy* ModifiedCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& ModifiedCubeParticle = ModifiedCubeProxy->GetGameThreadAPI();
		auto ModifiedCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		ModifiedCubeParticle.SetGeometry(ModifiedCubeGeom);
		Solver->RegisterObject(ModifiedCubeProxy);
		ModifiedCubeParticle.SetGravityEnabled(true);
		ModifiedCubeParticle.SetX(FVec3(200, 0, 500));
		SetCubeInertiaTensor(ModifiedCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ ModifiedCubeProxy->GetParticle_LowLevel() });


		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });

		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();
				int32 DynamicParticleIdx = (Particles[0]->ObjectState() == EObjectStateType::Dynamic) ? 0 : 1;
				PairModifier.ModifyParticleAngularVelocity(FVec3(0, 0, 1), DynamicParticleIdx);
			}
		};


		const float Dt = 0.1f;
		const int32 Steps = 10;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}


		// Did the modification of angular velocity work?
		EXPECT_NEAR(ModifiedCubeParticle.W().Z, 1.f, 0.1);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(ModifiedCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, ContactModification_ModifyParticlePositionAndVelocity)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);


		// simulated cube with downward velocity, on contact modification teleport particle and clear velocity.
		FSingleParticlePhysicsProxy* ModifiedCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& ModifiedCubeParticle = ModifiedCubeProxy->GetGameThreadAPI();
		auto ModifiedCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		ModifiedCubeParticle.SetGeometry(ModifiedCubeGeom);
		Solver->RegisterObject(ModifiedCubeProxy);
		ModifiedCubeParticle.SetGravityEnabled(false);
		ModifiedCubeParticle.SetX(FVec3(200, 0, 500));
		SetCubeInertiaTensor(ModifiedCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ ModifiedCubeProxy->GetParticle_LowLevel() });
		ModifiedCubeParticle.SetV(FVec3(0, 0, -100));


		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });

		FVec3 TeleportPosition(1000,2000,3000);
		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [TeleportPosition](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();
				int32 DynamicParticleIdx = (Particles[0]->ObjectState() == EObjectStateType::Dynamic) ? 0 : 1;

				// Clear velocity
				PairModifier.ModifyParticleVelocity(FVec3(0, 0, 0), DynamicParticleIdx);

				// Maintain change in velocity, we do not want moving particle to change implicit velocity.
				PairModifier.ModifyParticlePosition(TeleportPosition, /*bMaintainVelocity=*/true, DynamicParticleIdx);
			}
		};


		const float Dt = 1.0f;
		const int32 Steps = 10;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}


		// Are we where we moved particle to?
		EXPECT_NEAR(ModifiedCubeParticle.X().X, TeleportPosition.X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ModifiedCubeParticle.X().Y, TeleportPosition.Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ModifiedCubeParticle.X().Z, TeleportPosition.Z, KINDA_SMALL_NUMBER);

		// Make sure we did not get any velocity once clearing it in contact mod.
		EXPECT_NEAR(ModifiedCubeParticle.V().SizeSquared(), 0.f, KINDA_SMALL_NUMBER);


		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(ModifiedCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, ContactModification_ModifyParticleRotationAndMaintainAngularVelocity)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// We use contact mod to rotate cube and maintain angular velocity of 0.

		// simulated cube with downward velocity, on contact modification rotate particle.
		FSingleParticlePhysicsProxy* ModifiedCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& ModifiedCubeParticle = ModifiedCubeProxy->GetGameThreadAPI();
		auto ModifiedCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		ModifiedCubeParticle.SetGeometry(ModifiedCubeGeom);
		Solver->RegisterObject(ModifiedCubeProxy);
		ModifiedCubeParticle.SetGravityEnabled(true);
		ModifiedCubeParticle.SetX(FVec3(200, 0, 500));
		SetCubeInertiaTensor(ModifiedCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ ModifiedCubeProxy->GetParticle_LowLevel() });

		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });

		FRotation3 ModificationRotation(FQuat::MakeFromEuler(FVec3(0, 0, 45)));
		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [ModificationRotation](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();
				int32 DynamicParticleIdx = (Particles[0]->ObjectState() == EObjectStateType::Dynamic) ? 0 : 1;

				PairModifier.ModifyDynamicFriction(0);
				PairModifier.ModifyStaticFriction(0);
				PairModifier.ModifyParticleRotation(ModificationRotation, /*bMaintainVelocity=*/true, DynamicParticleIdx);
			}
		};


		const float Dt = .1f;
		const int32 Steps = 10;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		// Do we have rotation applied in contact mod?
		EXPECT_NEAR(ModifiedCubeParticle.R().X, ModificationRotation.X, .001);
		EXPECT_NEAR(ModifiedCubeParticle.R().Y, ModificationRotation.Y, .001);
		EXPECT_NEAR(ModifiedCubeParticle.R().Z, ModificationRotation.Z, .001);
		EXPECT_NEAR(ModifiedCubeParticle.R().W, ModificationRotation.W, .001);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(ModifiedCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}


	// Drop a dynamic cube onto a kinematic cube with an offset so the dynamic cube would start to
	// rotate, except that we have set the inertia scale to zero so it shouldn't rotate. It should actually 
	// just stop when it hits, and never tip off.
	GTEST_TEST(AllTraits, ContactModification_ModifyParticleInertiaZero)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// We use contact mod to rotate cube and maintain angular velocity of 0.

		// simulated cube dropped from just above the floor
		FSingleParticlePhysicsProxy* ModifiedCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& ModifiedCubeParticle = ModifiedCubeProxy->GetGameThreadAPI();
		auto ModifiedCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		ModifiedCubeParticle.SetGeometry(ModifiedCubeGeom);
		Solver->RegisterObject(ModifiedCubeProxy);
		ModifiedCubeParticle.SetGravityEnabled(true);
		ModifiedCubeParticle.SetX(FVec3(0, 0, 150));
		SetCubeInertiaTensor(ModifiedCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ ModifiedCubeProxy->GetParticle_LowLevel() });

		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(-90, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });

		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [](Chaos::FCollisionContactModifier& Modifier)
		{
			for (FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();
				int32 DynamicParticleIdx = (Particles[0]->ObjectState() == EObjectStateType::Dynamic) ? 0 : 1;

				// Make sure that the values were not held over from the last call
				EXPECT_NEAR(PairModifier.GetInvInertiaScale(0), 1.0, UE_SMALL_NUMBER);
				EXPECT_NEAR(PairModifier.GetInvInertiaScale(1), 1.0, UE_SMALL_NUMBER);

				PairModifier.ModifyRestitution(0);
				PairModifier.ModifyInvInertiaScale(0, DynamicParticleIdx);
			}
		};


		const float Dt = .1f;
		const int32 Steps = 20;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		// Do we have rotation applied in contact mod?
		EXPECT_NEAR(ModifiedCubeParticle.R().X, 0.0, 0.001);
		EXPECT_NEAR(ModifiedCubeParticle.R().Y, 0.0, 0.001);
		EXPECT_NEAR(ModifiedCubeParticle.R().Z, 0.0, 0.001);
		EXPECT_NEAR(ModifiedCubeParticle.R().W, 1.0, 0.001);

		// Body should be sat on the floor even though it is hanging off the edge
		EXPECT_NEAR(ModifiedCubeParticle.X().X, 0.0, 0.001);
		EXPECT_NEAR(ModifiedCubeParticle.X().Y, 0.0, 0.001);
		EXPECT_NEAR(ModifiedCubeParticle.X().Z, 100.0, 0.001);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(ModifiedCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, ContactModification_SelectByParticle)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// Similar to the "Disable" test:
		// - Create a static floor and two boxes falling onto it.
		// - One box has contacts disabled and should fall through, one should collide.
		// - Rather than loop over all contacts, get contacts for a particular particle proxy

		// simulated cube with downward velocity,should collide with floor and not fall through.
		FSingleParticlePhysicsProxy* CollidingCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& CollidingCubeParticle = CollidingCubeProxy->GetGameThreadAPI();
		auto CollidingCubeGeom = TRefCountPtr<FImplicitObject>(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		CollidingCubeParticle.SetGeometry(CollidingCubeGeom);
		Solver->RegisterObject(CollidingCubeProxy);
		CollidingCubeParticle.SetGravityEnabled(false);
		CollidingCubeParticle.SetV(FVec3(0, 0, -100));
		CollidingCubeParticle.SetX(FVec3(200, 0, 500));
		SetCubeInertiaTensor(CollidingCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ CollidingCubeProxy->GetParticle_LowLevel() });

		// Simulated cube with downawrd velocity, contact modification disables collision with floor, should fall through.
		FSingleParticlePhysicsProxy* ModifiedCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& ModifiedCubeParticle = ModifiedCubeProxy->GetGameThreadAPI();
		auto ModifiedCubeGeom = TRefCountPtr<FImplicitObject>(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		ModifiedCubeParticle.SetGeometry(ModifiedCubeGeom);
		Solver->RegisterObject(ModifiedCubeProxy);
		ModifiedCubeParticle.SetGravityEnabled(false);
		ModifiedCubeParticle.SetV(FVec3(0, 0, -100));
		ModifiedCubeParticle.SetX(FVec3(-200, 0, 500));
		SetCubeInertiaTensor(ModifiedCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ ModifiedCubeProxy->GetParticle_LowLevel() });

		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& FloorParticle = FloorProxy->GetGameThreadAPI();
		auto FloorGeom = TRefCountPtr<FImplicitObject>(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });

		// Save Unique indices of floor and modified cube to disable in contact mod.
		TVec2<FUniqueIdx> UniqueIndices({ ModifiedCubeParticle.UniqueIdx(), FloorParticle.UniqueIdx() });

		FContactModificationTestCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FContactModificationTestCallback>();
		Callback->TestLambda = [UniqueIndices, ModifiedCubeProxy](Chaos::FCollisionContactModifier& Modifier)
		{
			Chaos::FGeometryParticleHandle* ModifiedCubeParticle = ModifiedCubeProxy->GetHandle_LowLevel();
			for (FContactPairModifier& PairModifier : Modifier.GetContacts(ModifiedCubeParticle))
			{
				PairModifier.Disable();
			}
		};

		const float Dt = 1.0f;
		const int32 Steps = 10;
		for (int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		// Modified cube should be below floor because we disabled collision.
		EXPECT_LT(ModifiedCubeParticle.X().Z, FloorParticle.X().Z);

		// Colliding cube should be above floor due to collision.
		EXPECT_GT(CollidingCubeParticle.X().Z, FloorParticle.X().Z);

		// Floor should be at origin.
		EXPECT_EQ(FloorParticle.X().Z, 0);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(CollidingCubeProxy);
		Solver->UnregisterObject(ModifiedCubeProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

}
