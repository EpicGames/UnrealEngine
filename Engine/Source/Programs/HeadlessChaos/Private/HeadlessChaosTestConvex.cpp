// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestEPA.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/Core.h"
#include "Chaos/GJK.h"
#include "Chaos/Convex.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Particles.h"
#include "../Resource/TestGeometry2.h"
#include "Logging/LogScopedVerbosityOverride.h"

namespace ChaosTest
{
	using namespace Chaos;

	// Check that convex creation with face merging is working correctly.
	// The initial creation generates a set of triangles, and the merge step should
	// leave the hull with only one face per normal.
	void TestConvexBuilderConvexBoxFaceMerge(const TArray<FConvex::FVec3Type>& Vertices)
	{
		TArray<FConvex::FPlaneType> Planes;
		TArray<TArray<int32>> FaceVertices;
		TArray<Chaos::FConvex::FVec3Type> SurfaceParticles;
		FConvex::FAABB3Type LocalBounds;

		FConvexBuilder::Build(Vertices, Planes, FaceVertices, SurfaceParticles, LocalBounds);
		FConvexBuilder::MergeFaces(Planes, FaceVertices, SurfaceParticles, 1.0f);

		// Check that we have the right number of faces and particles
		EXPECT_EQ(SurfaceParticles.Num(), 8);
		EXPECT_EQ(Planes.Num(), 6);
		EXPECT_EQ(FaceVertices.Num(), 6);

		// Make sure the verts are correct and agree on the normal
		for (int32 FaceIndex = 0; FaceIndex < FaceVertices.Num(); ++FaceIndex)
		{
			EXPECT_EQ(FaceVertices[FaceIndex].Num(), 4);
			for (int32 VertexIndex0 = 0; VertexIndex0 < FaceVertices[FaceIndex].Num(); ++VertexIndex0)
			{
				int32 VertexIndex1 = Chaos::Utilities::WrapIndex(VertexIndex0 + 1, 0, FaceVertices[FaceIndex].Num());
				int32 VertexIndex2 = Chaos::Utilities::WrapIndex(VertexIndex0 + 2, 0, FaceVertices[FaceIndex].Num());
				const FVec3 Vertex0 = SurfaceParticles[FaceVertices[FaceIndex][VertexIndex0]];
				const FVec3 Vertex1 = SurfaceParticles[FaceVertices[FaceIndex][VertexIndex1]];
				const FVec3 Vertex2 = SurfaceParticles[FaceVertices[FaceIndex][VertexIndex2]];

				// All vertices should lie in a plane at the same distance
				const FReal Dist0 = FVec3::DotProduct(Vertex0, Planes[FaceIndex].Normal());
				const FReal Dist1 = FVec3::DotProduct(Vertex1, Planes[FaceIndex].Normal());
				const FReal Dist2 = FVec3::DotProduct(Vertex2, Planes[FaceIndex].Normal());
				EXPECT_NEAR(Dist0, 50.0f, 1.e-3f);
				EXPECT_NEAR(Dist1, 50.0f, 1.e-3f);
				EXPECT_NEAR(Dist2, 50.0f, 1.e-3f);

				// All sequential edge pairs should agree on winding
				const FReal Winding = FVec3::DotProduct(FVec3::CrossProduct(Vertex1 - Vertex0, Vertex2 - Vertex1), Planes[FaceIndex].Normal());
				EXPECT_GT(Winding, 0.0f);
			}
		}
	}

	// Check that face merging works for a convex box
	GTEST_TEST(ConvexStructureTests, TestConvexBoxFaceMerge)
	{
		const TArray<FConvex::FVec3Type> Vertices =
		{
			{-50,	-50,	-50},
			{-50,	-50,	50},
			{-50,	50,		-50},
			{-50,	50,		50},
			{50,	-50,	-50},
			{50,	-50,	50},
			{50,	50,		-50},
			{50,	50,		50},
		};

		TestConvexBuilderConvexBoxFaceMerge(Vertices);
	}

	// Check that the convex structure data is consistent (works for TBox and TConvex)
	template<typename T_GEOM> void TestConvexStructureDataImpl(const T_GEOM& Convex)
	{
		// Note: This tolerance matches the one passed to FConvexBuilder::MergeFaces in the FConvex constructor, but it should be dependent on size
		//const FReal Tolerance = 1.e-4f * Convex.BoundingBox().OriginRadius();
		const FReal Tolerance = 1.0f;

		// Check all per-plane data
		for (int32 PlaneIndex = 0; PlaneIndex < Convex.NumPlanes(); ++PlaneIndex)
		{
			// All vertices should be on the plane
			for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < Convex.NumPlaneVertices(PlaneIndex); ++PlaneVertexIndex)
			{
				const auto Plane = Convex.GetPlane(PlaneIndex);
				const int32 VertexIndex = Convex.GetPlaneVertex(PlaneIndex, PlaneVertexIndex);
				const FVec3 Vertex = Convex.GetVertex(VertexIndex);
				const FReal VertexDistance = FVec3::DotProduct(Plane.Normal(), Vertex - Plane.X());
				EXPECT_NEAR(VertexDistance, 0.0f, Tolerance);
			}
		}

		// Check all per-vertex data
		for (int32 VertexIndex = 0; VertexIndex < Convex.NumVertices(); ++VertexIndex)
		{
			// Get all the planes for the vertex
			TArray<int32> PlaneIndices;
			PlaneIndices.SetNum(128);
			int32 NumPlanes = Convex.FindVertexPlanes(VertexIndex, PlaneIndices.GetData(), PlaneIndices.Num());
			PlaneIndices.SetNum(NumPlanes);

			for (int32 PlaneIndex : PlaneIndices)
			{
				const auto Plane = Convex.GetPlane(PlaneIndex);
				const FVec3 Vertex = Convex.GetVertex(VertexIndex);
				const FReal VertexDistance = FVec3::DotProduct(Plane.Normal(), Vertex - Plane.X());
				EXPECT_NEAR(VertexDistance, 0.0f, Tolerance);
			}
		}
	}

	// Check that the convex structure data is consistent
	void TestConvexStructureData(const TArray<FConvex::FVec3Type>& Vertices)
	{
		FConvex Convex(Vertices, 0.0f);
		TestConvexStructureDataImpl(Convex);
	}

	// Check that the convex structure data is consistent for a simple convex box
	GTEST_TEST(ConvexStructureTests, TestConvexStructureData)
	{
		const TArray<FConvex::FVec3Type> Vertices =
		{
			{-50,		-50,	-50},
			{-50,		-50,	50},
			{-50,		50,		-50},
			{-50,		50,		50},
			{50,		-50,	-50},
			{50,		-50,	50},
			{50,		50,		-50},
			{50,		50,		50},
		};

		TestConvexStructureData(Vertices);
	}

	// Check that the convex structure data is consistent for a complex convex shape
	GTEST_TEST(ConvexStructureTests, TestConvexStructureData2)
	{
		const TArray<FConvex::FVec3Type> Vertices =
		{
			{0, 0, 12.0f},
			{-0.707f, -0.707f, 10.0f},
			{0, -1, 10.0f},
			{0.707f, -0.707f, 10.0f},
			{1, 0, 10.0f},
			{0.707f, 0.707f, 10.0f},
			{0.0f, 1.0f, 10.0f},
			{-0.707f, 0.707f, 10.0f},
			{-1.0f, 0.0f, 10.0f},
			{-0.707f, -0.707f, 0.0f},
			{0, -1, 0.0f},
			{0.707f, -0.707f, 0.0f},
			{1, 0, 0.0f},
			{0.707f, 0.707f, 0.0f},
			{0.0f, 1.0f, 0.0f},
			{-0.707f, 0.707f, 0.0f},
			{-1.0f, 0.0f, 0.0f},
			{0, 0, -2.0f},
		};

		TestConvexStructureData(Vertices);
	}

	// Check that the convex structure data is consistent for a standard box
	GTEST_TEST(ConvexStructureTests, TestBoxStructureData)
	{
		FImplicitBox3 Box(FVec3(-50, -50, -50), FVec3(50, 50, 50), 0.0f);

		TestConvexStructureDataImpl(Box);

		// Make sure all planes are at the correct distance
		for (int32 PlaneIndex = 0; PlaneIndex < Box.NumPlanes(); ++PlaneIndex)
		{
			// All vertices should be on the plane
			const TPlaneConcrete<FReal, 3> Plane = Box.GetPlane(PlaneIndex);
			EXPECT_NEAR(FVec3::DotProduct(Plane.X(), Plane.Normal()), 50.0f, KINDA_SMALL_NUMBER);
		}
	}

	// Check the reverse mapping planes->vertices->planes is intact
	template<typename T_STRUCTUREDATA>
	void TestConvexStructureDataMapping(const T_STRUCTUREDATA& StructureData)
	{
		// For each plane, get the list of vertices that make its edges.
		// Then check that the list of planes used by that vertex contains the original plane
		for (int32 PlaneIndex = 0; PlaneIndex < StructureData.NumPlanes(); ++PlaneIndex)
		{
			for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < StructureData.NumPlaneVertices(PlaneIndex); ++PlaneVertexIndex)
			{
				const int32 VertexIndex = StructureData.GetPlaneVertex(PlaneIndex, PlaneVertexIndex);

				// Check that the plane's vertex has the plane in its list
				TArray<int32> PlaneIndices;
				PlaneIndices.SetNum(128);
				const int32 NumPlanes = StructureData.FindVertexPlanes(VertexIndex, PlaneIndices.GetData(), PlaneIndices.Num());
				PlaneIndices.SetNum(NumPlanes);

				const bool bFoundPlane = PlaneIndices.Contains(PlaneIndex);
				EXPECT_TRUE(bFoundPlane);
			}
		}
	}

	// Check that the structure data is good for convex shapes that have faces merged during construction
	// This test uses the small index size in StructureData.
	GTEST_TEST(ConvexStructureTests, TestSmallIndexStructureData)
	{
		FMath::RandInit(53799058);
		const FReal Radius = 1000.0f;

		const int32 NumVertices = TestGeometry2::RawVertexArray.Num() / 3;
		TArray<FConvex::FVec3Type> Particles;
		Particles.SetNum(NumVertices);
		for (int32 ParticleIndex = 0; ParticleIndex < NumVertices; ++ParticleIndex)
		{
			Particles[ParticleIndex] = FConvex::FVec3Type(
				TestGeometry2::RawVertexArray[3 * ParticleIndex + 0],
				TestGeometry2::RawVertexArray[3 * ParticleIndex + 1],
				TestGeometry2::RawVertexArray[3 * ParticleIndex + 2]
			);
		}

		FConvex Convex(Particles, 0.0f);

		const FConvexStructureData::FConvexStructureDataMedium& StructureData = Convex.GetStructureData().DataM();
		TestConvexStructureDataMapping(StructureData);
		TestConvexStructureDataImpl(Convex);
	}


	// Check that the structure data is good for convex shapes that have faces merged during construction
	// This test uses the large index size in StructureData.
	// This test is disabled - the convex building is too slow for this many verts
	GTEST_TEST(ConvexStructureTests, DISABLED_TestLargeIndexStructureData2)
	{
		FMath::RandInit(53799058);
		const FReal Radius = 10000.0f;
		const int32 NumVertices = 50000;

		// Make a convex with points on a sphere.
		TArray<FConvex::FVec3Type> Vertices;
		Vertices.SetNum(NumVertices);
		for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			const FConvex::FRealType Theta = FMath::RandRange(-PI, PI);
			const FConvex::FRealType Phi = FMath::RandRange(-0.5f * PI, 0.5f * PI);
			Vertices[VertexIndex] = Radius * FConvex::FVec3Type(FMath::Cos(Theta), FMath::Sin(Theta), FMath::Sin(Phi));
		}
		FConvex Convex(Vertices, 0.0f);

		EXPECT_GT(Convex.NumVertices(), 800);
		EXPECT_GT(Convex.NumPlanes(), 500);

		const FConvexStructureData::FConvexStructureDataLarge& StructureData = Convex.GetStructureData().DataL();
		TestConvexStructureDataMapping(StructureData);
		TestConvexStructureDataImpl(Convex);
	}

	// Check that extremely small generated triangle don't trigger the normal check
	GTEST_TEST(ConvexStructureTests, TestConvexFaceNormalCheck)
	{
		// Create a long mesh with a extremely small end (YZ plane) 
		// so that it generate extremely sized triangle that will produce extremely small (unormalized) normals
		const float SmallNumber = 0.001f;
		const FConvex::FVec3Type Range{ 100.0f, SmallNumber, SmallNumber };

		const TArray<FConvex::FVec3Type> Vertices =
		{
			{0, 0, 0},
			{Range.X, 0, 0},
			{Range.X, Range.Y, 0},
			{Range.X, Range.Y, Range.Z},
			{Range.X + SmallNumber, Range.Y * 0.5f, Range.Z * 0.5f},
		};

		TestConvexStructureData(Vertices);
	}

	GTEST_TEST(ConvexStructureTests, TestConvexFailsSafelyOnPlanarObject)
	{
		using namespace Chaos;

		// This list of vertices is a plane with many duplicated vertices and previously was causing
		// a check to fire inside the convex builder as we classified the object incorrectly and didn't
		// safely handle a failure due to a planar object. This test verifies that the builder can
		// safely fail to build a convex from a plane.
		const TArray<FConvex::FVec3Type> Vertices =
		{
			{-15.1425571f, 16.9698563f, 0.502334476f},
			{-15.1425571f, 16.9698563f, 0.502334476f},
			{-15.1425571f, 16.9698563f, 0.502334476f},
			{-16.9772491f, -15.1373663f, -0.398189038f},
			{-15.1425571f, 16.9698563f, 0.502334476f},
			{16.9772491f, 15.1373663f, 0.398189038f},
			{16.9772491f, 15.1373663f, 0.398189038f},
			{16.9772491f, 15.1373663f, 0.398189038f},
			{-15.1425571f, 16.9698563f, 0.502334476f},
			{-16.9772491f, -15.1373663f, -0.398189038f},
			{-16.9772491f, -15.1373663f, -0.398189038f},
			{15.1425571f, -16.9698563f, -0.502334476f},
			{-16.9772491f, -15.1373663f, -0.398189038f},
			{-16.9772491f, -15.1373663f, -0.398189038f},
			{16.9772491f, 15.1373663f, 0.398189038f},
			{15.1425571f, -16.9698563f, -0.502334476f},
			{-15.1425571f, 16.9698563f, 0.502334476f},
			{16.9772491f, 15.1373663f, 0.398189038f},
			{15.1425571f, -16.9698563f, -0.502334476f},
			{15.1425571f, -16.9698563f, -0.502334476f},
			{16.9772491f, 15.1373663f, 0.398189038f},
			{15.1425571f, -16.9698563f, -0.502334476f},
			{-16.9772491f, -15.1373663f, -0.398189038f},
			{15.1425571f, -16.9698563f, -0.502334476f},
			{-15.1425571f, 16.9698563f, 0.502334476f},
			{-15.1425571f, 16.9698563f, 0.502334476f},
			{-15.1425571f, 16.9698563f, 0.502334476f},
			{-16.9772491f, -15.1373663f, -0.398189038f},
			{-15.1425571f, 16.9698563f, 0.502334476f},
			{16.9772491f, 15.1373663f, 0.398189038f},
			{16.9772491f, 15.1373663f, 0.398189038f},
			{16.9772491f, 15.1373663f, 0.398189038f},
			{-15.1425571f, 16.9698563f, 0.502334476f},
			{-16.9772491f, -15.1373663f, -0.398189038f},
			{-16.9772491f, -15.1373663f, -0.398189038f},
			{15.1425571f, -16.9698563f, -0.502334476f},
			{-16.9772491f, -15.1373663f, -0.398189038f},
			{-16.9772491f, -15.1373663f, -0.398189038f},
			{16.9772491f, 15.1373663f, 0.398189038f},
			{15.1425571f, -16.9698563f, -0.502334476f},
			{-15.1425571f, 16.9698563f, 0.502334476f},
			{16.9772491f, 15.1373663f, 0.398189038f},
			{15.1425571f, -16.9698563f, -0.502334476f},
			{15.1425571f, -16.9698563f, -0.502334476f},
			{16.9772491f, 15.1373663f, 0.398189038f},
			{15.1425571f, -16.9698563f, -0.502334476f},
			{-16.9772491f, -15.1373663f, -0.398189038f},
			{15.1425571f, -16.9698563f, -0.502334476f}
		};

		TArray<FConvex::FPlaneType> Planes;
		TArray<TArray<int32>> FaceIndices;
		TArray<FConvex::FVec3Type> FinalVertices;
		FConvex::FAABB3Type LocalBounds;

		{
			// Temporarily set LogChaos to error, we're expecting this to fire warnings and don't want that to fail a CIS run.
			LOG_SCOPE_VERBOSITY_OVERRIDE(LogChaos, ELogVerbosity::Error);
			FConvexBuilder::Build(Vertices, Planes, FaceIndices, FinalVertices, LocalBounds);
		}

		// Check that we've failed to build a 3D convex hull and safely returned
		EXPECT_EQ(Planes.Num(), 0);
	}

	GTEST_TEST(ConvexStructureTests, TestConvexHalfEdgeStructureData_Box)
	{
		const TArray<FConvex::FVec3Type> InputVertices =
		{
			FVec3(-50,		-50,	-50),
			FVec3(-50,		-50,	50),
			FVec3(-50,		50,		-50),
			FVec3(-50,		50,		50),
			FVec3(50,		-50,	-50),
			FVec3(50,		-50,	50),
			FVec3(50,		50,		-50),
			FVec3(50,		50,		50),
		};

		TArray<FConvex::FPlaneType> Planes;
		TArray<TArray<int32>> FaceVertices;
		TArray<FConvex::FVec3Type> Vertices;
		FConvex::FAABB3Type LocalBounds;
		FConvexBuilder::Build(InputVertices, Planes, FaceVertices, Vertices, LocalBounds);
		FConvexBuilder::MergeFaces(Planes, FaceVertices, Vertices, 1.0f);

		FConvex Convex(Vertices, 0.0f);

		const FConvexStructureData::FConvexStructureDataSmall& StructureData = Convex.GetStructureData().DataS();

		EXPECT_EQ(StructureData.NumPlanes(), 6);
		EXPECT_EQ(StructureData.NumHalfEdges(), 24);
		EXPECT_EQ(StructureData.NumVertices(), 8);

		// Count how many times each vertex and edge is referenced
		TArray<int32> VertexIndexCount;
		TArray<int32> EdgeIndexCount;
		VertexIndexCount.SetNumZeroed(StructureData.NumVertices());
		EdgeIndexCount.SetNumZeroed(StructureData.NumHalfEdges());
		for (int32 PlaneIndex = 0; PlaneIndex < StructureData.NumPlanes(); ++PlaneIndex)
		{
			EXPECT_EQ(StructureData.NumPlaneHalfEdges(PlaneIndex), 4);
			for (int32 PlaneEdgeIndex = 0; PlaneEdgeIndex < StructureData.NumPlaneHalfEdges(PlaneIndex); ++PlaneEdgeIndex)
			{
				const int32 EdgeIndex = StructureData.GetPlaneHalfEdge(PlaneIndex, PlaneEdgeIndex);
				const int32 VertexIndex = StructureData.GetHalfEdgeVertex(EdgeIndex);
				EdgeIndexCount[EdgeIndex]++;
				VertexIndexCount[VertexIndex]++;
			}
		}

		// Every vertex is used by 3 half-edges (and planes)
		for (int32 VertexCount : VertexIndexCount)
		{
			EXPECT_EQ(VertexCount, 3);
		}

		// Each half edge is used by a single plane
		for (int32 EdgeCount : EdgeIndexCount)
		{
			EXPECT_EQ(EdgeCount, 1);
		}

		// Vertex Plane iterator generates 3 planes and all the edges have the same primary vertex
		for (int32 VertexIndex = 0; VertexIndex < StructureData.NumVertices(); ++VertexIndex)
		{
			int32 PlaneCount = 0;

			TArray<int32> VertexPlanes;
			VertexPlanes.SetNum(128);
			const int32 NumPlanes = StructureData.FindVertexPlanes(VertexIndex, VertexPlanes.GetData(), VertexPlanes.Num());
			VertexPlanes.SetNum(NumPlanes);

			for (int32 PlaneIndex : VertexPlanes)
			{
				EXPECT_NE(PlaneIndex, INDEX_NONE);

				++PlaneCount;
			}

			// Everty vertex belongs to 3 planes
			EXPECT_EQ(PlaneCount, 3);

			// Every vertex's first edge should have that vertex as its root vertex
			const int32 VertexHalfEdgeIndex = StructureData.GetVertexFirstHalfEdge(VertexIndex);
			EXPECT_EQ(VertexIndex, StructureData.GetHalfEdgeVertex(VertexHalfEdgeIndex));
		}
	}

	template<typename ConvexType>
	void TestConvexPlaneVertices(const ConvexType& Convex)
	{
		const FReal NormalTolerance = UE_SMALL_NUMBER;
		const FReal PositionTolerance = UE_KINDA_SMALL_NUMBER;

		for (int32 PlaneIndex = 0; PlaneIndex < Convex.NumPlanes(); ++PlaneIndex)
		{
			const FVec3 PlaneN = Convex.GetPlane(PlaneIndex).Normal();
			const FVec3 PlaneX = Convex.GetPlane(PlaneIndex).X();

			const int NumPlaneVertices = Convex.NumPlaneVertices(PlaneIndex);
			for (int32 PlaneVertexIndex0 = 0; PlaneVertexIndex0 < NumPlaneVertices; ++PlaneVertexIndex0)
			{
				const int32 VertexIndex0 = Convex.GetPlaneVertex(PlaneIndex, PlaneVertexIndex0);

				// All vertices are actually on the plane
				const FVec3 Vertex0 = Convex.GetVertex(VertexIndex0);
				EXPECT_NEAR(FVec3::DotProduct(Vertex0, PlaneN), FVec3::DotProduct(PlaneX, PlaneN), PositionTolerance) << "PlaneIndex=" << PlaneIndex << " PlaneVertexIndex0=" << PlaneVertexIndex0;

				// Winding is correct
				int PlaneVertexIndex1 = (PlaneVertexIndex0 < NumPlaneVertices - 1) ? PlaneVertexIndex0 + 1 : 0;
				int PlaneVertexIndex2 = (PlaneVertexIndex0 < NumPlaneVertices - 2) ? PlaneVertexIndex0 + 2 : PlaneVertexIndex0 - NumPlaneVertices + 2;
				const int32 VertexIndex1 = Convex.GetPlaneVertex(PlaneIndex, PlaneVertexIndex1);
				const int32 VertexIndex2 = Convex.GetPlaneVertex(PlaneIndex, PlaneVertexIndex2);
				const FVec3 Vertex1 = Convex.GetVertex(VertexIndex1);
				const FVec3 Vertex2 = Convex.GetVertex(VertexIndex2);

				const FReal WindingMag = FVec3::DotProduct(FVec3::CrossProduct(Vertex1 - Vertex0, Vertex2 - Vertex1), PlaneN);
				const FReal Winding = FMath::Sign(WindingMag);
				const int32 ExpectedWinding = Convex.GetWindingOrder();
				EXPECT_EQ(Winding, ExpectedWinding) << "PlaneIndex=" << PlaneIndex << " PlaneVertexIndex0=" << PlaneVertexIndex0;
			}
		}
	}

	template<typename ConvexType>
	void TestConvexEdges(const ConvexType& Convex)
	{
		// Check the edges
		for (int32 EdgeIndex = 0; EdgeIndex < Convex.NumEdges(); ++EdgeIndex)
		{
			const int PlaneIndex0 = Convex.GetEdgePlane(EdgeIndex, 0);
			const int PlaneIndex1 = Convex.GetEdgePlane(EdgeIndex, 1);
			const int32 VertexIndex0 = Convex.GetEdgeVertex(EdgeIndex, 0);
			const int32 VertexIndex1 = Convex.GetEdgeVertex(EdgeIndex, 0);

			// Plane0 contains the two vertices
			bool bFoundVertex0 = false;
			bool bFoundVertex1 = false;
			for (int32 PlaneVertexIndex0 = 0; PlaneVertexIndex0 < Convex.NumPlaneVertices(PlaneIndex0); ++PlaneVertexIndex0)
			{
				const int32 ThisVertexIndex = Convex.GetPlaneVertex(PlaneIndex0, PlaneVertexIndex0);
				if (ThisVertexIndex == VertexIndex0)
				{
					bFoundVertex0 = true;
				}
				if (ThisVertexIndex == VertexIndex1)
				{
					bFoundVertex1 = true;
				}
			}
			EXPECT_TRUE(bFoundVertex0) << "EdgeIndex=" << EdgeIndex << "PlaneIndex=" << PlaneIndex0 << " VertexIndex=" << VertexIndex0;
			EXPECT_TRUE(bFoundVertex1) << "EdgeIndex=" << EdgeIndex << "PlaneIndex=" << PlaneIndex0 << " VertexIndex=" << VertexIndex1;

			// Plane1 contains the two vertices
			bFoundVertex0 = false;
			bFoundVertex1 = false;
			for (int32 PlaneVertexIndex1 = 0; PlaneVertexIndex1 < Convex.NumPlaneVertices(PlaneIndex1); ++PlaneVertexIndex1)
			{
				const int32 ThisVertexIndex = Convex.GetPlaneVertex(PlaneIndex1, PlaneVertexIndex1);
				if (ThisVertexIndex == VertexIndex0)
				{
					bFoundVertex0 = true;
				}
				if (ThisVertexIndex == VertexIndex1)
				{
					bFoundVertex1 = true;
				}
			}
			EXPECT_TRUE(bFoundVertex0) << "EdgeIndex=" << EdgeIndex << "PlaneIndex=" << PlaneIndex1 << " VertexIndex=" << VertexIndex0;
			EXPECT_TRUE(bFoundVertex1) << "EdgeIndex=" << EdgeIndex << "PlaneIndex=" << PlaneIndex1 << " VertexIndex=" << VertexIndex1;
		}
	}

	// Verify that the box Plane Edge and Vertex APIs return the elements exactly as they are defined in Box.cpp
	GTEST_TEST(ConvexStructureTests, TestBoxStructureDataDetails)
	{
		// These arrays are copied from Box.cpp - any changes there should trigger a failure here
		// so we can be sure the change was expected.
		const TArray<FVec3> PlaneNormals =
		{
			FVec3(-1,  0,  0),		// -X
			FVec3(0, -1,  0),		// -Y
			FVec3(0,  0, -1),		// -Z
			FVec3(1,  0,  0),		//  X
			FVec3(0,  1,  0),		//  Y
			FVec3(0,  0,  1),		//  Z
		};

		const TArray<FVec3> UnitVertices =
		{
			FVec3(-1, -1, -1),		// 0 
			FVec3(1, -1, -1),		// 1 
			FVec3(-1,  1, -1),		// 2 
			FVec3(1,  1, -1),		// 3 
			FVec3(-1, -1,  1),		// 4 
			FVec3(1, -1,  1),		// 5 
			FVec3(-1,  1,  1),		// 6 
			FVec3(1,  1,  1),		// 7 
		};

		TArray<TArray<int32>> PlaneVertices
		{
			{ 0, 4, 6, 2 },			// -X,
			{ 0, 1, 5, 4 },			// -Y
			{ 0, 2, 3, 1 },			// -Z
			{ 1, 3, 7, 5 },			//  X
			{ 2, 6, 7, 3 },			//  Y
			{ 4, 5, 7, 6 },			//  Z
		};

		const FReal NormalTolerance = UE_SMALL_NUMBER;
		const FReal PositionTolerance = UE_KINDA_SMALL_NUMBER;

		const FVec3 Center = FVec3(0, 0, 0);
		const FVec3 HalfExtent = FVec3(100, 200, 300);
		const FReal Margin = FReal(0);

		const FImplicitBox3 Box = FImplicitBox3(Center - HalfExtent, Center + HalfExtent, Margin);

		EXPECT_EQ(Box.NumPlanes(), 6);
		EXPECT_EQ(Box.NumEdges(), 12);
		EXPECT_EQ(Box.NumVertices(), 8);

		// Check that the vertices are in the expected order
		for (int32 VertexIndex = 0; VertexIndex < UnitVertices.Num(); ++VertexIndex)
		{
			const FVec3 Vertex = Box.GetVertex(VertexIndex);
			const FVec3 ExpectedVertex = UnitVertices[VertexIndex] * HalfExtent;
			EXPECT_NEAR(Vertex.X, ExpectedVertex.X, PositionTolerance);
			EXPECT_NEAR(Vertex.Y, ExpectedVertex.Y, PositionTolerance);
			EXPECT_NEAR(Vertex.Z, ExpectedVertex.Z, PositionTolerance);
		}

		// Check that the planes have the correct normal and position
		for (int32 PlaneIndex = 0; PlaneIndex < PlaneNormals.Num(); ++PlaneIndex)
		{
			TPlaneConcrete<FReal> Plane = Box.GetPlane(PlaneIndex);

			// Normals are in the expected direction
			EXPECT_NEAR(Plane.Normal().X, PlaneNormals[PlaneIndex].X, NormalTolerance) << "PlaneIndex=" << PlaneIndex;
			EXPECT_NEAR(Plane.Normal().Y, PlaneNormals[PlaneIndex].Y, NormalTolerance) << "PlaneIndex=" << PlaneIndex;
			EXPECT_NEAR(Plane.Normal().Z, PlaneNormals[PlaneIndex].Z, NormalTolerance) << "PlaneIndex=" << PlaneIndex;

			// Positions are in the correct plane
			const FReal PlaneDistance = FVec3::DotProduct(Plane.Normal(), Plane.X());
			const FReal ExpectedPlaneDistance = FVec3::DotProduct(PlaneNormals[PlaneIndex], PlaneNormals[PlaneIndex] * HalfExtent);
			EXPECT_NEAR(PlaneDistance, ExpectedPlaneDistance, PositionTolerance);
		}

		// Check that the planes have the correct vertices
		for (int32 PlaneIndex = 0; PlaneIndex < PlaneNormals.Num(); ++PlaneIndex)
		{
			const int NumPlaneVertices = Box.NumPlaneVertices(PlaneIndex);
			EXPECT_EQ(NumPlaneVertices, PlaneVertices[PlaneIndex].Num()) << "PlaneIndex=" << PlaneIndex;	// Always 4

			for (int32 PlaneVertexIndex0 = 0; PlaneVertexIndex0 < NumPlaneVertices; ++PlaneVertexIndex0)
			{
				const int32 VertexIndex0 = Box.GetPlaneVertex(PlaneIndex, PlaneVertexIndex0);
				EXPECT_EQ(VertexIndex0, PlaneVertices[PlaneIndex][PlaneVertexIndex0]) << "PlaneIndex=" << PlaneIndex << " PlaneVertexIndex0=" << PlaneVertexIndex0;
			}
		}

		// Check the plane vertices are in the plane and have the correct winding order 
		TestConvexPlaneVertices(Box);

		// Check that the edges report planes that actually share vertices
		TestConvexEdges(Box);
	}

	// Check that a Box implemented as a FImplicitConvex3 meets the same specs as ImplicitBox3
	GTEST_TEST(ConvexStructureTests, TestConvexBoxStructureDataDetails)
	{
		const FVec3f Center = FVec3(0, 0, 0);
		const FVec3f HalfExtent = FVec3(100, 200, 300);
		const FRealSingle Margin = FReal(0);

		const TArray<FVec3f> Vertices =
		{
			Center + HalfExtent * FVec3f(-1, -1, -1),		// 0 
			Center + HalfExtent * FVec3f( 1, -1, -1),		// 1 
			Center + HalfExtent * FVec3f(-1,  1, -1),		// 2 
			Center + HalfExtent * FVec3f( 1,  1, -1),		// 3 
			Center + HalfExtent * FVec3f(-1, -1,  1),		// 4 
			Center + HalfExtent * FVec3f( 1, -1,  1),		// 5 
			Center + HalfExtent * FVec3f(-1,  1,  1),		// 6 
			Center + HalfExtent * FVec3f( 1,  1,  1),		// 7 
		};

		FImplicitConvex3 Convex = FImplicitConvex3(Vertices, Margin);

		// Check the plane vertices are in the plane and have the correct winding order 
		TestConvexPlaneVertices(Convex);

		// Check that the edges report planes that actually share vertices
		TestConvexEdges(Convex);
	}

	// The set of vertices generated from a unit box when creating a GeometryCollection from the default box in the editor.
	// The default cube is tesselated. It has 26 vertices which include the 8 corners, plus mid-points along each edge and in the middle of each face.
	// 
	// This was causing the convex builder to produce a denegerate triangle (3 points in a row) leading to a zero normal and a crash in the solver.
	//
	// The fix was to modify TConvexHull3 to produce convex faces rather than triangles (one of which could be nearly degenerate), 
	// and a post process on its results to eliminate colinear edges (within some tolerance)
	//
	GTEST_TEST(ConvexBuilderTests, TestDefaultStaticMeshBox)
	{
		FConvexBuilder::EBuildMethod BuildMethod = FConvexBuilder::EBuildMethod::Default;
		const FReal Margin = 9.9999997473787516e-05;
		TArray<FVec3f> BoxVerts =
		{
			{-50.0000000f, 50.0000000f, -50.0000000f},
			{50.0000000f, 50.0000000f, -50.0000000f},
			{50.0000000f, -50.0000000f, -50.0000000f},
			{50.0000000f, -50.0000000f, 50.0000000f},
			{50.0000000f, 50.0000000f, 50.0000000f},
			{-50.0000000f, -50.0000000f, 50.0000000f},
			{-50.0000000f, 50.0000000f, 50.0000000f},
			{-50.0000000f, -50.0000000f, -50.0000000f},
			{0.00000000f, 50.0000000f, -50.0000000f},
			{50.0000000f, 0.00000000f, -50.0000000f},
			{0.00000000f, 50.0000000f, 50.0000000f},
			{-50.0000000f, -50.0000000f, 3.06161689e-15f},
			{-50.0000000f, 50.0000000f, -3.06161689e-15f},
			{-50.0000000f, 0.00000000f, -50.0000000f},
			{50.0000000f, -50.0000000f, 3.06161689e-15f},
			{0.00000000f, -50.0000000f, -50.0000000f},
			{50.0000000f, 1.22464676e-14f, 50.0000000f},
			{0.00000000f, -50.0000000f, 50.0000000f},
			{-50.0000000f, 1.22464676e-14f, 50.0000000f},
			{50.0000000f, 50.0000000f, -3.06161689e-15f},
			{0.00000000f, 50.0000000f, -3.06161689e-15f},
			{0.00000000f, 0.00000000f, -50.0000000f},
			{0.00000000f, 1.22464676e-14f, 50.0000000f},
			{0.00000000f, -50.0000000f, 3.06161689e-15f},
			{50.0000000f, 6.12323379e-15f, -1.87469967e-31f},
			{-50.0000000f, 6.12323379e-15f, -1.87469967e-31f},
		};

		FImplicitConvex3 Convex(BoxVerts, Margin, BuildMethod);

		// The convex should be a box
		EXPECT_EQ(Convex.NumVertices(), 8);
		EXPECT_EQ(Convex.NumEdges(), 12);
		EXPECT_EQ(Convex.NumPlanes(), 6);

		// All planes normals should be...normalized
		const FReal NormalTolerance = 1.e-4;
		for (int32 PlaneIndex = 0; PlaneIndex < Convex.NumPlanes(); ++PlaneIndex)
		{
			const FVec3 PlaneN = Convex.GetPlane(PlaneIndex).Normal();
			EXPECT_NEAR(PlaneN.Size(), FReal(1), NormalTolerance);
		}

	}

	// Create a tet with an extra degenerate triangle in it. Verify that MergeColinearEdges handles this case
	// and does not leave an invalid 2-vertex face behind.
	// NOTE: We should not be able to create a FImplicitConvex3 that calls MergeColinearEdges in this condition
	// but better safe than sorry.
	GTEST_TEST(ConvexBuilderTests, TestColinearEdgeInTriangle)
	{
		// A right angled tet with an extra degenerate triangular face in there
		TArray<FVec3f> TetVerts =
		{
			{0.0000000f, 0.0000000f, 50.0000000f},		// Top
			{0.0000000f, 0.0000000f, 0.0000000f},		// Base0
			{50.0000000f, 0.0000000f, 0.0000000f},		// Base1
			{0.0000000f, 50.0000000f, 0.0000000f},		// Base2
			{-1.e-15f, -1.e-14f, 25.0000000f},			// Extra vert along the vertical edge
		};
		TArray<TArray<int32>> TetFaces = 
		{
			{ 1, 2, 3 },								// Base
			{ 0, 2, 1 },								// Side0
			{ 0, 3, 2 },								// Side1
			{ 0, 1, 3 },								// Side2
			{ 0, 1, 4 },								// Extra degenerate face
		};
		TArray<TPlaneConcrete<FRealSingle>> TetPlanes =
		{
			// Values don't matter for this test
			TPlaneConcrete<FRealSingle>(FVec3(0), FVec3(0,0,1)),
			TPlaneConcrete<FRealSingle>(FVec3(0), FVec3(0,0,1)),
			TPlaneConcrete<FRealSingle>(FVec3(0), FVec3(0,0,1)),
			TPlaneConcrete<FRealSingle>(FVec3(0), FVec3(0,0,1)),
			TPlaneConcrete<FRealSingle>(FVec3(0), FVec3(0,0,1)),
		};

		const FRealSingle AngleTolerance = 1.e-6f;
		FConvexBuilder::MergeColinearEdges(TetPlanes, TetFaces, TetVerts, AngleTolerance);

		// The invalid face and its vertex should have been stripped
		EXPECT_EQ(TetVerts.Num(), 4);
		EXPECT_EQ(TetFaces.Num(), 4);
		EXPECT_EQ(TetPlanes.Num(), 4);
	}
}
