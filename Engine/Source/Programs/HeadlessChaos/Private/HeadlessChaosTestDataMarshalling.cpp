// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestUtility.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ErrorReporter.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Chaos/Utilities.h"
#include "PBDRigidsSolver.h"
#include "ChaosSolversModule.h"

#include "Modules/ModuleManager.h"
#include "Chaos/ChaosMarshallingManager.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Chaos/PullPhysicsDataImp.h"
#include "Chaos/Framework/ChaosResultsManager.h"

namespace ChaosTest
{

    using namespace Chaos;

	GTEST_TEST(DataMarshalling,Marshalling)
	{
		FChaosMarshallingManager Manager;

		float ExternalDt = 1/30.f;
		float InternalDt = ExternalDt;

		TSet<FPushPhysicsData*> BuffersSeen;
		for(int Step = 0; Step < 10; ++Step)
		{
			//Internal and external dt match so every internal step should get 1 data, the one we wrote to
			const auto DataWritten = Manager.GetProducerData_External();
			Manager.Step_External(ExternalDt);
			FPushPhysicsData* PushData = Manager.StepInternalTime_External();
			EXPECT_EQ(PushData,DataWritten);
			
			BuffersSeen.Add(DataWritten);
			EXPECT_EQ(BuffersSeen.Num(),Step == 0 ? 1 : 2);	//we should only ever use two buffers when dts match because we just keep cycling back and forth

			Manager.FreeData_Internal(PushData);

			EXPECT_EQ(Manager.StepInternalTime_External(), nullptr);	//no more data
		}

		BuffersSeen.Empty();
		InternalDt = ExternalDt * 0.5f;
		//tick internal dt twice as fast, should only get data every other step
#if 0
		//sub-stepping not supported yet
		//TODO: fix this
		for(int Step = 0; Step < 10; ++Step)
		{
			const auto DataWritten = Manager.GetProducerData_External();
			Manager.Step_External(ExternalDt);
			for(int InternalStep = 0; InternalStep < 2; ++InternalStep)
			{
				const auto PushData = Manager.StepInternalTime_External(InternalDt);
				if(InternalStep == 0)
				{
					EXPECT_EQ(PushData.Num(),1);
					EXPECT_EQ(DataWritten,PushData[0]);
					Manager.FreeData_Internal(PushData[0]);

					BuffersSeen.Add(DataWritten);
					EXPECT_EQ(BuffersSeen.Num(),Step == 0 ? 1 : 2);	//we should only ever use two buffers when dts match because we just keep cycling back and forth
				}
				else
				{
					EXPECT_EQ(PushData.Num(),0);
				}
			}
		}
#endif

		BuffersSeen.Empty();
		InternalDt = ExternalDt * 2;
		//tick internal dt for double the interval, should get two push datas per internal tick
		for(int Step = 0; Step < 10; ++Step)
		{
			const auto DataWritten1 = Manager.GetProducerData_External();
			Manager.Step_External(ExternalDt);

			const auto DataWritten2 = Manager.GetProducerData_External();
			Manager.Step_External(ExternalDt);

			for(int32 InternalStep = 0; InternalStep < 2; ++InternalStep)
			{
				FPushPhysicsData* PushData = Manager.StepInternalTime_External();
				EXPECT_EQ(PushData, InternalStep == 0 ? DataWritten1 : DataWritten2);
				BuffersSeen.Add(PushData);
				Manager.FreeData_Internal(PushData);
			}

			EXPECT_EQ(Manager.StepInternalTime_External(), nullptr);	//no more data
			
			EXPECT_EQ(BuffersSeen.Num(),Step == 0 ? 2 : 3);	//we should only ever use three buffers
		}
	}

	GTEST_TEST(AllTraits, DataMarshalling_Callbacks)
	{
		auto* Solver = FChaosSolversModule::GetModule()->CreateSolver(nullptr, /*AsyncDt=*/-1, EThreadingMode::SingleThread);
		
		int Count = 0;
		FReal Time = 0;
		const FReal Dt = 1 / 30.;

		struct FDummyInt : public FSimCallbackInput
		{
			void Reset() {}
			int32 Data;
		};

		struct FDummyOut : public FSimCallbackOutput
		{
			void Reset() {}
			int32 Data;
		};

		struct FCallback : public TSimCallbackObject<FDummyInt, FDummyOut>
		{
			virtual void OnPreSimulate_Internal() override
			{
				EXPECT_EQ((FReal)1 / (FReal)30., GetDeltaTime_Internal());
				EXPECT_EQ(GetConsumerInput_Internal()->Data, *CountPtr);
				GetProducerOutputData_Internal().Data = *CountPtr;
				++(*CountPtr);
			}

			int32* CountPtr;
			FReal* Time;
		};

		FCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FCallback>();
		Callback->CountPtr = &Count;
		Callback->Time = &Time;

		for(int Step = 0; Step < 10; ++Step)
		{
			Callback->GetProducerInputData_External()->Data = Step;
			
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
			TSimCallbackOutputHandle<FDummyOut> Output = Callback->PopOutputData_External();
			EXPECT_EQ(Output->Data, Step);	//output matched the step we ran
			EXPECT_EQ(Output->InternalTime, Time);
			TSimCallbackOutputHandle<FDummyOut> Output2 = Callback->PopOutputData_External();	//should get no output because already consumed data
			EXPECT_FALSE(Output2);
			Time += Dt;
		}
		
		EXPECT_EQ(Count,10);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);

		for(int Step = 0; Step < 10; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
			Time += Dt;
		}

		EXPECT_EQ(Count,10);
	}

	GTEST_TEST(AllTraits,DataMarshalling_OneShotCallbacks)
	{
		auto* Solver = FChaosSolversModule::GetModule()->CreateSolver(nullptr, /*AsyncDt=*/-1, EThreadingMode::SingleThread);
		
		int Count = 0;
		Solver->RegisterSimOneShotCallback([&Count]()
		{
			EXPECT_EQ(Count,0);
			++Count;
		});

		for(int Step = 0; Step < 10; ++Step)
		{
			Solver->RegisterSimOneShotCallback([Step, &Count]()
			{
				EXPECT_EQ(Count,Step+1);	//at step plus first one we registered
				++Count;
			});

			Solver->AdvanceAndDispatch_External(1/30.f);
			Solver->UpdateGameThreadStructures();
		}

		EXPECT_EQ(Count,11);

	}

	GTEST_TEST(DataMarshalling, InterpolatedPullData)
	{
		{
			FChaosMarshallingManager MarshallingManager;
			FChaosResultsManager ResultsManager(MarshallingManager);

			const FReal ExternalDt = 1 / 30.;
			FReal ExternalTime = 0;

			for (int Step = 0; Step < 10; ++Step)
			{
				const FReal StartTime = ExternalTime;	//external time we would have kicked the sim task off with
				ExternalTime += ExternalDt;
				MarshallingManager.FinalizePullData_Internal(Step, StartTime, ExternalDt);
				//in sync mode the external time we pass in doesn't matter
				const FChaosInterpolationResults& Results = ResultsManager.PullSyncPhysicsResults_External();
				EXPECT_EQ(Results.Next->ExternalStartTime, StartTime);
			}
		}
		{
			FChaosMarshallingManager MarshallingManager;
			FChaosResultsManager ResultsManager(MarshallingManager);

			const FReal ExternalDt = 1 / 30.;
			FReal ExternalTime = 0;

			for (int Step = 0; Step < 10; ++Step)
			{
				const FReal StartTime = ExternalTime;	//external time we would have kicked the sim task off with
				ExternalTime += ExternalDt;
				MarshallingManager.FinalizePullData_Internal(Step, StartTime, ExternalDt);
				TArray<const FChaosInterpolationResults*> Results = ResultsManager.PullAsyncPhysicsResults_External(ExternalTime);
				EXPECT_EQ(Results[0]->Alpha, 1);	//async mode but no buffer so no interpolation
				EXPECT_EQ(Results[0]->Next->ExternalStartTime, StartTime);	//async mode but no buffer so should appear the same as sync
			}
		}

		{
			FChaosMarshallingManager MarshallingManager;
			FChaosResultsManager ResultsManager(MarshallingManager);

			FReal PreTime;
			const FReal ExternalDt = 1 / 30.;
			FReal ExternalTime = 0;
			const FReal Delay = ExternalDt * 2 + 1e-2;

			for (int Step = 0; Step < 10; ++Step)
			{
				const FReal StartTime = ExternalTime;	//external time we would have kicked the sim task off with
				ExternalTime += ExternalDt;
				const FReal RenderTime = ExternalTime - Delay;
				MarshallingManager.FinalizePullData_Internal(Step, StartTime, ExternalDt);
				TArray<const FChaosInterpolationResults*> Results = ResultsManager.PullAsyncPhysicsResults_External(RenderTime);
				if(RenderTime < 0)
				{
					EXPECT_LT(Step, 2);	//first two frames treat as sync mode since we don't have enough delay
					EXPECT_EQ(Results[0]->Alpha, 1);
					EXPECT_EQ(Results[0]->Next, nullptr);
				}
				else
				{
					//after first two frames we have enough to interpolate
					EXPECT_GE(Step, 2);
					EXPECT_GT(Results[0]->Next->ExternalEndTime, RenderTime);
				}

				PreTime = StartTime;
			}
		}

		{
			FChaosMarshallingManager MarshallingManager;
			FChaosResultsManager ResultsManager(MarshallingManager);

			FReal PreTime;
			const FReal ExternalDt = 1 / 30.f;
			FReal ExternalTime = 0;
			const FReal Delay = ExternalDt * 2 + 1e-2;

			int InnerStepTotal = 0;
			for (int Step = 0; Step < 10; ++Step)
			{
				const FReal StartTime = ExternalTime;	//external time we would have kicked the sim task off with
				ExternalTime += ExternalDt;
				const FReal RenderTime = ExternalTime - Delay;

				//even if we have multiple smaller results, interpolate as needed
				const FReal InnerDt = ExternalDt / 3.;
				for(int InnerStep = 0; InnerStep < 3; ++InnerStep)
				{
					MarshallingManager.FinalizePullData_Internal(InnerStepTotal++, StartTime + InnerDt * InnerStep, InnerDt);
				}

				TArray<const FChaosInterpolationResults*> Results = ResultsManager.PullAsyncPhysicsResults_External(RenderTime);
				if (RenderTime < 0)
				{
					EXPECT_LT(Step, 2);	//first two frames treat as sync mode since we don't have enough delay
					EXPECT_EQ(Results[0]->Alpha, 1);
					EXPECT_EQ(Results[0]->Next, nullptr);	//until we have enough results we just use the first result
				}
				else
				{
					//after first two frames we have enough to interpolate
					EXPECT_GE(Step, 2);
					EXPECT_GT(Results[0]->Next->ExternalEndTime, RenderTime);
				}

				PreTime = StartTime;
			}
		}
	}

}
