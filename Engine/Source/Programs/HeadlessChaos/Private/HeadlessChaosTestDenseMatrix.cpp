// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestConstraints.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Modules/ModuleManager.h"
#include "Chaos/DenseMatrix.h"
#include "Chaos/Utilities.h"

namespace ChaosTest {

	using namespace Chaos;

	void InitDenseMatrixTest()
	{
		FMath::RandInit(7473957);
	}

	template<int32 E>
	void CompareMatrices(const TDenseMatrix<E>& A, const FReal B, FReal Epsilon = SMALL_NUMBER)
	{
		EXPECT_EQ(A.NumRows(), 1);
		EXPECT_EQ(A.NumColumns(), 1);
		EXPECT_NEAR(B, A.At(0, 0), Epsilon);
	}

	template<int32 E>
	void CompareMatrices(const TDenseMatrix<E>& A, const FVec3& B, FReal Epsilon = SMALL_NUMBER)
	{
		EXPECT_EQ(A.NumRows(), 3);
		EXPECT_EQ(A.NumColumns(), 1);
		for (int32 RowIndex = 0; RowIndex < 3; ++RowIndex)
		{
			EXPECT_NEAR(B[RowIndex], A.At(RowIndex, 0), Epsilon);
		}
	}

	template<int32 E>
	void CompareMatrices(const TDenseMatrix<E>& A, const FMatrix33& B, FReal Epsilon = SMALL_NUMBER)
	{
		EXPECT_EQ(A.NumRows(), 3);
		EXPECT_EQ(A.NumColumns(), 3);
		for (int32 RowIndex = 0; RowIndex < 3; ++RowIndex)
		{
			for (int32 ColIndex = 0; ColIndex < 3; ++ColIndex)
			{
				EXPECT_NEAR(B.GetRow(RowIndex)[ColIndex], A.At(RowIndex, ColIndex), Epsilon);
				EXPECT_NEAR(B.GetColumn(ColIndex)[RowIndex], A.At(RowIndex, ColIndex), Epsilon);
			}
		}
	}

	template<int32 EA, int32 EB>
	void CompareMatrices(const TDenseMatrix<EA>& A, const TDenseMatrix<EB>& B, FReal Epsilon = SMALL_NUMBER)
	{
		EXPECT_EQ(A.NumRows(), B.NumRows());
		EXPECT_EQ(A.NumColumns(), B.NumColumns());
		for (int32 RowIndex = 0; RowIndex < A.NumRows(); ++RowIndex)
		{
			for (int32 ColIndex = 0; ColIndex < A.NumColumns(); ++ColIndex)
			{
				EXPECT_NEAR(A.At(RowIndex, ColIndex), B.At(RowIndex, ColIndex), Epsilon);
			}
		}
	}

	template<int32 T_R, int32 T_C>
	TDenseMatrix<T_R * T_C> RandomSymmetricPositiveDefiniteDenseMatrix(FReal MinValue, FReal MaxValue)
	{
		TDenseMatrix<T_R * T_C> Result = TDenseMatrix<T_R * T_C>::Make(T_R, T_C);
		for (int32 RowIndex = 0; RowIndex < Result.NumRows(); ++RowIndex)
		{
			Result.At(RowIndex, RowIndex) = FMath::RandRange(MinValue, MaxValue);

			for (int32 ColIndex = RowIndex + 1; ColIndex < Result.NumColumns(); ++ColIndex)
			{
				FReal V = FMath::RandRange(-MinValue, MinValue);
				Result.At(RowIndex, ColIndex) = V;
				Result.At(ColIndex, RowIndex) = V;
			}
		}
		return Result;
	}

	template<int32 T_R, int32 T_C>
	TDenseMatrix<T_R * T_C> RandomDenseMatrix(FReal MinValue, FReal MaxValue)
	{
		TDenseMatrix<T_R * T_C> Result = TDenseMatrix<T_R * T_C>::Make(T_R, T_C);
		for (int32 RowIndex = 0; RowIndex < Result.NumRows(); ++RowIndex)
		{
			for (int32 ColIndex = 0; ColIndex < Result.NumColumns(); ++ColIndex)
			{
				Result.At(RowIndex, ColIndex) = FMath::RandRange(MinValue, MaxValue);
			}
		}
		return Result;
	}

	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMake31)
	{
		InitDenseMatrixTest();

		FVec3 V = RandomVector(-1000, 1000);
		TDenseMatrix<3> DM = TDenseMatrix<3>::Make(V);
		EXPECT_EQ(DM.NumRows(), 3);
		EXPECT_EQ(DM.NumColumns(), 1);
		CompareMatrices(DM, V);
	}

	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMake33)
	{
		InitDenseMatrixTest();

		FMatrix33 M = RandomMatrix(-1000, 1000);
		TDenseMatrix<9> DM = TDenseMatrix<9>::Make(M);
		EXPECT_EQ(DM.NumRows(), 3);
		EXPECT_EQ(DM.NumColumns(), 3);
		CompareMatrices(DM, M);
	}

	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMakeInit)
	{
		InitDenseMatrixTest();

		const FReal V = FMath::RandRange(-10, 10);
		TDenseMatrix<25> DM = TDenseMatrix<25>::Make(5, 5, V);
		EXPECT_EQ(DM.NumRows(), 5);
		EXPECT_EQ(DM.NumColumns(), 5);

		for (int RowIndex = 0; RowIndex < 5; ++RowIndex)
		{
			for (int ColIndex = 0; ColIndex < 5; ++ColIndex)
			{
				EXPECT_EQ(DM.At(RowIndex, ColIndex), V);
			}
		}
	}

	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMakeDiagonal)
	{
		InitDenseMatrixTest();

		const FReal V = FMath::RandRange(-10, 10);
		TDenseMatrix<25> DM = TDenseMatrix<25>::MakeDiagonal(5, 5, V);
		EXPECT_EQ(DM.NumRows(), 5);
		EXPECT_EQ(DM.NumColumns(), 5);

		for (int RowIndex = 0; RowIndex < 5; ++RowIndex)
		{
			for (int ColIndex = 0; ColIndex < 5; ++ColIndex)
			{
				EXPECT_EQ(DM.At(RowIndex, ColIndex), (RowIndex == ColIndex) ? V : (FReal)0);
			}
		}
	}

	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMakeIdentity)
	{
		InitDenseMatrixTest();

		const FReal V = (FReal)1;
		TDenseMatrix<9> DM = TDenseMatrix<9>::MakeIdentity(3);
		EXPECT_EQ(DM.NumRows(), 3);
		EXPECT_EQ(DM.NumColumns(), 3);

		for (int RowIndex = 0; RowIndex < 3; ++RowIndex)
		{
			for (int ColIndex = 0; ColIndex < 3; ++ColIndex)
			{
				EXPECT_EQ(DM.At(RowIndex, ColIndex), (RowIndex == ColIndex) ? (FReal)1 : (FReal)0);
			}
		}
	}

	GTEST_TEST(DenseMatrixTests, TestDenseMatrixSetDiagonal)
	{
		InitDenseMatrixTest();

		TDenseMatrix<25> DM = TDenseMatrix<25>::Make(5, 5, 0);
		EXPECT_EQ(DM.NumRows(), 5);
		EXPECT_EQ(DM.NumColumns(), 5);

		TDenseMatrix<25> DM2 = DM;
		DM2.SetDiagonal(0);

		for (int RowIndex = 0; RowIndex < 5; ++RowIndex)
		{
			for (int ColIndex = 0; ColIndex < 5; ++ColIndex)
			{
				FReal ExpectedV = (RowIndex == ColIndex) ? (FReal)0 : DM.At(RowIndex, ColIndex);
				EXPECT_EQ(DM2.At(RowIndex, ColIndex), ExpectedV);
			}
		}
	}

	GTEST_TEST(DenseMatrixTests, TestDenseMatrixSetDiagonalAt)
	{
		InitDenseMatrixTest();

		TDenseMatrix<25> DM = TDenseMatrix<25>::Make(5, 5, 0);
		EXPECT_EQ(DM.NumRows(), 5);
		EXPECT_EQ(DM.NumColumns(), 5);

		TDenseMatrix<25> DM2 = DM;
		DM2.SetDiagonalAt(1, 3, 0);

		for (int RowIndex = 0; RowIndex < 5; ++RowIndex)
		{
			for (int ColIndex = 0; ColIndex < 5; ++ColIndex)
			{
				FReal ExpectedV = ((RowIndex >= 1) && (RowIndex <= 3) && (RowIndex == ColIndex)) ? (FReal)0 : DM.At(RowIndex, ColIndex);
				EXPECT_EQ(DM2.At(RowIndex, ColIndex), ExpectedV);
			}
		}
	}

	GTEST_TEST(DenseMatrixTests, TestDenseMatrixSetRow)
	{
		InitDenseMatrixTest();

		TDenseMatrix<25> DM = TDenseMatrix<25>::Make(5, 5, 0);
		EXPECT_EQ(DM.NumRows(), 5);
		EXPECT_EQ(DM.NumColumns(), 5);

		FVec3 V = { 1, 1, 1 };
		DM.SetRowAt(2, 1, V);
		for (int RowIndex = 0; RowIndex < 5; ++RowIndex)
		{
			for (int ColIndex = 0; ColIndex < 5; ++ColIndex)
			{
				FReal ExpectedV = ((RowIndex == 2) && (ColIndex >= 1) && (ColIndex < 4)) ? 1 : 0;
				EXPECT_EQ(DM.At(RowIndex, ColIndex), ExpectedV);
			}
		}
	}

	GTEST_TEST(DenseMatrixTests, TestDenseMatrixSetColumn)
	{
		InitDenseMatrixTest();

		TDenseMatrix<25> DM = TDenseMatrix<25>::Make(5, 5, 0);
		EXPECT_EQ(DM.NumRows(), 5);
		EXPECT_EQ(DM.NumColumns(), 5);

		FVec3 V = { 1, 1, 1 };
		DM.SetColumnAt(2, 1, V);
		for (int RowIndex = 0; RowIndex < 5; ++RowIndex)
		{
			for (int ColIndex = 0; ColIndex < 5; ++ColIndex)
			{
				FReal ExpectedV = ((RowIndex >= 2) && (ColIndex == 1) && (RowIndex < 5)) ? 1 : 0;
				EXPECT_EQ(DM.At(RowIndex, ColIndex), ExpectedV);
			}
		}
	}

	GTEST_TEST(DenseMatrixTests, TestDenseMatrixSetBlock)
	{
		InitDenseMatrixTest();

		TDenseMatrix<25> D0 = RandomDenseMatrix<5, 5>(-10, 10);
		TDenseMatrix<25> DM = D0;
		EXPECT_EQ(DM.NumRows(), 5);
		EXPECT_EQ(DM.NumColumns(), 5);

		TDenseMatrix<9> V = RandomDenseMatrix<3,3>(-10, 10);

		DM.SetBlockAt(1, 1, V);

		for (int RowIndex = 0; RowIndex < 5; ++RowIndex)
		{
			for (int ColIndex = 0; ColIndex < 5; ++ColIndex)
			{
				FReal ExpectedV = ((RowIndex >= 1) && (RowIndex <= 3) && (ColIndex >= 1) && (ColIndex <= 3)) ? V.At(RowIndex - 1, ColIndex - 1) : D0.At(RowIndex, ColIndex);
				EXPECT_EQ(DM.At(RowIndex, ColIndex), ExpectedV);
			}
		}
	}

	GTEST_TEST(DenseMatrixTests, TestDenseMatrixSetBlock2)
	{
		InitDenseMatrixTest();

		TDenseMatrix<25> D0 = RandomDenseMatrix<5, 5>(-10, 10);
		TDenseMatrix<25> DM = D0;
		EXPECT_EQ(DM.NumRows(), 5);
		EXPECT_EQ(DM.NumColumns(), 5);

		FMatrix33 V = RandomMatrix(-10, 10);
		
		DM.SetBlockAt(1, 1, V);

		for (int RowIndex = 0; RowIndex < 5; ++RowIndex)
		{
			for (int ColIndex = 0; ColIndex < 5; ++ColIndex)
			{
				FReal ExpectedV = ((RowIndex >= 1) && (RowIndex <= 3) && (ColIndex >= 1) && (ColIndex <= 3)) ? V.M[ColIndex - 1][RowIndex - 1] : D0.At(RowIndex, ColIndex);
				EXPECT_EQ(DM.At(RowIndex, ColIndex), ExpectedV);
			}
		}
	}

	GTEST_TEST(DenseMatrixTests, TestDenseMatrixTranspose)
	{
		InitDenseMatrixTest();

		TDenseMatrix<10 * 10> DA = RandomDenseMatrix<10, 10>(-1000, 1000);
		TDenseMatrix<10 * 10> DAt = TDenseMatrix<10 * 10>::Transpose(DA);

		for (int RowIndex = 0; RowIndex < 5; ++RowIndex)
		{
			for (int ColIndex = 0; ColIndex < 5; ++ColIndex)
			{
				EXPECT_EQ(DA.At(RowIndex, ColIndex), DAt.At(ColIndex, RowIndex));
			}
		}
	}

	// Test C = -A
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixNegate)
	{
		InitDenseMatrixTest();

		FMatrix33 A = RandomMatrix(-1000, 1000);
		FMatrix33 C = -A;

		TDenseMatrix<3 * 3> DA = TDenseMatrix<3 * 3>::Make(A);
		TDenseMatrix<3 * 3> DC = TDenseMatrix<3 * 3>::Negate(DA);

		CompareMatrices(DC, C);
	}

	// Test C = A + B
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixAdd)
	{
		InitDenseMatrixTest();

		FMatrix33 A = RandomMatrix(-1000, 1000);
		FMatrix33 B = RandomMatrix(-1000, 1000);
		FMatrix33 C = A + B;

		TDenseMatrix<3 * 3> DA = TDenseMatrix<3 * 3>::Make(A);
		TDenseMatrix<3 * 3> DB = TDenseMatrix<3 * 3>::Make(B);
		TDenseMatrix<3 * 3> DC = TDenseMatrix<3 * 3>::Add(DA, DB);

		CompareMatrices(DC, C);
	}

	// Test C = A + B Symmetric
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixAddSymmetric)
	{
		InitDenseMatrixTest();

		TDenseMatrix<10 * 10> A = RandomSymmetricPositiveDefiniteDenseMatrix<10, 10>(-1000, 1000);
		TDenseMatrix<10 * 10> B = RandomSymmetricPositiveDefiniteDenseMatrix<10, 10>(-1000, 1000);
		TDenseMatrix<10 * 10> C = TDenseMatrix<10 * 10>::Add(A, B);
		TDenseMatrix<10 * 10> C2 = TDenseMatrix<10 * 10>::Add_Symmetric(A, B);

		CompareMatrices(C2, C);
	}

	// Test C = A - B
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixSubtract)
	{
		InitDenseMatrixTest();

		FMatrix33 A = RandomMatrix(-1000, 1000);
		FMatrix33 B = RandomMatrix(-1000, 1000);
		FMatrix33 C = A - B;

		TDenseMatrix<3 * 3> DA = TDenseMatrix<3 * 3>::Make(A);
		TDenseMatrix<3 * 3> DB = TDenseMatrix<3 * 3>::Make(B);
		TDenseMatrix<3 * 3> DC = TDenseMatrix<3 * 3>::Subtract(DA, DB);

		CompareMatrices(DC, C);
	}

	// Test C = A x V
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMultiplyVector)
	{
		InitDenseMatrixTest();

		FMatrix33 A = RandomMatrix(-1000, 1000);
		FVec3 B = RandomVector(-1000, 1000);
		FVec3 C = Utilities::Multiply(A, B);

		TDenseMatrix<3 * 3> DA = TDenseMatrix<3 * 3>::Make(A);
		TDenseMatrix<3 * 1> DB = TDenseMatrix<3 * 1>::Make(B);
		TDenseMatrix<3 * 1> DC = TDenseMatrix<3 * 1>::MultiplyAB(DA, DB);

		CompareMatrices(DC, C);
	}

	// Test C = A x r
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMultiplyReal)
	{
		InitDenseMatrixTest();

		FMatrix33 A = RandomMatrix(-1000, 1000);
		FReal B = FMath::RandRange(-1000, 1000);
		FMatrix33 C = A * B;

		TDenseMatrix<3 * 3> DA = TDenseMatrix<3 * 3>::Make(A);
		TDenseMatrix<3 * 3> DC1 = TDenseMatrix<3 * 3>::Multiply(DA, B);
		TDenseMatrix<3 * 3> DC2 = TDenseMatrix<3 * 3>::Multiply(B, DA);

		CompareMatrices(DC1, C, (FReal)1.e-6);
		CompareMatrices(DC2, C, (FReal)1.e-6);
	}

	// Test C = A / r
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixDivideReal)
	{
		InitDenseMatrixTest();

		FMatrix33 A = RandomMatrix(-1000, 1000);
		FReal B = FMath::RandRange(-1000, 1000);
		FMatrix33 C = A * ((FReal)1 / B);

		TDenseMatrix<3 * 3> DA = TDenseMatrix<3 * 3>::Make(A);
		TDenseMatrix<3 * 3> DC = TDenseMatrix<3 * 3>::Divide(DA, B);

		CompareMatrices(DC, C, (FReal)1.e-6);
	}

	// Test C = A x B
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMultiplyAB)
	{
		InitDenseMatrixTest();

		FMatrix33 A = RandomMatrix(-1000, 1000);
		FMatrix33 B = RandomMatrix(-1000, 1000);
		FMatrix33 C = Utilities::MultiplyAB(A, B);

		TDenseMatrix<3 * 3> DA = TDenseMatrix<3 * 3>::Make(A);
		TDenseMatrix<3 * 3> DB = TDenseMatrix<3 * 3>::Make(B);
		TDenseMatrix<3 * 3> DC = TDenseMatrix<3 * 3>::MultiplyAB(DA, DB);

		CompareMatrices(DC, C);
	}

	// Test C = A x B Symmetric
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMultiplyABSymmetric)
	{
		InitDenseMatrixTest();

		// B is sym pos def
		// A.B.At is sym pos def
		TDenseMatrix<5 * 10> A = RandomDenseMatrix<5, 10>(-10, 10);
		TDenseMatrix<10 * 10> B = RandomSymmetricPositiveDefiniteDenseMatrix<10, 10>(-10, 10);
		TDenseMatrix<10 * 5> BAt = TDenseMatrix<10 * 5>::MultiplyABt(B, A);
		TDenseMatrix<5 * 5> ABAt = TDenseMatrix<5 * 5>::MultiplyAB(A, BAt);
		TDenseMatrix<5 * 5> ABAt2 = TDenseMatrix<5 * 5>::MultiplyAB_Symmetric(A, BAt);
		
		CompareMatrices(ABAt, ABAt2, 1.e-2f);
	}

	// Test D = A + B x C Symmetric
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMultiplyBCAddASymmetric)
	{
		InitDenseMatrixTest();

		// C is sym
		// B.C.Bt is sym
		// A + B.C.Bt is sym
		TDenseMatrix<10 * 10> A = RandomSymmetricPositiveDefiniteDenseMatrix<10, 10>(-10, 10);
		TDenseMatrix<10 * 10> B = RandomDenseMatrix<10, 10>(-10, 10);
		TDenseMatrix<10 * 10> C = RandomSymmetricPositiveDefiniteDenseMatrix<10, 10>(-10, 10);
		TDenseMatrix<10 * 10> CBt = TDenseMatrix<10 * 10>::MultiplyABt(C, B);
		TDenseMatrix<10 * 10> BCBt = TDenseMatrix<10 * 10>::MultiplyAB(B, CBt);
		TDenseMatrix<10 * 10> ApBCBt = TDenseMatrix<10 * 10>::Add(A, BCBt);
		TDenseMatrix<10 * 10> ApBCBt2 = TDenseMatrix<10 * 10>::MultiplyBCAddA_Symmetric(A, B, CBt);

		CompareMatrices(ApBCBt, ApBCBt2, 1.e-2f);
	}

	// Test C = Transpose(A) x B
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMultiplyAtB)
	{
		InitDenseMatrixTest();

		FMatrix33 A = RandomMatrix(-1000, 1000);
		FMatrix33 B = RandomMatrix(-1000, 1000);
		FMatrix33 C = Utilities::MultiplyAB(A.GetTransposed(), B);

		TDenseMatrix<3 * 3> DA = TDenseMatrix<3 * 3>::Make(A);
		TDenseMatrix<3 * 3> DB = TDenseMatrix<3 * 3>::Make(B);
		TDenseMatrix<3 * 3> DC = TDenseMatrix<3 * 3>::MultiplyAtB(DA, DB);

		CompareMatrices(DC, C);
	}

	// Test C = A x Transpose(B)
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMultiplyABt)
	{
		InitDenseMatrixTest();

		FMatrix33 A = RandomMatrix(-1000, 1000);
		FMatrix33 B = RandomMatrix(-1000, 1000);
		FMatrix33 C = Utilities::MultiplyAB(A, B.GetTransposed());

		TDenseMatrix<3 * 3> DA = TDenseMatrix<3 * 3>::Make(A);
		TDenseMatrix<3 * 3> DB = TDenseMatrix<3 * 3>::Make(B);
		TDenseMatrix<3 * 3> DC = TDenseMatrix<3 * 3>::MultiplyABt(DA, DB);

		CompareMatrices(DC, C);
	}

	// Test C = Transpose(A) x Transpose(B)
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMultiplyAtBt)
	{
		InitDenseMatrixTest();

		FMatrix33 A = RandomMatrix(-1000, 1000);
		FMatrix33 B = RandomMatrix(-1000, 1000);
		FMatrix33 C = Utilities::MultiplyAB(A.GetTransposed(), B.GetTransposed());

		TDenseMatrix<3 * 3> DA = TDenseMatrix<3 * 3>::Make(A);
		TDenseMatrix<3 * 3> DB = TDenseMatrix<3 * 3>::Make(B);
		TDenseMatrix<3 * 3> DC = TDenseMatrix<3 * 3>::MultiplyAtBt(DA, DB);

		CompareMatrices(DC, C);
	}

	// Test C = A x M, where M is a mass matrix
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMassMatrixMultiply)
	{
		InitDenseMatrixTest();

		FReal M = 1.0f / FMath::RandRange(10, 100);
		FMatrix33 Ilocal = FMatrix33(1.0f / FMath::RandRange(10, 100), 1.0f / FMath::RandRange(10, 100), 1.0f / FMath::RandRange(10, 100));
		FMatrix33 R = RandomRotation(360, 180, 180).ToMatrix();
		FMatrix33 Iworld = Utilities::ComputeWorldSpaceInertia(R, Ilocal);

		TDenseMatrix<10 * 6> A = RandomDenseMatrix<10, 6>(-1000, 1000);
		TDenseMatrix<6 * 6> I = TDenseMatrix<6 * 6>::Make(6, 6, 0);
		I.SetDiagonalAt(0, 3, M);
		I.SetBlockAt(3, 3, Iworld);
		TDenseMatrix<10 * 6> Expected = TDenseMatrix<10 * 6>::MultiplyAB(A, I);
		EXPECT_EQ(Expected.NumRows(), 10);
		EXPECT_EQ(Expected.NumColumns(), 6);

		FMassMatrix DI = FMassMatrix::Make(M, Iworld);
		TDenseMatrix<10 * 6> Result = TDenseMatrix<10 * 6>::MultiplyAB(A, DI);

		CompareMatrices(Expected, Result, KINDA_SMALL_NUMBER);
	}

	// Test C = M x Transpose(A), where M is a mass matrix
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMassMatrixMultiply2)
	{
		InitDenseMatrixTest();

		FReal M = 1.0f / FMath::RandRange(10, 100);
		FMatrix33 Ilocal = FMatrix33(1.0f / FMath::RandRange(10, 100), 1.0f / FMath::RandRange(10, 100), 1.0f / FMath::RandRange(10, 100));
		FMatrix33 R = RandomRotation(360, 180, 180).ToMatrix();
		FMatrix33 Iworld = Utilities::ComputeWorldSpaceInertia(R, Ilocal);

		TDenseMatrix<6 * 10> A = RandomDenseMatrix<6, 10>(-1000, 1000);
		TDenseMatrix<6 * 6> I = TDenseMatrix<6 * 6>::Make(6, 6, 0);
		I.SetDiagonalAt(0, 3, M);
		I.SetBlockAt(3, 3, Iworld);
		TDenseMatrix<10 * 6> Expected = TDenseMatrix<10 * 6>::MultiplyAB(I, A);
		EXPECT_EQ(Expected.NumRows(), 6);
		EXPECT_EQ(Expected.NumColumns(), 10);

		FMassMatrix DI = FMassMatrix::Make(M, Iworld);
		TDenseMatrix<10 * 6> Result = TDenseMatrix<10 * 6>::MultiplyAB(DI, A);

		CompareMatrices(Expected, Result, KINDA_SMALL_NUMBER);
	}

	// Test C = M x Transpose(A), where M is a mass matrix
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixMassMatrixMultiply3)
	{
		InitDenseMatrixTest();

		FReal M = 1.0f / FMath::RandRange(10, 100);
		FMatrix33 Ilocal = FMatrix33(1.0f / FMath::RandRange(10, 100), 1.0f / FMath::RandRange(10, 100), 1.0f / FMath::RandRange(10, 100));
		FMatrix33 R = RandomRotation(360, 180, 180).ToMatrix();
		FMatrix33 Iworld = Utilities::ComputeWorldSpaceInertia(R, Ilocal);

		TDenseMatrix<10 * 6> A = RandomDenseMatrix<10, 6>(-1000, 1000);
		TDenseMatrix<6 * 6> I = TDenseMatrix<6 * 6>::Make(6, 6, 0);
		I.SetDiagonalAt(0, 3, M);
		I.SetBlockAt(3, 3, Iworld);
		TDenseMatrix<6 * 10> Expected = TDenseMatrix<6 * 10>::MultiplyABt(I, A);
		EXPECT_EQ(Expected.NumRows(), 6);
		EXPECT_EQ(Expected.NumColumns(), 10);

		FMassMatrix DI = FMassMatrix::Make(M, Iworld);
		TDenseMatrix<6 * 10> Result = TDenseMatrix<6 * 10>::MultiplyABt(DI, A);

		CompareMatrices(Expected, Result, KINDA_SMALL_NUMBER);
	}

	// Test C = DotProduct(A, B) = <A|B> = Transpose(A) x B
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixDotProduct)
	{
		InitDenseMatrixTest();

		FVec3 A = RandomVector(-1000, 1000);
		FVec3 B = RandomVector(-1000, 1000);
		FReal C = FVec3::DotProduct(A, B);

		TDenseMatrix<3 * 1> DA = TDenseMatrix<3 * 1>::Make(A);
		TDenseMatrix<3 * 1> DB = TDenseMatrix<3 * 1>::Make(B);
		TDenseMatrix<1 * 1> DC = TDenseMatrix<1 * 1>::DotProduct(DA, DB);

		CompareMatrices(DC, C);
	}

	// Test A = GGt, where G = Cholesky decomposition of A
	GTEST_TEST(DenseMatrixTests, TestCholeskyDecomposition)
	{
		InitDenseMatrixTest();

		// Generate a Positive Definite Matrix
		const int32 Dim = 4;
		TDenseMatrix<Dim * Dim> A = RandomSymmetricPositiveDefiniteDenseMatrix<Dim, Dim>(10, 100);
		TDenseMatrix<Dim * Dim> G = A;

		// Calculate Cholesky Factor G, where A = GGt
		bool bSuccess = FDenseMatrixSolver::CholeskyFactorize(G);
		EXPECT_TRUE(bSuccess);

		// Veryify that A = GGt
		TDenseMatrix<Dim * Dim> L = TDenseMatrix<Dim * Dim>::MultiplyABt(G, G);
		CompareMatrices(A, L, KINDA_SMALL_NUMBER);
	}

	// Test AX = B, solve for X with trivial A
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixSolver_Identity)
	{
		InitDenseMatrixTest();

		// Set of trivial equations x = b (x, b scalars), stored in a matrix
		const int32 Dim = 4;
		TDenseMatrix<Dim * Dim> A = TDenseMatrix<Dim * Dim>::MakeIdentity(Dim);
		TDenseMatrix<Dim> B = RandomDenseMatrix<Dim, 1>(1, 100);

		// Solve for X
		TDenseMatrix<Dim> X;
		bool bSuccess = FDenseMatrixSolver::SolvePositiveDefinite(A, B, X);
		EXPECT_TRUE(bSuccess);

		// Verify that we calculated X = B
		CompareMatrices(X, B, KINDA_SMALL_NUMBER);
	}

	// Test AX = B, solve for X with non-trivial A
	GTEST_TEST(DenseMatrixTests, TestDenseMatrixSolver)
	{
		InitDenseMatrixTest();

		// Set of 10 equations of 10 variables: 
		// A(10x10) . X(10x1) = B(10x1)
		// where A is symmetric positive definite
		const int32 Dim = 10;
		TDenseMatrix<Dim * Dim> A = RandomSymmetricPositiveDefiniteDenseMatrix<Dim, Dim>(1, 10);
		TDenseMatrix<Dim> B = RandomDenseMatrix<Dim, 1>(1, 100);

		// Solve AX = B for X
		TDenseMatrix<Dim> X;
		bool bSuccess = FDenseMatrixSolver::SolvePositiveDefinite(A, B, X);
		EXPECT_TRUE(bSuccess);

		// Verify that AX = B
		TDenseMatrix<Dim> B2 = TDenseMatrix<Dim>::MultiplyAB(A, X);
		CompareMatrices(B, B2, KINDA_SMALL_NUMBER);
	}
}