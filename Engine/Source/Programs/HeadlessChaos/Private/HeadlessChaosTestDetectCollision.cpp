// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosCollisionConstraints.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/Collision/CapsuleConvexContactPoint.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/Collision/SphereConvexContactPoint.h"
#include "Chaos/CollisionOneShotManifolds.h"
#include "Chaos/GJK.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"
#include "Modules/ModuleManager.h"

namespace ChaosTest 
{
	using namespace Chaos;

	void CheckContactPoint(const FContactPoint& ContactPoint, const FVec3& ShapeContactPos0, const FVec3& ShapeContactPos1, const FVec3& ShapeContactNormal, const int32 NormalOwnerIndex, const FVec3 Normal, const FReal Phi)
	{
		const FReal DistanceTolerance = 1.e-3f;
		const FReal NormalTolerance = 1.e-3f;

		EXPECT_NEAR(ContactPoint.ShapeContactPoints[0].X, ShapeContactPos0.X, DistanceTolerance);
		EXPECT_NEAR(ContactPoint.ShapeContactPoints[0].Y, ShapeContactPos0.Y, DistanceTolerance);
		EXPECT_NEAR(ContactPoint.ShapeContactPoints[0].Z, ShapeContactPos0.Z, DistanceTolerance);
		EXPECT_NEAR(ContactPoint.ShapeContactPoints[1].X, ShapeContactPos1.X, DistanceTolerance);
		EXPECT_NEAR(ContactPoint.ShapeContactPoints[1].Y, ShapeContactPos1.Y, DistanceTolerance);
		EXPECT_NEAR(ContactPoint.ShapeContactPoints[1].Z, ShapeContactPos1.Z, DistanceTolerance);
		EXPECT_NEAR(ContactPoint.ShapeContactNormal.X, ShapeContactNormal.X, NormalTolerance);
		EXPECT_NEAR(ContactPoint.ShapeContactNormal.Y, ShapeContactNormal.Y, NormalTolerance);
		EXPECT_NEAR(ContactPoint.ShapeContactNormal.Z, ShapeContactNormal.Z, NormalTolerance);
		EXPECT_NEAR(ContactPoint.Phi, Phi, DistanceTolerance);
	}

	//
	//
	// SPHERE - CONVEX TESTS
	//
	//

	// Sphere-Convex(with margin) far separated when a face is the nearest feature
	GTEST_TEST(DetectCollisionTests, TestSphereConvex_Separated_Face)
	{
		const FReal SphereRadius = 50.0f;
		const FVec3 BoxSize = FVec3(1000, 1000, 50);
		const FReal BoxMargin = 10.0f;
		const FVec3 SpherePos = FVec3(0, 0, 70);
		const FVec3 ConvexPos = FVec3(0, 0, -25);

		const FImplicitSphere3 Sphere(FVec3(0,0,0), SphereRadius);
		const FImplicitConvex3 Convex = CreateConvexBox(BoxSize, BoxMargin);

		const FRigidTransform3 SphereTransform = FRigidTransform3(SpherePos, FRotation3::FromIdentity());
		const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

		{
			FContactPoint ContactPoint = SphereConvexContactPoint(Sphere, SphereTransform, Convex, ConvexTransform);
			EXPECT_TRUE(ContactPoint.IsSet());
			if (ContactPoint.IsSet())
			{
				CheckContactPoint(ContactPoint, FVec3(0, 0, -SphereRadius), FVec3(0, 0, 0.5f * BoxSize.Z), FVec3(0,0,1), 1, FVec3(0,0,1), SpherePos.Z - SphereRadius);
			}
		}
	}

	// Sphere-Convex(with margin) far separated when an edge is the nearest feature.
	GTEST_TEST(DetectCollisionTests, TestSphereConvex_Separated_Edge)
	{
		const FReal SphereRadius = 50.0f;
		const FVec3 BoxSize = FVec3(1000, 1000, 50);
		const FReal BoxMargin = 10.0f;
		const FVec3 SpherePos = FVec3(600, 0, 70);
		const FVec3 ConvexPos = FVec3(0, 0, -25);

		const FImplicitSphere3 Sphere(FVec3(0, 0, 0), SphereRadius);
		const FImplicitConvex3 Convex = CreateConvexBox(BoxSize, BoxMargin);

		const FRigidTransform3 SphereTransform = FRigidTransform3(SpherePos, FRotation3::FromIdentity());
		const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

		FContactPoint ContactPoint = SphereConvexContactPoint(Sphere, SphereTransform, Convex, ConvexTransform);
		EXPECT_TRUE(ContactPoint.IsSet());
		if (ContactPoint.IsSet())
		{
			const FVec3 SphereOffset = FVec3(SpherePos.X - 0.5f * BoxSize.X, 0, SpherePos.Z);
			const FVec3 ExpectedNormal = SphereOffset.GetSafeNormal();
			const FReal ExpectedDistance = SphereOffset.Size() - Sphere.GetRadiusf();

			CheckContactPoint(ContactPoint, -SphereRadius * ExpectedNormal, 0.5f * FVec3(BoxSize.X, 0, BoxSize.Z), ExpectedNormal, 1, ExpectedNormal, ExpectedDistance);
		}
	}

	// Sphere-Convex(with margin) far separated when a vertex is the nearest feature.
	GTEST_TEST(DetectCollisionTests, TestSphereConvex_Separated_Vertex)
	{
		const FReal SphereRadius = 50.0f;
		const FVec3 BoxSize = FVec3(1000, 1000, 50);
		const FReal BoxMargin = 10.0f;
		const FVec3 SpherePos = FVec3(600, 540, 70);
		const FVec3 ConvexPos = FVec3(0, 0, -25);

		const FImplicitSphere3 Sphere(FVec3(0, 0, 0), SphereRadius);
		const FImplicitConvex3 Convex = CreateConvexBox(BoxSize, BoxMargin);

		const FRigidTransform3 SphereTransform = FRigidTransform3(SpherePos, FRotation3::FromIdentity());
		const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

		FContactPoint ContactPoint = SphereConvexContactPoint(Sphere, SphereTransform, Convex, ConvexTransform);
		EXPECT_TRUE(ContactPoint.IsSet());
		if (ContactPoint.IsSet())
		{
			const FVec3 SphereOffset = FVec3(SpherePos.X - 0.5f * BoxSize.X, SpherePos.Y - 0.5f * BoxSize.Y, SpherePos.Z);
			const FVec3 ExpectedNormal = SphereOffset.GetSafeNormal();
			const FReal ExpectedDistance = SphereOffset.Size() - Sphere.GetRadiusf();

			CheckContactPoint(ContactPoint, -SphereRadius * ExpectedNormal, 0.5f * FVec3(BoxSize.X, BoxSize.Y, BoxSize.Z), ExpectedNormal, 1, ExpectedNormal, ExpectedDistance);
		}
	}

	// Sphere-Convex(with margin) far separated when a vertex is the nearest feature.
	GTEST_TEST(DetectCollisionTests, TestSphereConvex_Scaled_Separated_Vertex)
	{
		const FReal SphereRadius = 50.0f;
		const FVec3 BoxSize = FVec3(1000, 1000, 50);
		const FReal BoxMargin = 10.0f;
		const FVec3 SpherePos = FVec3(600, 540, 70);
		const FVec3 ConvexPos = FVec3(0, 0, -25);

		const FImplicitSphere3 Sphere(FVec3(0, 0, 0), SphereRadius);
		const TImplicitObjectScaled<FImplicitConvex3> Convex = CreateScaledConvexBox(FVec3(1,1,1), BoxSize, BoxMargin);

		const FRigidTransform3 SphereTransform = FRigidTransform3(SpherePos, FRotation3::FromIdentity());
		const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

		FContactPoint ContactPoint = SphereConvexContactPoint(Sphere, SphereTransform, Convex, ConvexTransform);
		EXPECT_TRUE(ContactPoint.IsSet());
		if (ContactPoint.IsSet())
		{
			const FVec3 SphereOffset = FVec3(SpherePos.X - 0.5f * BoxSize.X, SpherePos.Y - 0.5f * BoxSize.Y, SpherePos.Z);
			const FVec3 ExpectedNormal = SphereOffset.GetSafeNormal();
			const FReal ExpectedDistance = SphereOffset.Size() - Sphere.GetRadiusf();

			CheckContactPoint(ContactPoint, -SphereRadius * ExpectedNormal, 0.5f * FVec3(BoxSize.X, BoxSize.Y, BoxSize.Z), ExpectedNormal, 1, ExpectedNormal, ExpectedDistance);
		}
	}

	// Sphere-Convex(with margin) deep contact
	GTEST_TEST(DetectCollisionTests, TestSphereConvex_DeepContact)
	{
		const FReal SphereRadius = 50.0f;
		const FVec3 BoxSize = FVec3(1000, 1000, 50);
		const FReal BoxMargin = 10.0f;
		const FVec3 SpherePos = FVec3(400, 20, 10);
		const FVec3 ConvexPos = FVec3(0, 0, -25);

		const FImplicitSphere3 Sphere(FVec3(0, 0, 0), SphereRadius);
		const FImplicitConvex3 Convex = CreateConvexBox(BoxSize, BoxMargin);

		const FRigidTransform3 SphereTransform = FRigidTransform3(SpherePos, FRotation3::FromIdentity());
		const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

		FContactPoint ContactPoint = SphereConvexContactPoint(Sphere, SphereTransform, Convex, ConvexTransform);
		EXPECT_TRUE(ContactPoint.IsSet());
		if (ContactPoint.IsSet())
		{
			CheckContactPoint(ContactPoint, FVec3(0, 0, -SphereRadius), FVec3(SpherePos.X, SpherePos.Y, 0.5f * BoxSize.Z), FVec3(0, 0, 1), 1, FVec3(0, 0, 1), SpherePos.Z - SphereRadius);
		}
	}

	// Sphere-Convex(with margin) and touching core shapes
	GTEST_TEST(DetectCollisionTests, TestSphereConvex_Touching_Face)
	{
		const FReal SphereRadius = 50.0f;
		const FVec3 BoxSize = FVec3(1000, 1000, 50);
		const FReal BoxMargin = 10.0f;
		const FVec3 SpherePos = FVec3(0, 0, -50);
		const FVec3 ConvexPos = FVec3(0, 0, -25);

		const FImplicitSphere3 Sphere(FVec3(0, 0, 0), SphereRadius);
		const FImplicitConvex3 Convex = CreateConvexBox(BoxSize, BoxMargin);

		const FRigidTransform3 SphereTransform = FRigidTransform3(SpherePos, FRotation3::FromIdentity());
		const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

		FContactPoint ContactPoint = SphereConvexContactPoint(Sphere, SphereTransform, Convex, ConvexTransform);
		EXPECT_TRUE(ContactPoint.IsSet());
		if (ContactPoint.IsSet())
		{
			CheckContactPoint(ContactPoint, FVec3(0, 0, SphereRadius), FVec3(SpherePos.X, SpherePos.Y, -0.5f * BoxSize.Z), FVec3(0, 0, -1), 1, FVec3(0, 0, -1), SpherePos.Z + SphereRadius - BoxSize.Z);
		}
	}

	// Sphere-Convex(with margin) and core shapes separated by a small epsilon (less than GJK epsilon)
	GTEST_TEST(DetectCollisionTests, TestSphereConvex_LessEpsilon_Face)
	{
		const FReal SphereRadius = 50.0f;
		const FVec3 BoxSize = FVec3(1000, 1000, 50);
		const FReal BoxMargin = 10.0f;
		const FVec3 SpherePos = FVec3(0, 0, -50.0005f);		// Note: implicit knowledge of default epsilon of GJKDistance (1.e-3f)
		const FVec3 ConvexPos = FVec3(0, 0, -25);

		const FImplicitSphere3 Sphere(FVec3(0, 0, 0), SphereRadius);
		const FImplicitConvex3 Convex = CreateConvexBox(BoxSize, BoxMargin);

		const FRigidTransform3 SphereTransform = FRigidTransform3(SpherePos, FRotation3::FromIdentity());
		const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

		FContactPoint ContactPoint = SphereConvexContactPoint(Sphere, SphereTransform, Convex, ConvexTransform);
		EXPECT_TRUE(ContactPoint.IsSet());
		if (ContactPoint.IsSet())
		{
			CheckContactPoint(ContactPoint, FVec3(0, 0, SphereRadius), FVec3(SpherePos.X, SpherePos.Y, -0.5f * BoxSize.Z), FVec3(0, 0, -1), 1, FVec3(0, 0, -1), SpherePos.Z + SphereRadius - BoxSize.Z);
		}
	}

	// Sphere-Convex(with margin) and core shapes separated by a small epsilon (just greater than GJK epsilon)
	GTEST_TEST(DetectCollisionTests, TestSphereConvex_GreaterEpsilon_Face)
	{
		const FReal SphereRadius = 50.0f;
		const FVec3 BoxSize = FVec3(1000, 1000, 50);
		const FReal BoxMargin = 10.0f;
		const FVec3 SpherePos = FVec3(0, 0, -50.0015f);	// Note: implicit knowledge of default epsilon of GJKDistance (1.e-3f)
		const FVec3 ConvexPos = FVec3(0, 0, -25);

		const FImplicitSphere3 Sphere(FVec3(0, 0, 0), SphereRadius);
		const FImplicitConvex3 Convex = CreateConvexBox(BoxSize, BoxMargin);

		const FRigidTransform3 SphereTransform = FRigidTransform3(SpherePos, FRotation3::FromIdentity());
		const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

		FContactPoint ContactPoint = SphereConvexContactPoint(Sphere, SphereTransform, Convex, ConvexTransform);
		EXPECT_TRUE(ContactPoint.IsSet());
		if (ContactPoint.IsSet())
		{
			CheckContactPoint(ContactPoint, FVec3(0, 0, SphereRadius), FVec3(SpherePos.X, SpherePos.Y, -0.5f * BoxSize.Z), FVec3(0, 0, -1), 1, FVec3(0, 0, -1), -BoxSize.Z - (SpherePos.Z + SphereRadius));
		}
	}


	//
	//
	// CAPSULE - CONVEX TESTS
	//
	//

	// Capsule-Convex far separated when a face is the nearest feature
	GTEST_TEST(DetectCollisionTests, TestCapsuleConvex_Separated_Parallel_Face)
	{
		const FReal CapsuleRadius = 50.0f;
		const FReal CapsuleSegmentLength = 100.0f;
		const FVec3 BoxSize = FVec3(1000, 1000, 50);
		const FReal BoxMargin = 10.0f;
		const FVec3 CapsulePos = FVec3(0, 0, 70);
		const FVec3 ConvexPos = FVec3(0, 0, -25);

		const FImplicitCapsule3 Capsule(FVec3(-0.5f * CapsuleSegmentLength, 0.0f, 0.0f), FVec3(0.5f * CapsuleSegmentLength, 0.0f, 0.0f), CapsuleRadius);
		const TImplicitObjectScaled<FImplicitConvex3> Convex = CreateScaledConvexBox(FVec3(1, 1, 1), BoxSize, BoxMargin);

		const FRigidTransform3 CapsuleTransform = FRigidTransform3(CapsulePos, FRotation3::FromIdentity());
		const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

		FContactPoint ContactPoint = CapsuleConvexContactPoint(Capsule, CapsuleTransform, Convex, ConvexTransform);
		EXPECT_TRUE(ContactPoint.IsSet());
		if (ContactPoint.IsSet())
		{
			CheckContactPoint(ContactPoint, FVec3(0, 0, -CapsuleRadius), FVec3(0, 0, 0.5f * BoxSize.Z), FVec3(0, 0, 1), 1, FVec3(0, 0, 1), CapsulePos.Z - CapsuleRadius);
		}
	}

	// Capsule-Convex far separated when the nearest features are an edge on the convex and the middle of the capsule segment (capsule is rotated)
	GTEST_TEST(DetectCollisionTests, TestCapsuleConvex_Separated_EdgeEdge)
	{
		const FReal CapsuleRadius = 50.0f;
		const FReal CapsuleSegmentLength = 100.0f;
		const FVec3 BoxSize = FVec3(1000, 1000, 50);
		const FReal BoxMargin = 10.0f;
		const FVec3 CapsulePos = FVec3(600, 0, 100);
		const FVec3 ConvexPos = FVec3(0, 0, -25);

		const FImplicitCapsule3 Capsule(FVec3(-0.5f * CapsuleSegmentLength, 0.0f, 0.0f), FVec3(0.5f * CapsuleSegmentLength, 0.0f, 0.0f), CapsuleRadius);
		const TImplicitObjectScaled<FImplicitConvex3> Convex = CreateScaledConvexBox(FVec3(1, 1, 1), BoxSize, BoxMargin);

		const FRigidTransform3 CapsuleTransform = FRigidTransform3(CapsulePos, FRotation3::FromAxisAngle(FVec3(0,1,0), FMath::DegreesToRadians(45)));
		const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

		FContactPoint ContactPoint = CapsuleConvexContactPoint(Capsule, CapsuleTransform, Convex, ConvexTransform);
		EXPECT_TRUE(ContactPoint.IsSet());
		if (ContactPoint.IsSet())
		{
			const FVec3 CapsuleOffset = CapsulePos - FVec3(0.5f * BoxSize.X, 0.0f, 0.0f);
			const FVec3 ExpectedNormal = CapsuleOffset.GetSafeNormal();
			const FReal ExpectedPhi = CapsuleOffset.Size() - CapsuleRadius;
			CheckContactPoint(ContactPoint, FVec3(0, 0, -CapsuleRadius), FVec3(0.5f * BoxSize.X, 0, 0.5f * BoxSize.Z), ExpectedNormal, 1, ExpectedNormal, ExpectedPhi);
		}
	}


	// Capsule-Convex deep contact when the nearest features are an edge on the convex and the middle of the capsule segment (capsule is rotated)
	GTEST_TEST(DetectCollisionTests, TestCapsuleConvex_DeepContact_EdgeEdge)
	{
		const FReal CapsuleRadius = 50.0f;
		const FReal CapsuleSegmentLength = 100.0f;
		const FVec3 BoxSize = FVec3(1000, 1000, 50);
		const FReal BoxMargin = 10.0f;
		const FVec3 CapsulePos = FVec3(480, 0, -20);
		const FVec3 ConvexPos = FVec3(0, 0, -25);

		const FImplicitCapsule3 Capsule(FVec3(-0.5f * CapsuleSegmentLength, 0.0f, 0.0f), FVec3(0.5f * CapsuleSegmentLength, 0.0f, 0.0f), CapsuleRadius);
		const TImplicitObjectScaled<FImplicitConvex3> Convex = CreateScaledConvexBox(FVec3(1, 1, 1), BoxSize, BoxMargin);

		const FRigidTransform3 CapsuleTransform = FRigidTransform3(CapsulePos, FRotation3::FromAxisAngle(FVec3(0, 1, 0), FMath::DegreesToRadians(45)));
		const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

		FContactPoint ContactPoint = CapsuleConvexContactPoint(Capsule, CapsuleTransform, Convex, ConvexTransform);
		EXPECT_TRUE(ContactPoint.IsSet());
		if (ContactPoint.IsSet())
		{
			const FVec3 CapsuleOffset = CapsulePos - FVec3(0.5f * BoxSize.X, 0.0f, 0.0f);
			const FVec3 ExpectedNormal = -CapsuleOffset.GetSafeNormal();
			const FReal ExpectedPhi = -(CapsuleOffset.Size() + CapsuleRadius);
			CheckContactPoint(ContactPoint, FVec3(0, 0, -CapsuleRadius), FVec3(0.5f * BoxSize.X, 0, 0.5f * BoxSize.Z), ExpectedNormal, 1, ExpectedNormal, ExpectedPhi);
		}
	}

	// Capsule-Convex OneShot Manifold when the convex lands on the end of the capsule and the convex face points exactly down the axis
	// This exposed a bug in ConstructCapsuleConvexOneShotManifold where it would try to add a cylinder face contact resulting in a NaN 
	// from the call to GetUnsafeNormal.
	GTEST_TEST(DetectCollisionTests, TestCapsuleConvex_Manifold_OpposingFaceNormalAxis)
	{
		const FRealSingle CapsuleRadius = 50.0f;
		const FRealSingle CapsuleSegmentLength = 100.0f;
		const FVec3f BoxSize = FVec3(100, 100, 100);
		const FRealSingle BoxMargin = 0.0f;
		const FRealSingle CullDistance = 10.0f;

		const FImplicitCapsule3 Capsule(FVec3(0.0f, 0.0f, -0.5f * CapsuleSegmentLength), FVec3(0.0f, 0.0f, 0.5f * CapsuleSegmentLength), CapsuleRadius);
		const FImplicitConvex3 Convex = CreateConvexBox(-0.5f * BoxSize, 0.5f * BoxSize, BoxMargin);

		{
			const FVec3f CapsulePos = FVec3(0, 0, -50);
			const FVec3f ConvexPos = FVec3(0, 0, 100);
			const FRigidTransform3 CapsuleTransform = FRigidTransform3(CapsulePos, FRotation3::FromIdentity());
			const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

			FContactPointManifold ManifoldPoints;
			Collisions::ConstructCapsuleConvexOneShotManifold(Capsule, CapsuleTransform, Convex, ConvexTransform, CullDistance, ManifoldPoints);

			EXPECT_EQ(ManifoldPoints.Num(), 1);
			if (ManifoldPoints.Num() == 1)
			{
				EXPECT_EQ(ManifoldPoints[0].ContactType, EContactPointType::VertexPlane);
				EXPECT_NEAR(ManifoldPoints[0].ShapeContactNormal.Z, -1.0f, UE_KINDA_SMALL_NUMBER);
				EXPECT_NEAR(ManifoldPoints[0].ShapeContactPoints[0].Z, 100.0f, UE_KINDA_SMALL_NUMBER);
				EXPECT_NEAR(ManifoldPoints[0].ShapeContactPoints[1].Z, -50.0f, UE_KINDA_SMALL_NUMBER);
				EXPECT_NEAR(ManifoldPoints[0].Phi, 0.0f, UE_KINDA_SMALL_NUMBER);
			}
		}

		{
			const FVec3f CapsulePos = FVec3(0, 0, 50);
			const FVec3f ConvexPos = FVec3(0, 0, -100);
			const FRigidTransform3 CapsuleTransform = FRigidTransform3(CapsulePos, FRotation3::FromIdentity());
			const FRigidTransform3 ConvexTransform = FRigidTransform3(ConvexPos, FRotation3::FromIdentity());

			FContactPointManifold ManifoldPoints;
			Collisions::ConstructCapsuleConvexOneShotManifold(Capsule, CapsuleTransform, Convex, ConvexTransform, CullDistance, ManifoldPoints);

			EXPECT_EQ(ManifoldPoints.Num(), 1);
			if (ManifoldPoints.Num() == 1)
			{
				EXPECT_EQ(ManifoldPoints[0].ContactType, EContactPointType::VertexPlane);
				EXPECT_NEAR(ManifoldPoints[0].ShapeContactNormal.Z, 1.0f, UE_KINDA_SMALL_NUMBER);
				EXPECT_NEAR(ManifoldPoints[0].ShapeContactPoints[0].Z, -100.0f, UE_KINDA_SMALL_NUMBER);
				EXPECT_NEAR(ManifoldPoints[0].ShapeContactPoints[1].Z, 50.0f, UE_KINDA_SMALL_NUMBER);
				EXPECT_NEAR(ManifoldPoints[0].Phi, 0.0f, UE_KINDA_SMALL_NUMBER);
			}
		}
	}

}