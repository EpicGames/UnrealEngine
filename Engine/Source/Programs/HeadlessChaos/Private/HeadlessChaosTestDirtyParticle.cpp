// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "ChaosSolversModule.h"
#include "Chaos/ParticleHandle.h"
#include "PhysicsProxy/PhysicsProxies.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/Box.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"

#include "Modules/ModuleManager.h"

namespace ChaosTest
{

	using namespace Chaos;

	GTEST_TEST(DirtyParticleTests,Basic)
	{
#if 0
		auto Particle = FGeometryParticle::CreateParticle();
		Particle->SetX(FVec3(1,1,1));

		Chaos::FImplicitObjectPtr Ptr(new TSphere<FReal,3>(FVec3(0),0));
		const auto RawPtr = Ptr.Get();
		TWeakPtr<FImplicitObject,ESPMode::ThreadSafe> WeakPtr(Ptr);
		FDirtyPropertiesManager Manager;

		{
			FParticlePropertiesData RemoteData(&Manager);
			FShapeRemoteDataContainer ContainerData(&Manager);

			Particle->SetRemoteData(RemoteData, ContainerData);

			//then on PushToPhysics we'd do a pointer swap and use RemoteData internally

			EXPECT_TRUE(RemoteData.HasX());
			EXPECT_EQ(RemoteData.GetX(),FVec3(1,1,1));
			EXPECT_FALSE(RemoteData.HasInvM());	//was never set so it's false

			Particle->SetX(FVec3(2,1,1));
			EXPECT_EQ(RemoteData.GetX(),FVec3(2,1,1));	//remote is set so immediate change

			//make sure we are not leaking shared ptrs
			Particle->SetGeometry(Ptr);
			Ptr = nullptr;

			EXPECT_TRUE(WeakPtr.IsValid());	//still around because particle is holding on to it
			EXPECT_TRUE(RemoteData.HasGeometry());
			EXPECT_EQ(RemoteData.GetGeometry().Get(),RawPtr);

			Particle->DetachRemoteData();	//disconnect remote data so we can pretend we are freeing things on GT without affecting PT

			Particle->SetGeometry(Ptr);	//free geometry on GT side

			//geometry still on PT side
			EXPECT_TRUE(RemoteData.HasGeometry());
			EXPECT_EQ(RemoteData.GetGeometry().Get(),RawPtr);

			EXPECT_TRUE(WeakPtr.IsValid());	//still around because particle is holding on to it
		}
		
		//remote data is gone so geometry shared ptr is freed, even though pool is still around (i.e. it was removed from pool)
		EXPECT_FALSE(WeakPtr.IsValid());



#endif
	}

	// Simple dynamics particle falling under gravity, 
	// Then switch it to a kinematic state
	GTEST_TEST(DirtyParticleTests, ChangeStateTest)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();

		const float AsyncDt = -1.0f;
		Chaos::FPBDRigidsSolver* Solver = Module->CreateSolver(nullptr, AsyncDt, Chaos::EThreadingMode::SingleThread, TEXT("Test Dirty Particles"));
		ChaosTest::InitSolverSettings(Solver);

		FVector3d HalfExtents = FVec3(1.0, 1.0, 1.0) * FTransform::Identity.GetScale3D().GetAbs();
		FImplicitObjectPtr BoxGeom = FImplicitObjectPtr(new Chaos::FImplicitBox3(-HalfExtents, HalfExtents));
		FSingleParticlePhysicsProxy* Proxy = FSingleParticlePhysicsProxy::Create(Chaos::TPBDRigidParticle<FReal, 3>::CreateParticle());
		FRigidBodyHandle_External& ParticleGT = Proxy->GetGameThreadAPI();
		ParticleGT.SetGeometry(BoxGeom);
		ParticleGT.SetX(FTransform::Identity.GetLocation());
		ParticleGT.SetR(FTransform::Identity.GetRotation());
		ParticleGT.SetM(1.0);
		ParticleGT.SetInvM(1.0f);

		Proxy->GetGameThreadAPI().SetObjectState(EObjectStateType::Dynamic);
		Solver->RegisterObject(Proxy);

		FReal Altitude = ParticleGT.GetX().Z;
		EXPECT_EQ(Altitude, 0.0);

		const float Dt = 1 / 60.0f;
		Solver->AdvanceAndDispatch_External(Dt);
		Solver->UpdateGameThreadStructures();
		Solver->SyncEvents_GameThread();

		FReal OldAltitude = Altitude;
		Altitude = ParticleGT.GetX().Z;
		EXPECT_LT(Altitude, OldAltitude);

		Solver->AdvanceAndDispatch_External(Dt);
		Solver->UpdateGameThreadStructures();
		Solver->SyncEvents_GameThread();

		OldAltitude = Altitude;
		Altitude = ParticleGT.GetX().Z;
		EXPECT_LT(Altitude, OldAltitude);

		// Change the particle state to kinematic
		Proxy->GetGameThreadAPI().SetObjectState(EObjectStateType::Kinematic);

		Solver->AdvanceAndDispatch_External(Dt);
		Solver->UpdateGameThreadStructures();
		Solver->SyncEvents_GameThread();

		// Expecting the particle is not falling down anymore
		OldAltitude = Altitude;
		Altitude = ParticleGT.GetX().Z;
		EXPECT_EQ(Altitude, OldAltitude);

		Solver->UnregisterObject(Proxy);
		FChaosSolversModule::GetModule()->DestroySolver(Solver);
	}
}

