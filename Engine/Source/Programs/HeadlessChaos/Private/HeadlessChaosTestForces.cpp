// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestForces.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Modules/ModuleManager.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"

namespace ChaosTest
{
	template<typename TEvolution>
	void Gravity()
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		TEvolution Evolution(Particles, PhysicalMaterials);
		
		TArray<FPBDRigidParticleHandle*> Dynamics = Evolution.CreateDynamicParticles(1);
		Evolution.EnableParticle(Dynamics[0]);
		Evolution.AdvanceOneTimeStep(0.1);
		EXPECT_LT(Dynamics[0]->GetX()[2], 0);
	}
	
	GTEST_TEST(AllEvolutions,Forces)
	{
		ChaosTest::Gravity<FPBDRigidsEvolutionGBF>();
		SUCCEED();
	}
}