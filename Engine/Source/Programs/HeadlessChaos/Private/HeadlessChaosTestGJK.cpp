// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestGJK.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/GJK.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/GJK.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/TriangleRegister.h"

namespace ChaosTest
{
	using namespace Chaos;

	//for each simplex test:
	//- points get removed
	// - points off simplex return false
	//- points in simplex return true
	//- degenerate simplex

	void SimplexLine()
	{
		{
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,-1,-1}, {-1,-1,1} };
			int32 Idxs[] = { 0,1 };
			int32 NumVerts = 2;
			const FVec3 ClosestPoint = LineSimplexFindOrigin(Simplex, Idxs, NumVerts, Barycentric);
			EXPECT_EQ(NumVerts, 2);
			EXPECT_FLOAT_EQ(ClosestPoint[0], -1);
			EXPECT_FLOAT_EQ(ClosestPoint[1], -1);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 0);
			EXPECT_FLOAT_EQ(Barycentric[0], 0.5);
			EXPECT_FLOAT_EQ(Barycentric[1], 0.5);
		}

		{
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,-1,-1}, {1,1,1} };
			int32 Idxs[] = { 0,1 };
			int32 NumVerts = 2;
			const FVec3 ClosestPoint = LineSimplexFindOrigin(Simplex, Idxs, NumVerts, Barycentric);
			EXPECT_EQ(NumVerts, 2);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 0);
			EXPECT_FLOAT_EQ(Barycentric[0], 0.5);
			EXPECT_FLOAT_EQ(Barycentric[1], 0.5);
		}

		{
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {1,1,1}, {1,2,3} };
			int32 Idxs[] = { 0,1 };
			int32 NumVerts = 2;
			const FVec3 ClosestPoint = LineSimplexFindOrigin(Simplex, Idxs, NumVerts, Barycentric);
			EXPECT_EQ(NumVerts, 1);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 1);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 1);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 1);
			EXPECT_FLOAT_EQ(Barycentric[0], 1);
			EXPECT_EQ(Idxs[0], 0);
		}

		{
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {10,11,12}, {1,2,3} };
			int32 Idxs[] = { 0,1 };
			int32 NumVerts = 2;
			const FVec3 ClosestPoint = LineSimplexFindOrigin(Simplex, Idxs, NumVerts, Barycentric);
			EXPECT_EQ(NumVerts, 1);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 1);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 2);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 3);
			EXPECT_FLOAT_EQ(Barycentric[1], 1);
			EXPECT_EQ(Idxs[0], 1);
		}

		{
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {1,1,1}, {1,1,1} };
			int32 Idxs[] = { 0,1 };
			int32 NumVerts = 2;
			const FVec3 ClosestPoint = LineSimplexFindOrigin(Simplex, Idxs, NumVerts, Barycentric);
			EXPECT_EQ(NumVerts, 1);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 1);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 1);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 1);
			EXPECT_FLOAT_EQ(Barycentric[0], 1);
			EXPECT_EQ(Idxs[0], 0);
		}

		{
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {1,-1e-16,1}, {1,1e-16,1} };
			int32 Idxs[] = { 0,1 };
			int32 NumVerts = 2;
			const FVec3 ClosestPoint = LineSimplexFindOrigin(Simplex, Idxs, NumVerts, Barycentric);
			EXPECT_EQ(NumVerts, 2);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 1);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 1);
			EXPECT_FLOAT_EQ(Barycentric[0], 0.5);
			EXPECT_FLOAT_EQ(Barycentric[1], 0.5);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);
		}
	}

	void SimplexTriangle()
	{
		{
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,-1,-1}, {-1,1,-1}, {-2,1,-1} };
			FSimplex Idxs = { 0,1, 2 };
			
			const FVec3 ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 2);
			EXPECT_FLOAT_EQ(ClosestPoint[0], -1);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], -1);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);
			EXPECT_FLOAT_EQ(Barycentric[0], 0.5);
			EXPECT_FLOAT_EQ(Barycentric[1], 0.5);
		}

		{
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,-1,-1},{-2,1,-1}, {-1,1,-1} };
			FSimplex Idxs = { 0,1, 2 };
			const FVec3 ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 2);
			EXPECT_FLOAT_EQ(ClosestPoint[0], -1);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], -1);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 2);
			EXPECT_FLOAT_EQ(Barycentric[0], 0.5);
			EXPECT_FLOAT_EQ(Barycentric[2], 0.5);
		}

		{
			//corner
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {1,1,1},{2,1,1}, {2,2,1} };
			FSimplex Idxs = { 1,0, 2 };
			const FVec3 ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 1);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 1);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 1);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 1);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_FLOAT_EQ(Barycentric[0], 1);
		}

		{
			//corner equal
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {0,0,0},{2,1,1}, {2,2,1} };
			FSimplex Idxs = { 0,1, 2 };
			const FVec3 ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 1);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 0);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_FLOAT_EQ(Barycentric[0], 1);
		}

		{
			//edge equal
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,0,0},{1,0,0}, {0,2,0} };
			FSimplex Idxs = { 2,0, 1 };
			const FVec3 ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 2);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 0);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);
			EXPECT_FLOAT_EQ(Barycentric[0], 0.5);
			EXPECT_FLOAT_EQ(Barycentric[1], 0.5);
		}

		{
			//triangle equal
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,0,-1},{1,0,-1}, {0,0,1} };
			FSimplex Idxs = { 0,1, 2 };
			const FVec3 ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 3);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 0);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);
			EXPECT_EQ(Idxs[2], 2);
			EXPECT_FLOAT_EQ(Barycentric[0], 0.25);
			EXPECT_FLOAT_EQ(Barycentric[1], 0.25);
			EXPECT_FLOAT_EQ(Barycentric[2], 0.5);
		}

		{
			//co-linear
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,-1,-1},{-1,1,-1}, {-1,1.2,-1} };
			FSimplex Idxs = { 0,1, 2 };
			const FVec3 ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 2);
			EXPECT_FLOAT_EQ(ClosestPoint[0], -1);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], -1);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);	//degenerate triangle throws out newest point
			EXPECT_FLOAT_EQ(Barycentric[0], 0.5);
			EXPECT_FLOAT_EQ(Barycentric[1], 0.5);
		}

		{
			//single point
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,-1,-1},{-1,-1,-1}, {-1,-1,-1} };
			FSimplex Idxs = { 0,2, 1 };
			const FVec3 ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 1);
			EXPECT_FLOAT_EQ(ClosestPoint[0], -1);
			EXPECT_FLOAT_EQ(ClosestPoint[1], -1);
			EXPECT_FLOAT_EQ(ClosestPoint[2], -1);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_FLOAT_EQ(Barycentric[0], 1);
		}

		{
			//corner perfect split
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,-1,0},{1,-1,0}, {0,-0.5,0} };
			FSimplex Idxs = { 0,2, 1 };
			const FVec3 ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 1);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[1], -0.5);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 0);
			EXPECT_EQ(Idxs[0], 2);
			EXPECT_FLOAT_EQ(Barycentric[2], 1);
		}

		{
			//triangle face correct distance
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,-1,-1},{1,-1,-1}, {0,1,-1} };
			FSimplex Idxs = { 0,1,2 };
			const FVec3 ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 3);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], -1);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);
			EXPECT_EQ(Idxs[2], 2);
			EXPECT_FLOAT_EQ(Barycentric[0], 0.25);
			EXPECT_FLOAT_EQ(Barycentric[1], 0.25);
			EXPECT_FLOAT_EQ(Barycentric[2], 0.5);
		}

		{
			//tiny triangle middle point
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1e-9,-1e-9,-1e-9},{-1e-9,1e-9,-1e-9}, {-1e-9,0,1e-9} };
			FSimplex Idxs = { 0,1,2 };
			const FVec3 ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 3);
			EXPECT_FLOAT_EQ(ClosestPoint[0], -1e-9f);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 0);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);
			EXPECT_EQ(Idxs[2], 2);
			EXPECT_FLOAT_EQ(Barycentric[0], 0.25);
			EXPECT_FLOAT_EQ(Barycentric[1], 0.25);
			EXPECT_FLOAT_EQ(Barycentric[2], 0.5);
		}

		{
			//non cartesian triangle plane
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {2, 0, -1}, {0, 2, -1}, {1, 1, 1} };
			FSimplex Idxs = { 0,1,2 };
			const FVec3 ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 3);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 1);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 1);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 0);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);
			EXPECT_EQ(Idxs[2], 2);
			EXPECT_FLOAT_EQ(Barycentric[0], 0.25);
			EXPECT_FLOAT_EQ(Barycentric[1], 0.25);
			EXPECT_FLOAT_EQ(Barycentric[2], 0.5);
		}
	}

	void SimplexTetrahedron()
	{
		{
			//top corner
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,-1,-1}, {1,-1,-1}, {0,1,-1}, {0,0,-0.5} };
			FSimplex Idxs = { 0,1,2,3 };
			const FVec3 ClosestPoint = TetrahedronSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 1);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], -0.5);
			EXPECT_EQ(Idxs[0], 3);
			EXPECT_FLOAT_EQ(Barycentric[3], 1);
		}

		{
			//inside
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,-1,-1}, {1,-1,-1}, {0,1,-1}, {0,0,0.5} };
			FSimplex Idxs = { 0,1,2,3 };
			const FVec3 ClosestPoint = TetrahedronSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 4);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 0);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);
			EXPECT_EQ(Idxs[2], 2);
			EXPECT_EQ(Idxs[3], 3);
			EXPECT_FLOAT_EQ(Barycentric[0] + Barycentric[1] + Barycentric[2] + Barycentric[3], 1);
		}

		{
			//face
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {0,0,-1.5}, {-1,-1,-1}, {1,-1,-1}, {0,1,-1} };
			FSimplex Idxs = { 0,1,2,3 }; 
			const FVec3 ClosestPoint = TetrahedronSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 3);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[1], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[2], -1);
			EXPECT_EQ(Idxs[0], 1);
			EXPECT_EQ(Idxs[1], 2);
			EXPECT_EQ(Idxs[2], 3);
			EXPECT_FLOAT_EQ(Barycentric[1] + Barycentric[2] + Barycentric[3], 1);
		}

		{
			//edge
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,-1,0}, {1,-1,0}, {0,-1,-1}, {0, -2, -1} };
			FSimplex Idxs = { 0,1,2,3 };
			const FVec3 ClosestPoint = TetrahedronSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 2);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[1], -1);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 0);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);
			EXPECT_FLOAT_EQ(Barycentric[0], 0.5);
			EXPECT_FLOAT_EQ(Barycentric[1], 0.5);
		}

		{
			//degenerate
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-1,-1,0}, {1,-1,0}, {0,-1,-1}, {0, -1, -0.5} };
			FSimplex Idxs = { 0,1,2,3 };
			const FVec3 ClosestPoint = TetrahedronSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 2);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[1], -1);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 0);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);
			EXPECT_FLOAT_EQ(Barycentric[0], 0.5);
			EXPECT_FLOAT_EQ(Barycentric[1], 0.5);
		}

		{
			//wide angle, bad implementation would return edge but it's really a face
			FReal Barycentric[4];
			const FVec3 Simplex[] = { {-10000,-1,10000}, {1,-1,10000}, {4,-3,10000}, {1, -1, -10000} };
			FSimplex Idxs = { 0,1,2,3 };
			const FVec3 ClosestPoint = TetrahedronSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 3);
			EXPECT_FLOAT_EQ(ClosestPoint[0], 0);
			EXPECT_FLOAT_EQ(ClosestPoint[1], -1);
			EXPECT_FLOAT_EQ(ClosestPoint[2], 0);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);
			EXPECT_EQ(Idxs[2], 3);
			EXPECT_FLOAT_EQ(Barycentric[0] + Barycentric[1] + Barycentric[3], 1);
		}

		// LWC-TODO : this is failing when using LWC, disabling it for now to avoid blocking builds
#if 0
		{
			// Previous failing case observed with Voronoi region implementation - Not quite degenerate (totally degenerate cases work)
			FReal Barycentric[4];
			FVec3 Simplex[] = { { -15.9112930, -15.2787428, 1.33070087 },
								{ 1.90487099, 2.25161266, 0.439208984 },
								{ -15.8914719, -15.2915068, 1.34186459 },
								{ 1.90874290, 2.24025059, 0.444719315 } };

			FSimplex Idxs = { 0,1,2,3 };
			const FVec3 ClosestPoint = TetrahedronSimplexFindOrigin(Simplex, Idxs, Barycentric);
			EXPECT_EQ(Idxs.NumVerts, 3);
			EXPECT_EQ(Idxs[0], 0);
			EXPECT_EQ(Idxs[1], 1);
			EXPECT_EQ(Idxs[2], 2);
		}
#endif
	}

	//For each gjk test we should test:
	// - thickness
	// - transformed geometry
	// - rotated geometry
	// - degenerate cases
	// - near miss, near hit
	// - multiple initial dir

	void GJKSphereSphereTest()
	{
		Chaos::FSphere A(FVec3(10, 0, 0), 5);
		Chaos::FSphere B(FVec3(4, 0, 0), 2);

		FVec3 InitialDirs[] = { FVec3(1,0,0), FVec3(-1,0,0), FVec3(0,1,0), FVec3(0,-1,0), FVec3(0,0,1), FVec3(0,0,-1) };

		for (const FVec3& InitialDir : InitialDirs)
		{
			EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3::Identity, 0, InitialDir));

			//miss
			EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(-1.1, 0, 0), FRotation3::Identity), 0, InitialDir));

			//hit from thickness
			EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(-1.1, 0, 0), FRotation3::Identity), 0.105, InitialDir));

			//miss with thickness
			EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(-1.1, 0, 0), FRotation3::Identity), 0.095, InitialDir));

			//hit with rotation
			EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(6.5, 0, 0), FRotation3::FromVector(FVec3(0, 0, PI))), 1, InitialDir));

			//miss with rotation
			EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(6.5, 0, 0), FRotation3::FromVector(FVec3(0, 0, PI))), 0.01, InitialDir));

			//hit tiny
			Chaos::FSphere Tiny(FVec3(0), 1e-2);
			EXPECT_TRUE(GJKIntersection<FReal>(A, Tiny, FRigidTransform3(FVec3(15, 0, 0), FRotation3::Identity), 0, InitialDir));

			//miss tiny
			EXPECT_FALSE(GJKIntersection<FReal>(A, Tiny, FRigidTransform3(FVec3(15 + 1e-1, 0, 0), FRotation3::Identity), 0, InitialDir));
		}
	}


	void GJKSphereBoxTest()
	{
		Chaos::FSphere A(FVec3(10, 0, 0), 5);
		FAABB3 B(FVec3(-4, -2, -4), FVec3(4,2,4));

		FVec3 InitialDirs[] = { FVec3(1,0,0), FVec3(-1,0,0), FVec3(0,1,0), FVec3(0,-1,0), FVec3(0,0,1), FVec3(0,0,-1) };

		for (const FVec3& InitialDir : InitialDirs)
		{
			EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(1, 0, 0), FRotation3::Identity), 0, InitialDir));

			//miss
			EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(0.9, 0, 0), FRotation3::Identity), 0, InitialDir));

			//rotate and hit
			EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(3.1, 0, 0), FRotation3::FromVector(FVec3(0,0,PI*0.5))), 0, InitialDir));

			//rotate and miss
			EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(2.9, 0, 0), FRotation3::FromVector(FVec3(0, 0, PI*0.5))), 0, InitialDir));

			//rotate and hit from thickness
			EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(2.9, 0, 0), FRotation3::FromVector(FVec3(0, 0, PI*0.5))), 0.1, InitialDir));

			//hit thin
			FAABB3 Thin(FVec3(4, -2, -4), FVec3(4, 2, 4));
			EXPECT_TRUE(GJKIntersection<FReal>(A, Thin, FRigidTransform3(FVec3(1+1e-2, 0, 0), FRotation3::Identity), 0, InitialDir));

			//miss
			EXPECT_FALSE(GJKIntersection<FReal>(A, Thin, FRigidTransform3(FVec3(1 - 1e-2, 0, 0), FRotation3::Identity), 0, InitialDir));

			//hit line
			FAABB3 Line(FVec3(4, -2, 0), FVec3(4, 2, 0));
			EXPECT_TRUE(GJKIntersection<FReal>(A, Line, FRigidTransform3(FVec3(1 + 1e-2, 0, 0), FRotation3::Identity), 0, InitialDir));

			//miss
			EXPECT_FALSE(GJKIntersection<FReal>(A, Line, FRigidTransform3(FVec3(1 - 1e-2, 0, 0), FRotation3::Identity), 0, InitialDir));
		}
	}


	void GJKSphereCapsuleTest()
	{
		Chaos::FSphere A(FVec3(10, 0, 0), 5);
		FCapsule B(FVec3(0, 0, -3), FVec3(0, 0, 3), 3);

		FVec3 InitialDirs[] = { FVec3(1,0,0), FVec3(-1,0,0), FVec3(0,1,0), FVec3(0,-1,0), FVec3(0,0,1), FVec3(0,0,-1) };

		for (const FVec3& InitialDir : InitialDirs)
		{
			EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(2, 0, 0), FRotation3::Identity), 0, InitialDir));

			//miss
			EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(2-1e-2, 0, 0), FRotation3::Identity), 0, InitialDir));

			//thickness
			EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(1, 0, 0), FRotation3::Identity), 1.01, InitialDir));

			//miss
			EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(1, 0, 0), FRotation3::Identity), 0.99, InitialDir));

			//rotation hit
			EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(-1+1e-2, 0, 0), FRotation3::FromVector(FVec3(0,PI*0.5,0))), 0, InitialDir));

			//miss
			EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(-1-1e-2, 0, 0), FRotation3::FromVector(FVec3(0, PI*0.5, 0))), 0, InitialDir));

			//degenerate
			FCapsule Line(FVec3(0, 0, -3), FVec3(0, 0, 3), 0);
			EXPECT_TRUE(GJKIntersection<FReal>(A, Line, FRigidTransform3(FVec3(5+1e-2, 0, 0), FRotation3::Identity), 0, InitialDir));

			//miss
			EXPECT_FALSE(GJKIntersection<FReal>(A, Line, FRigidTransform3(FVec3(5 - 1e-2, 0, 0), FRotation3::Identity), 0, InitialDir));
		}
	}


	void GJKSphereConvexTest()
	{
		FVec3 InitialDirs[] = { FVec3(1,0,0), FVec3(-1,0,0), FVec3(0,1,0), FVec3(0,-1,0), FVec3(0,0,1), FVec3(0,0,-1) };
		Chaos::FSphere A(FVec3(10, 0, 0), 5);

		{
			//Tetrahedron
			TArray<FConvex::FVec3Type> HullParticles;
			HullParticles.SetNum(4);
			HullParticles[0] = { -1,-1,-1 };
			HullParticles[1] = { 1,-1,-1 };
			HullParticles[2] = { 0,1,-1 };
			HullParticles[3] = { 0,0,1 };
			FConvex B(HullParticles, 0.0f);

			for (const FVec3& InitialDir : InitialDirs)
			{
				//hit
				EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(5, 0, 0), FRotation3::Identity), 0, InitialDir));

				//near hit
				EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(4 + 1e-4, 1, 1), FRotation3::Identity), 0, InitialDir));

				//near miss
				EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(4 - 1e-2, 1, 1), FRotation3::Identity), 0, InitialDir));

				//rotated hit
				EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(4 + 1e-4, 0, 1), FRotation3::FromVector(FVec3(0, 0, PI*0.5))), 0, InitialDir));

				//rotated miss
				EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(4 - 1e-2, 0, 1), FRotation3::FromVector(FVec3(0, 0, PI*0.5))), 0, InitialDir));

				//rotated and inflated hit
				EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(3.5, 0, 1), FRotation3::FromVector(FVec3(0, 0, PI*0.5))), 0.5 + 1e-4, InitialDir));

				//rotated and inflated miss
				EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(3.5, 0, 1), FRotation3::FromVector(FVec3(0, 0, PI*0.5))), 0.5 - 1e-2, InitialDir));
			}
		}

		{
			//Triangle
			TArray<FConvex::FVec3Type> TriangleParticles;
			TriangleParticles.SetNum(3);
			TriangleParticles[0] = { -1,-1,-1 };
			TriangleParticles[1] = { 1,-1,-1 };
			TriangleParticles[2] = { 0,1,-1 };
			FConvex B(TriangleParticles, 0.0f);

			//triangle
			for (const FVec3& InitialDir : InitialDirs)
			{
				//hit
				EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(5, 0, 0), FRotation3::Identity), 0, InitialDir));

				//near hit
				EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(4 + 1e-2, 1, 1), FRotation3::Identity), 0, InitialDir));

				//near miss
				EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(4 - 1e-2, 1, 1), FRotation3::Identity), 0, InitialDir));

				//rotated hit
				EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(4 + 1e-2, 0, 1), FRotation3::FromVector(FVec3(0, 0, PI*0.5))), 0, InitialDir));

				//rotated miss
				EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(4 - 1e-2, 0, 1), FRotation3::FromVector(FVec3(0, 0, PI*0.5))), 0, InitialDir));

				//rotated and inflated hit
				EXPECT_TRUE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(3.5, 0, 1), FRotation3::FromVector(FVec3(0, 0, PI*0.5))), 0.5 + 1e-2, InitialDir));

				//rotated and inflated miss
				EXPECT_FALSE(GJKIntersection<FReal>(A, B, FRigidTransform3(FVec3(3.5, 0, 1), FRotation3::FromVector(FVec3(0, 0, PI*0.5))), 0.5 - 1e-2, InitialDir));
			}
		}
	}


	void GJKConvexCapsule()
	{
		// The following tests are coming form a bug found in EPAComputeVisibilityBorder. The fix was adding an epsilon test, instead of a hard zero comparison.
		TArray<FConvex::FVec3Type> HullParticles;
		HullParticles.SetNum(12);
		HullParticles[0] = { 100.000000, -460.000000, -100.000000 };
		HullParticles[1] = { 100.000000, -460.000000, 100.000000 };
		HullParticles[2] = { 400.000000, 0.00000000, 100.000000 };
		HullParticles[3] = { 400.000000, 0.00000000, -100.000000 };
		HullParticles[4] = { -100.000000, 390.000000, 100.000000 };
		HullParticles[5] = { -500.000000, 0.00000000, 100.000000 };
		HullParticles[6] = { -500.000000, 0.00000000, -100.000000 };
		HullParticles[7] = { -100.000000, 390.000000, -100.000000 };
		HullParticles[8] = { -100.000000, -460.000000, 100.000000 };
		HullParticles[9] = { 100.000000, 390.000000, 100.000000 };
		HullParticles[10] = { 100.000000, 390.000000, -100.000000 };
		HullParticles[11] = { -100.000000, -460.000000, -100.000000 };
		FConvex Convex(HullParticles, 0.0);
		TImplicitObjectScaled<FConvex> ImplicitScaledConvex(&Convex, FVec3(2.00000000, 2.00000000, 2.00000000));
		FCapsule Capsule(FVec3(0.00000000, 0.00000000, -55.0000000), FVec3(0.00000000, 0.00000000, 55.00000000), 35.000000);
		
		FReal Thickness = 0;
		bool bComputeMTD = true;

		FVec3 LocalPosition, LocalNormal;
		FReal OutTime = -1;
		{
			FRigidTransform3 BToAFullTM(FVec3(-23.634607186206267, 18.980813439935446, 2.2124981121014571),
				FRotation3(FQuat(0.0000000000000000, 0.0000000000000000, -0.57785636374940896, 0.81613848265739242)));
			FVec3 LocalDir(0.39419760716989327, -0.91902570906429382, 0.0000000000000000);
			FVec3 Offset(14.467136106672115, 2.3920753243291983, -2.2124981121014571);
			FReal Length = 6.9428286552429199;
			bool bIsHitting = GJKRaycast2<FReal>(ImplicitScaledConvex, Capsule, BToAFullTM, LocalDir, Length, OutTime, LocalPosition, LocalNormal, Thickness, bComputeMTD, Offset, Thickness);
			EXPECT_TRUE(bIsHitting);
			EXPECT_LT(OutTime, 0.0);
		}
		{
			FRigidTransform3 BToAFullTM(FVec3(27.711238420990412, -2.1483584435627563, 2.0126037597656250),
				FRotation3(FQuat(0.0000000000000000, 0.0000000000000000, 0.12793015790484025, 0.99178317927783100)));
			FVec3 LocalDir(0.96726777078778936, 0.25375796307645249, 0.0000000000000000);
			FVec3 Offset(-27.711238420990412, 2.1483584435627563, -2.0126037597656250);
			FReal Length = 15.497749328613281;
			bool bIsHitting = GJKRaycast2<FReal>(ImplicitScaledConvex, Capsule, BToAFullTM, LocalDir, Length, OutTime, LocalPosition, LocalNormal, Thickness, bComputeMTD, Offset, Thickness);
			EXPECT_TRUE(bIsHitting);
			EXPECT_LT(OutTime, 0.0);
		}
		{
			FRigidTransform3 BToAFullTM(FVec3(-22.297271080315113, 30.859426572671509, 2.2125010768795619),
			FRotation3(FQuat(0.0000000000000000, 0.0000000000000000, 0.076283974503952301, 0.99708613230446663)));
			FVec3 LocalDir(0.59685429204739793, 0.091864661873730533, 0.79707334589105316);
			FReal Length = 38.664058685302734;
			FVec3 Offset(22.297271080315113, -30.859426572671509, -2.2125010768795619);
			bool bIsHitting = GJKRaycast2<FReal>(ImplicitScaledConvex, Capsule, BToAFullTM, LocalDir, Length, OutTime, LocalPosition, LocalNormal, Thickness, bComputeMTD, Offset, Thickness);
			EXPECT_TRUE(bIsHitting);
			EXPECT_LT(OutTime, 0.0);
		}
	}


	void GJKSphereScaledSphereTest()
	{
		Chaos::FSphere A(FVec3(10, 0, 0), 5);
		FSpherePtr Sphere( new Chaos::FSphere(FVec3(4, 0, 0), 2));
		TImplicitObjectScaled<Chaos::FSphere> Unscaled(Sphere, FVec3(1));
		TImplicitObjectScaled<Chaos::FSphere> UniformScaled(Sphere, FVec3(2));
		TImplicitObjectScaled<Chaos::FSphere> NonUniformScaled(Sphere, FVec3(2,1,1));

		FVec3 InitialDirs[] = { FVec3(1,0,0), FVec3(-1,0,0), FVec3(0,1,0), FVec3(0,-1,0), FVec3(0,0,1), FVec3(0,0,-1) };

		for (const FVec3& InitialDir : InitialDirs)
		{
			EXPECT_TRUE(GJKIntersection<FReal>(A, Unscaled, FRigidTransform3::Identity, 0, InitialDir));
			EXPECT_TRUE(GJKIntersection<FReal>(A, UniformScaled, FRigidTransform3::Identity, 0, InitialDir));
			//EXPECT_TRUE(GJKIntersection<FReal>(A, NonUniformScaled, FRigidTransform3::Identity, 0, InitialDir));

			//miss
			EXPECT_FALSE(GJKIntersection<FReal>(A, Unscaled, FRigidTransform3(FVec3(-1.1, 0, 0), FRotation3::Identity), 0, InitialDir));
			EXPECT_FALSE(GJKIntersection<FReal>(A, UniformScaled, FRigidTransform3(FVec3(-7.1, 0, 0), FRotation3::Identity), 0, InitialDir));
			//EXPECT_FALSE(GJKIntersection<FReal>(A, NonUniformScaled, FRigidTransform3(FVec3(-7.1, 0, 0), FRotation3::Identity), 0, InitialDir));

			//hit from thickness
			EXPECT_TRUE(GJKIntersection<FReal>(A, Unscaled, FRigidTransform3(FVec3(-1.1, 0, 0), FRotation3::Identity), 0.105, InitialDir));
			EXPECT_TRUE(GJKIntersection<FReal>(A, UniformScaled, FRigidTransform3(FVec3(-7.1, 0, 0), FRotation3::Identity), 0.105, InitialDir));
			//EXPECT_TRUE(GJKIntersection<FReal>(A, NonUniformScaled, FRigidTransform3(FVec3(-7.1, 0, 0), FRotation3::Identity), 0.105, InitialDir));

			//miss with thickness
			EXPECT_FALSE(GJKIntersection<FReal>(A, Unscaled, FRigidTransform3(FVec3(-1.1, 0, 0), FRotation3::Identity), 0.095, InitialDir));
			EXPECT_FALSE(GJKIntersection<FReal>(A, UniformScaled, FRigidTransform3(FVec3(-7.1, 0, 0), FRotation3::Identity), 0.095, InitialDir));
			//EXPECT_FALSE(GJKIntersection<FReal>(A, NonUniformScaled, FRigidTransform3(FVec3(-7.1, 0, 0), FRotation3::Identity), 0.095, InitialDir));

			//hit with rotation
			EXPECT_TRUE(GJKIntersection<FReal>(A, Unscaled, FRigidTransform3(FVec3(6.5, 0, 0), FRotation3::FromVector(FVec3(0, 0, PI))), 1, InitialDir));
			EXPECT_TRUE(GJKIntersection<FReal>(A, UniformScaled, FRigidTransform3(FVec3(8.1, 0, 0), FRotation3::FromVector(FVec3(0, 0, PI))), 1, InitialDir));
			//EXPECT_TRUE(GJKIntersection<FReal>(A, NonUniformScaled, FRigidTransform3(FVec3(8.1, 0, 0), FRotation3::FromVector(FVec3(0, 0, PI))), 1, InitialDir));

			//miss with rotation
			EXPECT_FALSE(GJKIntersection<FReal>(A, Unscaled, FRigidTransform3(FVec3(6.5, 0, 0), FRotation3::FromVector(FVec3(0, 0, PI))), 0.01, InitialDir));
			EXPECT_FALSE(GJKIntersection<FReal>(A, UniformScaled, FRigidTransform3(FVec3(8.1, 0, 0), FRotation3::FromVector(FVec3(0, 0, PI))), 0.01, InitialDir));
			//EXPECT_FALSE(GJKIntersection<FReal>(A, NonUniformScaled, FRigidTransform3(FVec3(8.1, 0, 0), FRotation3::FromVector(FVec3(0, 0, PI))), 0.01, InitialDir));
		}
	}

	//For each gjkraycast test we should test:
	// - thickness
	// - initial overlap
	// - transformed geometry
	// - rotated geometry
	// - offset transform
	// - degenerate cases
	// - near miss, near hit
	// - multiple initial dir

	void GJKSphereSphereSweep()
	{
		Chaos::FSphere A(FVec3(10, 0, 0), 5);
		Chaos::FSphere B(FVec3(1, 0, 0), 2);

		FVec3 InitialDirs[] = { FVec3(1,0,0), FVec3(-1,0,0), FVec3(0,1,0), FVec3(0,-1,0), FVec3(0,0,1), FVec3(0,0,-1) };

		constexpr FReal Eps = 1e-1;

		for (const FVec3& InitialDir : InitialDirs)
		{
			FReal Time;
			FVec3 Position;
			FVec3 Normal;

			//hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3::Identity, FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			//hit offset
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(1,0,0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 1, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			//initial overlap
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(7, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, false, InitialDir));
			EXPECT_FLOAT_EQ(Time, 0);

			//MTD
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(7, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, true, InitialDir));
			EXPECT_FLOAT_EQ(Time, -5);
			EXPECT_VECTOR_NEAR(Position, FVec3(5,0,0), Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			
			//EPA
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(9, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, true, InitialDir));
			EXPECT_FLOAT_EQ(Time, -7);	//perfect overlap, will default to 0,0,1 normal
			EXPECT_VECTOR_NEAR(Position, FVec3(10,0,5), Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(0, 0, 1), Eps);

			//miss
			EXPECT_FALSE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//hit with thickness
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0.2, InitialDir));

			//hit rotated
			const FRotation3 RotatedDown(FRotation3::FromVector(FVec3(0, PI * 0.5, 0)));
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7.9), RotatedDown), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//miss rotated
			EXPECT_FALSE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 8.1), RotatedDown), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//hit rotated with inflation
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7.9), RotatedDown), FVec3(1, 0, 0), 20, Time, Position, Normal, 0.2, InitialDir));

			//near hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7 - 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//near miss
			EXPECT_FALSE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7 + 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//degenerate
			Chaos::FSphere Tiny(FVec3(1, 0, 0), 1e-8);
			EXPECT_TRUE(GJKRaycast<FReal>(A, Tiny, FRigidTransform3::Identity, FVec3(1, 0, 0), 8, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 4, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			//right at end
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3::Identity, FVec3(1, 0, 0), 2, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);

			// not far enough
			EXPECT_FALSE(GJKRaycast<FReal>(A, B, FRigidTransform3::Identity, FVec3(1, 0, 0), 2 - 1e-2, Time, Position, Normal, 0, InitialDir));
		}
	}

	void GJKSphereBoxSweep()
	{
		FAABB3 A(FVec3(3, -1, 0), FVec3(4, 1, 4));
		Chaos::FSphere B(FVec3(0, 0, 0), 1);

		FVec3 InitialDirs[] = { FVec3(1,0,0), FVec3(-1,0,0), FVec3(0,1,0), FVec3(0,-1,0), FVec3(0,0,1), FVec3(0,0,-1) };

		constexpr FReal Eps = 1e-1;

		for (const FVec3& InitialDir : InitialDirs)
		{
			FReal Time(0.0);
			FVec3 Position(0.0);
			FVec3 Normal(0.0);

			//hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3::Identity, FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(3, 0, 0), Eps);

			//hit offset
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(1.5, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 0.5, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(3, 0, 0), Eps);

			//initial overlap
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(4, 0, 4), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, false, InitialDir));
			EXPECT_FLOAT_EQ(Time, 0);

			//MTD without EPA
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(4.25, 0, 2), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, true, InitialDir));
			EXPECT_FLOAT_EQ(Time, -0.75);
			EXPECT_VECTOR_NEAR(Position, FVec3(4, 0, 2), Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(1, 0, 0), Eps);

			//MTD with EPA
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(4, 0, 2), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, true, InitialDir));
			EXPECT_FLOAT_EQ(Time, -1);
			EXPECT_VECTOR_NEAR(Position, FVec3(4, 0, 2), Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(1, 0, 0), Eps);

			//MTD with EPA
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(3.25, 0, 2), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, true, InitialDir));
			EXPECT_FLOAT_EQ(Time, -1.25);
			EXPECT_VECTOR_NEAR(Position, FVec3(3, 0, 2), Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);

			//MTD with EPA
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(3.4, 0, 3.75), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, true, InitialDir));
			EXPECT_FLOAT_EQ(Time, -1.25);
			EXPECT_VECTOR_NEAR(Position, FVec3(3.4, 0, 4), Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(0, 0, 1), Eps);

			//hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(1, 0, 6), FRotation3::Identity), FVec3(1, 0, -1).GetUnsafeNormal(), 4, Time, Position, Normal, 0, InitialDir));
			const FReal ExpectedTime = ((FVec3(3, 0, 4) - FVec3(1, 0, 6)).Size() - 1);
			EXPECT_NEAR(Time, ExpectedTime, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-sqrt(2) / 2, 0, sqrt(2) / 2), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(3, 0, 4), Eps);

			//near miss
			EXPECT_FALSE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 5+1e-2), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));

			//near hit with inflation
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 5 + 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 2e-2, InitialDir));
			const FReal DistanceFromCorner = (Position - FVec3(3, 0, 4)).Size();
			EXPECT_LT(DistanceFromCorner, 1e-1);

			//rotated box
			const FRotation3 Rotated(FRotation3::FromVector(FVec3(0, 0, PI * 0.5)));
			EXPECT_TRUE(GJKRaycast<FReal>(B, A, FRigidTransform3(FVec3(0), Rotated), FVec3(0, -1, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(0, 1, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(0, 1, 0), Eps);

			//degenerate box
			FAABB3 Needle(FVec3(3, 0, 0), FVec3(4, 0, 0));
			EXPECT_TRUE(GJKRaycast<FReal>(B, Needle, FRigidTransform3(FVec3(0), Rotated), FVec3(0, -1, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(0, 1, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(0, 1, 0), Eps);
		}
	}


	void GJKSphereCapsuleSweep()
	{
		Chaos::FSphere A(FVec3(10, 0, 0), 5);
		FCapsule B(FVec3(1, 0, 0), FVec3(-3, 0, 0), 2);

		FVec3 InitialDirs[] = { FVec3(1,0,0), FVec3(-1,0,0), FVec3(0,1,0), FVec3(0,-1,0), FVec3(0,0,1), FVec3(0,0,-1) };

		constexpr FReal Eps = 1e-1;

		for (const FVec3& InitialDir : InitialDirs)
		{
			FReal Time;
			FVec3 Position;
			FVec3 Normal;

			//hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3::Identity, FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			//hit offset
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(1, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 1, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);
			
			//initial overlap
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(7, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, false, InitialDir));
			EXPECT_FLOAT_EQ(Time, 0);

			//MTD
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(7, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, true, InitialDir));
			EXPECT_FLOAT_EQ(Time, -5);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);

			//miss
			EXPECT_FALSE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//hit with thickness
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0.2, InitialDir));

			//hit rotated
			const FRotation3 RotatedDown(FRotation3::FromVector(FVec3(0, PI * 0.5, 0)));
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7.9), RotatedDown), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//miss rotated
			EXPECT_FALSE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 8.1), RotatedDown), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//hit rotated with inflation
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7.9), RotatedDown), FVec3(1, 0, 0), 20, Time, Position, Normal, 0.2, InitialDir));

			//near hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7 - 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//near miss
			EXPECT_FALSE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7 + 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//degenerate
			Chaos::FSphere Tiny(FVec3(1, 0, 0), 1e-8);
			EXPECT_TRUE(GJKRaycast<FReal>(A, Tiny, FRigidTransform3::Identity, FVec3(1, 0, 0), 8, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 4, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			//right at end
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3::Identity, FVec3(1, 0, 0), 2, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);

			// not far enough
			EXPECT_FALSE(GJKRaycast<FReal>(A, B, FRigidTransform3::Identity, FVec3(1, 0, 0), 2 - 1e-2, Time, Position, Normal, 0, InitialDir));
		}
	}


	void GJKSphereConvexSweep()
	{
		//Tetrahedron
		TArray<FConvex::FVec3Type> HullParticles;
		HullParticles.SetNum(4);
		HullParticles[0] = { 3,0,4 };
		HullParticles[1] = { 3,1,0 };
		HullParticles[2] = { 3,-1,0 };
		HullParticles[3] = { 4,0,2 };
		FConvex A(HullParticles, 0.0f);
		Chaos::FSphere B(FVec3(0, 0, 0), 1);

		FVec3 InitialDirs[] = { FVec3(1,0,0), FVec3(-1,0,0), FVec3(0,1,0), FVec3(0,-1,0), FVec3(0,0,1), FVec3(0,0,-1) };

		constexpr FReal Eps = 1e-1;

		for (const FVec3& InitialDir : InitialDirs)
		{
			FReal Time;
			FVec3 Position(0);
			FVec3 Normal;

			//hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3::Identity, FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(3, 0, 0), Eps);

			//hit offset
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(1.5, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 0.5, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(3, 0, 0), Eps);

			//initial overlap
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(4, 0, 4), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, false, InitialDir));
			EXPECT_FLOAT_EQ(Time, 0);

			//MTD
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(2.5, 0, 2), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, true, InitialDir));
			EXPECT_FLOAT_EQ(Time, -0.5);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0).GetUnsafeNormal(), Eps);

			//MTD
			FReal Penetration;
			FVec3 ClosestA, ClosestB;
			int32 ClosestVertexIndexA, ClosestVertexIndexB;
			EXPECT_TRUE((GJKPenetration<false, FReal>(A, B, FRigidTransform3(FVec3(2.5, 0, 2), FRotation3::Identity), Penetration, ClosestA, ClosestB, Normal, ClosestVertexIndexA, ClosestVertexIndexB, 0, 0, InitialDir)));
			EXPECT_FLOAT_EQ(Penetration, 0.5);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0).GetUnsafeNormal(), Eps);
			EXPECT_NEAR(ClosestA[0], 3, Eps);	//could be any point on face, but should have x == 3
			EXPECT_VECTOR_NEAR(ClosestB, FVec3(3.5, 0, 2), Eps);

			//hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(1, 0, 6), FRotation3::Identity), FVec3(1, 0, -1).GetUnsafeNormal(), 4, Time, Position, Normal, 0, InitialDir));
			const FReal ExpectedTime = ((FVec3(3, 0, 4) - FVec3(1, 0, 6)).Size() - 1);
			EXPECT_NEAR(Time, ExpectedTime, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-sqrt(2) / 2, 0, sqrt(2) / 2), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(3, 0, 4), Eps);

			//near miss
			EXPECT_FALSE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 5 + 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));

			//near hit with inflation
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 5 + 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 2e-2, InitialDir));
			const FReal DistanceFromCorner = (Position - FVec3(3, 0, 4)).Size();
			EXPECT_LT(DistanceFromCorner, 1e-1);

			//rotated box
			const FRotation3 Rotated(FRotation3::FromVector(FVec3(0, 0, PI * 0.5)));
			EXPECT_TRUE(GJKRaycast<FReal>(B, A, FRigidTransform3(FVec3(0), Rotated), FVec3(0, -1, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);
			EXPECT_NEAR(Normal.X, 0, Eps);
			EXPECT_NEAR(Normal.Y, 1, Eps);
			//EXPECT_NEAR(Normal.Z, 0, Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(0, 1, 0), Eps);

			//degenerate box
			FAABB3 Needle(FVec3(3, 0, 0), FVec3(4, 0, 0));
			EXPECT_TRUE(GJKRaycast<FReal>(B, Needle, FRigidTransform3(FVec3(0), Rotated), FVec3(0, -1, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(0, 1, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(0, 1, 0), Eps);
		}
	}

	void GJKSphereScaledSphereSweep()
	{
		Chaos::FSphere A(FVec3(10, 0, 0), 5);
		FSpherePtr Sphere( new Chaos::FSphere(FVec3(0, 0, 0), 2));
		TImplicitObjectScaled<Chaos::FSphere> Unscaled(Sphere, FVec3(1));
		TImplicitObjectScaled<Chaos::FSphere> UniformScaled(Sphere, FVec3(2));
		TImplicitObjectScaled<Chaos::FSphere> NonUniformScaled(Sphere, FVec3(2, 1, 1));

		FVec3 InitialDirs[] = { FVec3(1,0,0), FVec3(-1,0,0), FVec3(0,1,0), FVec3(0,-1,0), FVec3(0,0,1), FVec3(0,0,-1) };

		constexpr FReal Eps = 1e-1;

		for (const FVec3& InitialDir : InitialDirs)
		{
			FReal Time;
			FVec3 Position;
			FVec3 Normal;

			//hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, Unscaled, FRigidTransform3::Identity, FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 3, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			EXPECT_TRUE(GJKRaycast<FReal>(A, UniformScaled, FRigidTransform3::Identity, FVec3(1, 0, 0), 6, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 1, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			EXPECT_TRUE(GJKRaycast<FReal>(A, NonUniformScaled, FRigidTransform3::Identity, FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 1, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			//hit offset
			EXPECT_TRUE(GJKRaycast<FReal>(A, Unscaled, FRigidTransform3(FVec3(1, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			EXPECT_TRUE(GJKRaycast<FReal>(A, UniformScaled, FRigidTransform3(FVec3(1, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 0, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			EXPECT_TRUE(GJKRaycast<FReal>(A, NonUniformScaled, FRigidTransform3(FVec3(1, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 0, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			//initial overlap
			EXPECT_TRUE(GJKRaycast<FReal>(A, Unscaled, FRigidTransform3(FVec3(8, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_FLOAT_EQ(Time, 0);
			EXPECT_TRUE(GJKRaycast<FReal>(A, UniformScaled, FRigidTransform3(FVec3(6, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_FLOAT_EQ(Time, 0);
			EXPECT_TRUE(GJKRaycast<FReal>(A, NonUniformScaled, FRigidTransform3(FVec3(6, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_FLOAT_EQ(Time, 0);

			//miss
			EXPECT_FALSE(GJKRaycast<FReal>(A, Unscaled, FRigidTransform3(FVec3(0, 0, 7.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));
			EXPECT_FALSE(GJKRaycast<FReal>(A, UniformScaled, FRigidTransform3(FVec3(0, 0, 9.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));
			EXPECT_FALSE(GJKRaycast<FReal>(A, NonUniformScaled, FRigidTransform3(FVec3(0, 0, 7.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//hit with thickness
			EXPECT_TRUE(GJKRaycast<FReal>(A, Unscaled, FRigidTransform3(FVec3(0, 0, 7.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0.2, InitialDir));
			EXPECT_TRUE(GJKRaycast<FReal>(A, UniformScaled, FRigidTransform3(FVec3(0, 0, 9.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0.2, InitialDir));
			EXPECT_TRUE(GJKRaycast<FReal>(A, NonUniformScaled, FRigidTransform3(FVec3(0, 0, 7.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0.2, InitialDir));

			//hit rotated
			const FRotation3 RotatedInPlace(FRotation3::FromVector(FVec3(0, PI * 0.5, 0)));
			EXPECT_TRUE(GJKRaycast<FReal>(A, Unscaled, FRigidTransform3(FVec3(0, 0, 0), RotatedInPlace), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));
			EXPECT_TRUE(GJKRaycast<FReal>(A, UniformScaled, FRigidTransform3(FVec3(0, 0, 0), RotatedInPlace), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));
			EXPECT_TRUE(GJKRaycast<FReal>(A, NonUniformScaled, FRigidTransform3(FVec3(0, 0, 0), RotatedInPlace), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//miss rotated
			EXPECT_FALSE(GJKRaycast<FReal>(A, Unscaled, FRigidTransform3(FVec3(0, 0, 7.1), RotatedInPlace), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));
			EXPECT_FALSE(GJKRaycast<FReal>(A, UniformScaled, FRigidTransform3(FVec3(0, 0, 9.1), RotatedInPlace), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));
			EXPECT_FALSE(GJKRaycast<FReal>(A, NonUniformScaled, FRigidTransform3(FVec3(0, 0, 9.1), RotatedInPlace), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//near hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, Unscaled, FRigidTransform3(FVec3(0, 0, 7 - 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//near miss
			EXPECT_FALSE(GJKRaycast<FReal>(A, Unscaled, FRigidTransform3(FVec3(0, 0, 7 + 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//degenerate
			Chaos::FSphere Tiny(FVec3(1, 0, 0), 1e-8);
			EXPECT_TRUE(GJKRaycast<FReal>(A, Tiny, FRigidTransform3::Identity, FVec3(1, 0, 0), 8, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 4, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			//right at end
			EXPECT_TRUE(GJKRaycast<FReal>(A, Unscaled, FRigidTransform3::Identity, FVec3(1, 0, 0), 3, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 3, Eps);

			// not far enough
			EXPECT_FALSE(GJKRaycast<FReal>(A, Unscaled, FRigidTransform3::Identity, FVec3(1, 0, 0), 3 - 1e-2, Time, Position, Normal, 0, InitialDir));
		}
	}


	void GJKSphereTransformedSphereSweep()
	{
		Chaos::FSphere A(FVec3(10, 0, 0), 5);

		Chaos::FSphere Sphere(FVec3(0), 2);
		Chaos::FSphere Translated(Sphere.GetCenterf() + FVec3f(1, 0, 0), Sphere.GetRadiusf());
		Chaos::FSphere Transformed(FRigidTransform3(FVec3(1, 0, 0), FRotation3::FromVector(FVec3(0, 0, PI))).TransformPosition(FVec3(Sphere.GetCenterf())), Sphere.GetRadiusf());

		FVec3 InitialDirs[] = { FVec3(1,0,0), FVec3(-1,0,0), FVec3(0,1,0), FVec3(0,-1,0), FVec3(0,0,1), FVec3(0,0,-1) };

		constexpr FReal Eps = 1e-1;

		for (const FVec3& InitialDir : InitialDirs)
		{
			FReal Time;
			FVec3 Position;
			FVec3 Normal;

			//hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, Translated, FRigidTransform3::Identity, FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);
			EXPECT_TRUE(GJKRaycast<FReal>(A, Transformed, FRigidTransform3::Identity, FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			//hit offset
			EXPECT_TRUE(GJKRaycast<FReal>(A, Translated, FRigidTransform3(FVec3(1, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 1, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);
			EXPECT_TRUE(GJKRaycast<FReal>(A, Transformed, FRigidTransform3(FVec3(1, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 1, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Position, FVec3(5, 0, 0), Eps);

			//initial overlap
			EXPECT_TRUE(GJKRaycast<FReal>(A, Translated, FRigidTransform3(FVec3(7, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_FLOAT_EQ(Time, 0);
			EXPECT_TRUE(GJKRaycast<FReal>(A, Transformed, FRigidTransform3(FVec3(7, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_FLOAT_EQ(Time, 0);

			//miss
			EXPECT_FALSE(GJKRaycast<FReal>(A, Translated, FRigidTransform3(FVec3(0, 0, 7.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));
			EXPECT_FALSE(GJKRaycast<FReal>(A, Transformed, FRigidTransform3(FVec3(0, 0, 7.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//hit with thickness
			EXPECT_TRUE(GJKRaycast<FReal>(A, Translated, FRigidTransform3(FVec3(0, 0, 7.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0.2, InitialDir));
			EXPECT_TRUE(GJKRaycast<FReal>(A, Transformed, FRigidTransform3(FVec3(0, 0, 7.1), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0.2, InitialDir));

			//hit rotated
			const FRotation3 RotatedDown(FRotation3::FromVector(FVec3(0, PI * 0.5, 0)));
			EXPECT_TRUE(GJKRaycast<FReal>(A, Translated, FRigidTransform3(FVec3(0, 0, 7.9), RotatedDown), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));
			EXPECT_TRUE(GJKRaycast<FReal>(A, Transformed, FRigidTransform3(FVec3(0, 0, 7.9), RotatedDown), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//miss rotated
			EXPECT_FALSE(GJKRaycast<FReal>(A, Translated, FRigidTransform3(FVec3(0, 0, 8.1), RotatedDown), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));
			EXPECT_FALSE(GJKRaycast<FReal>(A, Transformed, FRigidTransform3(FVec3(0, 0, 8.1), RotatedDown), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//hit rotated with inflation
			EXPECT_TRUE(GJKRaycast<FReal>(A, Translated, FRigidTransform3(FVec3(0, 0, 7.9), RotatedDown), FVec3(1, 0, 0), 20, Time, Position, Normal, 0.2, InitialDir));
			EXPECT_TRUE(GJKRaycast<FReal>(A, Transformed, FRigidTransform3(FVec3(0, 0, 7.9), RotatedDown), FVec3(1, 0, 0), 20, Time, Position, Normal, 0.2, InitialDir));

			//near hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, Translated, FRigidTransform3(FVec3(0, 0, 7 - 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));
			EXPECT_TRUE(GJKRaycast<FReal>(A, Transformed, FRigidTransform3(FVec3(0, 0, 7 - 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//near miss
			EXPECT_FALSE(GJKRaycast<FReal>(A, Translated, FRigidTransform3(FVec3(0, 0, 7 + 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));
			EXPECT_FALSE(GJKRaycast<FReal>(A, Transformed, FRigidTransform3(FVec3(0, 0, 7 + 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 20, Time, Position, Normal, 0, InitialDir));

			//right at end
			EXPECT_TRUE(GJKRaycast<FReal>(A, Translated, FRigidTransform3::Identity, FVec3(1, 0, 0), 2, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);
			EXPECT_TRUE(GJKRaycast<FReal>(A, Transformed, FRigidTransform3::Identity, FVec3(1, 0, 0), 2, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 2, Eps);

			// not far enough
			EXPECT_FALSE(GJKRaycast<FReal>(A, Translated, FRigidTransform3::Identity, FVec3(1, 0, 0), 2 - 1e-2, Time, Position, Normal, 0, InitialDir));
			EXPECT_FALSE(GJKRaycast<FReal>(A, Transformed, FRigidTransform3::Identity, FVec3(1, 0, 0), 2 - 1e-2, Time, Position, Normal, 0, InitialDir));
		}
	}


	void GJKBoxCapsuleSweep()
	{
		FAABB3 A(FVec3(3, -1, 0), FVec3(4, 1, 4));
		FCapsule B(FVec3(0, 0, -1), FVec3(0, 0, 1), 2);

		FVec3 InitialDirs[] = { FVec3(1,0,0), FVec3(-1,0,0), FVec3(0,1,0), FVec3(0,-1,0), FVec3(0,0,1), FVec3(0,0,-1) };

		constexpr FReal Eps = 1e-1;

		for (const FVec3& InitialDir : InitialDirs)
		{
			FReal Time;
			FVec3 Position;
			FVec3 Normal;

			//hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3::Identity, FVec3(1, 0, 0), 2, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 1, Eps);
			EXPECT_NEAR(Normal.X, -1, Eps);
			EXPECT_NEAR(Normal.Y, 0, Eps);
			EXPECT_NEAR(Normal.Z, 0, Eps);
			EXPECT_NEAR(Position.X, 3, Eps);
			//EXPECT_NEAR(Position.Y, 0, Eps);	//todo: look into inaccuracy here (0.015) instead of <1e-2
			EXPECT_LE(Position.Z, 1 + Eps);
			EXPECT_GE(Position.Z, -1 - Eps);

			//hit offset
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0.5, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 0.5, Eps);
			EXPECT_NEAR(Normal.X, -1, Eps);
			EXPECT_NEAR(Normal.Y, 0, Eps);
			EXPECT_NEAR(Normal.Z, 0, Eps);
			EXPECT_NEAR(Position.X, 3, Eps);
			//EXPECT_NEAR(Position.Y, 0, Eps);	//todo: look into inaccuracy here (0.015) instead of <1e-2
			EXPECT_LE(Position.Z, 1 + Eps);
			EXPECT_GE(Position.Z, -1 - Eps);

			//initial overlap
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(3, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 2, Time, Position, Normal, 0, false, InitialDir));
			EXPECT_FLOAT_EQ(Time, 0);

			//MTD
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(2.5, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 2, Time, Position, Normal, 0, true, InitialDir));
			EXPECT_FLOAT_EQ(Time, -1.5);
			EXPECT_NEAR(Position[0], 3, Eps);	//many possible, but x must be on 3
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);

			//MTD
			FReal Penetration;
			FVec3 ClosestA, ClosestB;
			int32 ClosestVertexIndexA, ClosestVertexIndexB;
			EXPECT_TRUE((GJKPenetration<false, FReal>(A, B, FRigidTransform3(FVec3(2.5, 0, 0), FRotation3::Identity), Penetration, ClosestA, ClosestB, Normal, ClosestVertexIndexA, ClosestVertexIndexB, 0, 0, InitialDir)));
			EXPECT_FLOAT_EQ(Penetration, 1.5);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_NEAR(ClosestA[0], 3, Eps);	//could be any point on face, but should have x == 3
			EXPECT_NEAR(ClosestB[0], 4.5, Eps);
			EXPECT_NEAR(ClosestB[1], 0, Eps);

			//EPA
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(3, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 2, Time, Position, Normal, 0, true, InitialDir));
			EXPECT_FLOAT_EQ(Time, -2);
			EXPECT_NEAR(Position[0], 3, Eps);	//many possible, but x must be on 3
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);

			//EPA
			EXPECT_TRUE((GJKPenetration<false, FReal>(A, B, FRigidTransform3(FVec3(3, 0, 0), FRotation3::Identity), Penetration, ClosestA, ClosestB, Normal, ClosestVertexIndexA, ClosestVertexIndexB, 0, 0, InitialDir)));
			EXPECT_NEAR(Penetration, 2, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_NEAR(ClosestA[0], 3, Eps);	//could be any point on face, but should have x == 3
			EXPECT_NEAR(ClosestB[0], 5, Eps);
			EXPECT_NEAR(ClosestB[1], 0, Eps);

			//EPA
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(3.25, 0, 0), FRotation3::Identity), FVec3(1, 0, 0), 2, Time, Position, Normal, 0, true, InitialDir));
			EXPECT_FLOAT_EQ(Time, -2.25);
			EXPECT_NEAR(Position[0], 3, Eps);	//many possible, but x must be on 3
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);

			//EPA
			EXPECT_TRUE((GJKPenetration<false, FReal>(A, B, FRigidTransform3(FVec3(3.25, 0, 0), FRotation3::Identity), Penetration, ClosestA, ClosestB, Normal, ClosestVertexIndexA, ClosestVertexIndexB, 0, 0, InitialDir)));
			EXPECT_NEAR(Penetration, 2.25, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), Eps);
			EXPECT_NEAR(ClosestA[0], 3, Eps);	//could be any point on face, but should have x == 3
			EXPECT_NEAR(ClosestB[0], 5.25, Eps);
			EXPECT_NEAR(ClosestB[1], 0, Eps);

			//MTD
			EXPECT_TRUE(GJKRaycast2<FReal>(A, B, FRigidTransform3(FVec3(3.25, 0, -2.875), FRotation3::Identity), FVec3(1, 0, 0), 2, Time, Position, Normal, 0, true, InitialDir));
			EXPECT_FLOAT_EQ(Time, -0.125);
			EXPECT_VECTOR_NEAR(Position, FVec3(3.25, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(0, 0, -1), Eps);

			//MTD
			EXPECT_TRUE((GJKPenetration<false, FReal>(A, B, FRigidTransform3(FVec3(3.25, 0, -2.875), FRotation3::Identity), Penetration, ClosestA, ClosestB, Normal, ClosestVertexIndexA, ClosestVertexIndexB, 0, 0, InitialDir)));
			EXPECT_NEAR(Penetration, 0.125, Eps);
			EXPECT_VECTOR_NEAR(Normal, FVec3(0, 0, -1), Eps);
			EXPECT_VECTOR_NEAR(ClosestA, FVec3(3.25, 0, 0), Eps);
			EXPECT_VECTOR_NEAR(ClosestB, FVec3(3.25, 0, 0.125), Eps);

			//near miss
			EXPECT_FALSE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7 + 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));

			//near hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7 - 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Position.X, 3, Eps);
			EXPECT_NEAR(Position.Z, 4, 10 * Eps);

			//near hit inflation
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 7 - 1e-2), FRotation3::Identity), FVec3(1, 0, 0), 4, Time, Position, Normal, 2e-2, InitialDir));
			EXPECT_NEAR(Position.X, 3, Eps);
			EXPECT_NEAR(Position.Z, 4, 10 * Eps);

			//rotation hit
			FRotation3 Rotated(FRotation3::FromVector(FVec3(0, -PI * 0.5, 0)));
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(-0.5, 0, 0), Rotated), FVec3(1, 0, 0), 1, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 0.5, Eps);
			EXPECT_NEAR(Position.X, 3, Eps);
			EXPECT_NEAR(Normal.X, -1, Eps);
			EXPECT_NEAR(Normal.Y, 0, Eps);
			EXPECT_NEAR(Normal.Z, 0, Eps);

			//rotation near hit
			EXPECT_TRUE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 6 - 1e-2), Rotated), FVec3(1, 0, 0), 10, Time, Position, Normal, 0, InitialDir));

			//rotation near miss
			EXPECT_FALSE(GJKRaycast<FReal>(A, B, FRigidTransform3(FVec3(0, 0, 6 + 1e-2), Rotated), FVec3(1, 0, 0), 10, Time, Position, Normal, 0, InitialDir));

			//degenerate capsule
			FCapsule Needle(FVec3(0, 0, -1), FVec3(0, 0, 1), 1e-8);
			EXPECT_TRUE(GJKRaycast<FReal>(A, Needle, FRigidTransform3::Identity, FVec3(1, 0, 0), 6, Time, Position, Normal, 0, InitialDir));
			EXPECT_NEAR(Time, 3, Eps);
			EXPECT_NEAR(Normal.X, -1, Eps);
			EXPECT_NEAR(Normal.Y, 0, Eps);
			EXPECT_NEAR(Normal.Z, 0, Eps);
			EXPECT_NEAR(Position.X, 3, Eps);
			//EXPECT_NEAR(Position.Y, 0, Eps);	//todo: look into inaccuracy here (0.015) instead of <1e-2
			EXPECT_LE(Position.Z, 1 + Eps);
			EXPECT_GE(Position.Z, -1 - Eps);
		}
	}

	void GJKBoxBoxSweep()
	{
		{
			//based on real sweep from game
			const FAABB3 A(FVec3(-2560.00000, -268.000031, -768.000122), FVec3(0.000000000, 3.99996948, 0.000000000));
			const FAABB3 B(FVec3(-248.000000, -248.000000, -9.99999975e-05), FVec3(248.000000, 248.000000, 9.99999975e-05));
			const FRigidTransform3 BToATM(FVec3(-2559.99780, -511.729492, -8.98901367), FRotation3::FromElements(1.51728955e-06, 1.51728318e-06, 0.707108259, 0.707105279));
			const FVec3 LocalDir(-4.29153351e-06, 0.000000000, -1.00000000);
			const FReal Length = 393.000000;
			const FVec3 SearchDir(511.718750, -2560.00000, 9.00000000);

			FReal Time;
			FVec3 Pos, Normal;
			GJKRaycast2<FReal>(A, B, BToATM, LocalDir, Length, Time, Pos, Normal, 0, true, SearchDir, 0);
		}

		{
			//based on real sweep from game
			TArray<FConvex::FVec3Type> ConvexParticles;
			ConvexParticles.SetNum(10);

			ConvexParticles[0] = { 51870.2305f, 54369.6719f, 19200.0000f };
			ConvexParticles[1] = { -91008.5625f, -59964.0000f, -19199.9629f };
			ConvexParticles[2] = { 51870.2305f, 54369.6758f, -19199.9668f };
			ConvexParticles[3] = { 22164.4883f, 124647.500f, -19199.9961f };
			ConvexParticles[4] = { 34478.5000f, 123975.492f, -19199.9961f };
			ConvexParticles[5] = { -91008.5000f, -59963.9375f, 19200.0000f };
			ConvexParticles[6] = { -91008.5000f, 33715.5625f, 19200.0000f };
			ConvexParticles[7] = { 34478.4961f, 123975.500f, 19200.0000f };
			ConvexParticles[8] = { 22164.4922f, 124647.500f, 19200.0000f };
			ConvexParticles[9] = { -91008.5000f, 33715.5625f, -19199.9961f };


			const FConvex A(ConvexParticles, 0.0f);
			const FAABB3 B(FVec3{ -6.00000000, -248.000000, -9.99999975e-05 }, FVec3{ 6.00000000, 248.000000, 9.99999975e-05 });
			const FRigidTransform3 BToATM(FVec3{33470.5000, 41570.5000, -1161.00000}, FRotation3::FromIdentity());
			const FVec3 LocalDir(0, 0, -1);
			const FReal Length = 393.000000;
			const FVec3 SearchDir{ -33470.5000, -41570.5000, 1161.00000 };

			FReal Time;
			FVec3 Pos, Normal;
			GJKRaycast2<FReal>(A, B, BToATM, LocalDir, Length, Time, Pos, Normal, 0, true, SearchDir, 0);
		}
	}

	// When we have a capsule and box that are reported as initially-overlapping because they are within
	// the GJK epsilon of each other (but actually positively separated), verify that we get a zero time of impact.
	// Previously the slightly-positive separation would result in a negative penetration and a positive TOI.
	// Bug fix: CL 10942094.
	// NOTE: this issue no longer manifests with this example because GJK no longer reports this case as
	// overlapping> The GJK epsilon no longer takes part in the distance calculation when the near point
	// is on the face of the convex.
	GTEST_TEST(GJKTests, DISABLED_TestGJKCapsuleConvexInitialOverlapSweep_Fixed)
	{
		{
			TArray<FConvex::FVec3Type> ConvexParticles;
			ConvexParticles.SetNum(8);

			ConvexParticles[0] ={-256.000031f,12.0000601f,384.000061f};
			ConvexParticles[1] ={256.000031f,12.0000601f,384.000061f};
			ConvexParticles[2] ={256.000031f,12.0000601f,6.10351563e-05f};
			ConvexParticles[3] ={-256.000031f,-11.9999399f,6.10351563e-05f};
			ConvexParticles[4] ={-256.000031f,12.0000601f,6.10351563e-05f};
			ConvexParticles[5] ={-256.000031f,-11.9999399f,384.000061f};
			ConvexParticles[6] ={256.000031f,-11.9999399f,6.10351563e-05f};
			ConvexParticles[7] ={256.000031f,-11.9999399f,384.000061f};

			FConvexPtr UniqueConvex( new FConvex(ConvexParticles, 0.0f));
			const TImplicitObjectScaled<FConvex> A(UniqueConvex, FVec3(1.0,1.0,1.0));

			const FVec3 Pt0(0.0,0.0,-33.0);
			FVec3 Pt1 = Pt0;
			Pt1 += (FVec3(0.0,0.0,1.0) * 66.0);

			const FCapsule B(Pt0,Pt1,42.0);

			const FRigidTransform3 BToATM(FVec3(157.314758,-54.0000839,76.1436157), FRotation3::FromElements(0.0,0.0,0.704960823,0.709246278));
			const FVec3 LocalDir(-0.00641351938,-0.999979556,0.0);
			const FReal Length = 0.0886496082;
			const FVec3 SearchDir(-3.06152344,166.296631,-76.1436157);

			FReal Time;
			FVec3 Position,Normal;
			EXPECT_TRUE(GJKRaycast2<FReal>(A,B,BToATM,LocalDir,Length,Time,Position,Normal,0,true,SearchDir,0));
			EXPECT_FLOAT_EQ(Time,0.0);
		}
	}

	void GJKCapsuleConvexInitialOverlapSweep()
	{
		{
			TArray<FConvex::FVec3Type> ConvexParticles;
			ConvexParticles.SetNum(16);

			ConvexParticles[0] ={-127.216454f,203.240234f,124.726524f};
			ConvexParticles[1] ={125.708847f,203.240295f,124.726524f};
			ConvexParticles[2] ={-120.419685f,207.124924f,-0.386817127f};
			ConvexParticles[3] ={-32.9052734f,91.5147095f,199.922119f};
			ConvexParticles[4] ={118.912071f,91.3693237f,155.363205f};
			ConvexParticles[5] ={31.3977623f,91.5147705f,199.922150f};
			ConvexParticles[6] ={115.392204f,91.6678925f,162.647476f};
			ConvexParticles[7] ={-120.419701f,91.1026840f,-0.386809498f};
			ConvexParticles[8] ={118.912086f,207.124985f,-0.386806667f};
			ConvexParticles[9] ={118.912086f,91.1027603f,-0.386806667f};
			ConvexParticles[10] ={-120.419685f,91.3692703f,155.363174f};
			ConvexParticles[11] ={-110.103012f,199.020554f,160.910324f};
			ConvexParticles[12] ={-116.899742f,91.6678467f,162.647491f};
			ConvexParticles[13] ={31.3977337f,194.240265f,194.534988f};
			ConvexParticles[14] ={-32.9052925f,194.240204f,194.534958f};
			ConvexParticles[15] ={108.595482f,199.020599f,160.910309f};

			auto Convex = MakeShared<FConvex, ESPMode::ThreadSafe>(ConvexParticles, 0.0f);
			const auto& A = *Convex;
			//const TImplicitObjectInstanced<FConvex> A(Convex);

			const FVec3 Pt0(0.0,0.0,-45);
			FVec3 Pt1 = Pt0;
			Pt1 += (FVec3(0.0,0.0,1.0) * 90);

			const FCapsule B(Pt0,Pt1,33.8499985);

			const FRigidTransform3 ATM(FVec3(2624.00024, -383.998962, 4.00000000), FRotation3::FromElements(-5.07916162e-08, -3.39378659e-08, 0.555569768, 0.831469893));
			const FRigidTransform3 BTM(FVec3(2461.92749, -205.484283, 106.071632), FRotation3::FromElements(0,0,0,1));
			const FRigidTransform3 BToATM(FVec3(102.903252, 218.050415, 102.071655), FRotation3::FromElements(5.07916162e-08, 3.39378659e-08, -0.555569768, 0.831469893));

			FReal Penetration = 0;
			FVec3 ClosestA = FVec3(0);
			FVec3 ClosestB = FVec3(0);
			FVec3 Normal = FVec3(0);
			int32 ClosestVertexIndexA = INDEX_NONE;
			int32 ClosestVertexIndexB = INDEX_NONE;
			const FVec3 Offset ={162.072754,-178.514679,-102.071632};
			EXPECT_TRUE((GJKPenetration<false, FReal>(A,B,BToATM,Penetration,ClosestA,ClosestB,Normal,ClosestVertexIndexA,ClosestVertexIndexB,0,0,Offset)));

			const FRigidTransform3 NewAToBTM (BToATM.GetTranslation() + (0.01 + Penetration) * Normal,BToATM.GetRotation());;

			EXPECT_FALSE((GJKPenetration<false, FReal>(A,B,NewAToBTM,Penetration,ClosestA,ClosestB,Normal,ClosestVertexIndexA,ClosestVertexIndexB,0,0,Offset)));

		}

		{
			//capsule perfectly aligned with another capsule but a bit off on the z
			const FVec3 Pt0(0.0,0.0,-45.0);
			FVec3 Pt1 = Pt0;
			Pt1 += (FVec3(0.0,0.0,1.0) * 90.0);

			const FCapsule A(Pt0,Pt1,34.f);
			const FCapsule B(Pt0,Pt1,33.8499985f);

			const FRigidTransform3 BToATM(FVec3(0.0f,0.0f,-23.4092140f), FRotation3::FromElements(0.0,0.0,0.0,1.0));

			EXPECT_TRUE(GJKIntersection<FReal>(A,B,BToATM,0.0,FVec3(0,0,23.4092140)));

			FReal Penetration;
			FVec3 ClosestA,ClosestB,Normal;
			int32 ClosestVertexIndexA, ClosestVertexIndexB;
			EXPECT_TRUE((GJKPenetration<false, FReal>(A,B,BToATM, Penetration, ClosestA, ClosestB, Normal, ClosestVertexIndexA, ClosestVertexIndexB, 0.0, 0.0, FVec3(0,0,23.4092140))));
			EXPECT_FLOAT_EQ(Normal.Z,0);
			EXPECT_FLOAT_EQ(Penetration,A.GetRadiusf() + B.GetRadiusf());
		}

		{
			//capsule vs triangle as we make the sweep longer the world space point of impact should stay the same
			TArray<FConvex::FVec3Type> ConvexParticles;
			ConvexParticles.SetNum(3);

			ConvexParticles[0] ={7400.00000f, 12600.0000f, 206.248123f};
			ConvexParticles[1] ={7500.00000f, 12600.0000f, 199.994904f};
			ConvexParticles[2] ={7500.00000f, 12700.0000f, 189.837433f};

			FConvexPtr UniqueConvex( new FConvex(ConvexParticles, 0.0f));
			const TImplicitObjectScaled<FConvex> AConvScaled(UniqueConvex, FVec3(1.0,1.0,1.0));

			FTriangle A(ConvexParticles[0],ConvexParticles[1],ConvexParticles[2]);
			FTriangleRegister AReg(
				MakeVectorRegisterFloat(ConvexParticles[0].X, ConvexParticles[0].Y, ConvexParticles[0].Z, 0.0f),
				MakeVectorRegisterFloat(ConvexParticles[1].X, ConvexParticles[1].Y, ConvexParticles[1].Z, 0.0f),
				MakeVectorRegisterFloat(ConvexParticles[2].X, ConvexParticles[2].Y, ConvexParticles[2].Z, 0.0f));

			const FVec3 Pt0(0.0,0.0,-29.6999969);
			FVec3 Pt1 = Pt0;
			Pt1 += (FVec3(0.0,0.0,1.0) * 59.3999939);

			const FCapsule B(Pt0,Pt1,42.0);

			const FRigidTransform3 BToATM(FVec3(7475.74512, 12603.9082, 277.767120), FRotation3::FromElements(0,0,0,1));
			const FVec3 LocalDir(0,0,-0.999999940);
			const FReal Length = 49.9061584;
			const FVec3 SearchDir(1,0,0);

			FReal Time;
			FVec3 Position,Normal;
			EXPECT_TRUE(GJKRaycast2<FReal>(AConvScaled,B,BToATM,LocalDir,Length,Time,Position,Normal,0,true,SearchDir,0));

			const FRigidTransform3 BToATM2(FVec3(7475.74512, 12603.9082, 277.767120+100), FRotation3::FromElements(0,0,0,1));

			FReal Time2;
			FVec3 Position2,Normal2;
			EXPECT_TRUE(GJKRaycast2<FReal>(AConvScaled,B,BToATM2,LocalDir,Length+100,Time2,Position2,Normal2,0,true,SearchDir,0));
			EXPECT_TRUE(GJKRaycast2<FReal>(AReg,B,BToATM2,LocalDir,Length+100,Time2,Position2,Normal2,0,true,SearchDir,0));

			EXPECT_NEAR(Time+100,Time2, 1.0f); // TODO: Investigate: This used to be 0
			EXPECT_VECTOR_NEAR(Normal,Normal2,1e-3); // TODO: Investigate: This used to be 1e-4
			EXPECT_VECTOR_NEAR(Position,Position2,1e-1); // TODO: Investigate: This used to be 1e-3
		}

		
		{
			// For this test we are clearly not penetrating
			// but we had an actual bug (edge condition) that showed we are

			const FVec3 Pt0(0.0, 0.0, 0.0);
			FVec3 Pt1(100.0,0,0);
			FVec3 Pt2(0, 1000000.0, 0);
			

			const FCapsule A(Pt1, Pt2, 1.0);
			const Chaos::FSphere B(Pt0, 1.0);

			const FRigidTransform3 BToATM(FVec3(0, 0, 0), FRotation3::FromElements(0.0, 0.0, 0, 1)); // Unit transform
			const FVec3 InitDir(0.1, 0.0, 0.0);

			FReal Penetration;
			FVec3 ClosestA, ClosestB, Normal;			
			int32 ClosestVertexIndexA, ClosestVertexIndexB;

			// First demonstrate the distance between the shapes are more than 90cm.
			bool IsValid = GJKPenetration<true, FReal>(A, B, BToATM, Penetration, ClosestA, ClosestB, Normal, ClosestVertexIndexA, ClosestVertexIndexB, 0, 0, InitDir);
			EXPECT_TRUE(IsValid);
			EXPECT_TRUE(Penetration < -90.0f);

			// Since there is no penetration (by more than 90cm) this function should return false when negative penetration is not supported
			bool IsPenetrating = GJKPenetration<false, FReal>(A, B, BToATM, Penetration, ClosestA, ClosestB, Normal, ClosestVertexIndexA, ClosestVertexIndexB, 0, 0, InitDir);
			EXPECT_FALSE(IsPenetrating);
			
		}
		
	}

	// Tests a case where we have a reasonable query but the target shape is a very large distance away.
	// This should result in a miss but currently doesn't - and gives an OutTime that is infinite.
	// Detected when querying global payload object in the SQ system where we test each object without
	// considering its bounds.
	void GJKLargeDistanceCapsuleSweep()
	{
		// Data from repro case
		Chaos::TBox<Chaos::FReal, 3> A({ -24.011219501495361, -7.4698066711425781, -0.83472084999084473 }, { 32.555774211883545, 10.860815048217773, 14.719563245773315 }, 0);
		Chaos::FCapsule B({0.0000000000000000, 0.0000000000000000, -67.499992370605469}, {0.0000000000000000, 0.0000000000000000, -67.499992370605469 + 134.99998474121094 }, 67.274772644042969);
		Chaos::TRigidTransform<Chaos::FReal, 3> BToA;
		BToA.SetRotation({0.0000000000000000, 0.0000000000000000, -0.70710678118654757, 0.70710678118654746});
		BToA.SetTranslation({0.0000000000000000, -3.3237259359872290e+32, 7460.1000976562500});
		const Chaos::FVec3 LocalDir{0.93683970992769239, 0.040186153777059030, 0.34744278175123056};
		const Chaos::FVec3 InitialDir{-3.3237259359872290e+32, 27121.400390625000, -7460.1000976562500};
		const FReal Length = 13.27157020568847;
		const FReal Thickness = 0;
		const bool bComputeMtd = true;

		FReal OutTime;
		FVec3 OutLoc;
		FVec3 OutNorm;

		// Should fail and give a valid time
		bool bHit = GJKRaycast2(A, B, BToA, LocalDir, Length, OutTime, OutLoc, OutNorm, Thickness, bComputeMtd, InitialDir, Thickness);

		EXPECT_FALSE(bHit);

		// Expect to receive a valid time.
		EXPECT_TRUE(FMath::IsFinite(OutTime));
	}

	// Check that GJKPenetrationCore returns the correct result when two objects are within various distances
	// of each other. When distance is less that GJKEpsilon, GJK will abort and call into EPA.
	void GJKBoxBoxZeroMarginSeparationTest(FReal GJKEpsilon, FReal SeparationSize, int32 SeparationAxis)
	{
		FVec3 SeparationDir = FVec3(0);
		SeparationDir[SeparationAxis] = 1.0f;
		FVec3 Separation = SeparationSize * SeparationDir;

		// Extents covering both boxes - we will split this in the middle using the separation axis
		FVec3 MinExtent = FVec3(-100, -100, -100);
		FVec3 MaxExtent = FVec3(100, 100, 100);

		// A is most positive along separation axis and shifted by SeperationSize (e.g., the top is axis is Z)
		FVec3 MinA = MinExtent;
		FVec3 MaxA = MaxExtent;
		MinA[SeparationAxis] = SeparationSize;
		MaxA[SeparationAxis] = 100.0f + SeparationSize;

		// B is most negative along separation axis and shifted by SeperationSize (e.g., the bottom if axis is Z)
		FVec3 MinB = MinExtent;
		FVec3 MaxB = MaxExtent;
		MaxB[SeparationAxis] = 0.0f;
		
		// Create the shapes
		float MarginA = 0.0f;
		float MarginB = 0.0f;
		FImplicitBox3 ShapeA(MinA, MaxA, MarginA);
		FImplicitBox3 ShapeB(MinB, MaxB, MarginB);
		const FRigidTransform3 TransformA = FRigidTransform3::Identity;
		const FRigidTransform3 TransformB = FRigidTransform3::Identity;
		const FRigidTransform3 TransformBtoA = FRigidTransform3::Identity;
		const FReal ThicknessA = 0.0f;
		const FReal ThicknessB = 0.0f;

		// Run GJK/EPA
		FReal Penetration;
		FVec3 ClosestA, ClosestBInA, Normal;
		int32 ClosestVertexIndexA, ClosestVertexIndexB;
		bool bSuccess = GJKPenetration<true>(ShapeA, ShapeB, TransformBtoA, Penetration, ClosestA, ClosestBInA, Normal, ClosestVertexIndexA, ClosestVertexIndexB, ThicknessA, ThicknessB, FVec3(1, 0, 0), GJKEpsilon);
		EXPECT_TRUE(bSuccess);

		if (bSuccess)
		{
			// Convert the contact data to world-space (not really necessary here)
			const FVec3 ResultLocation = TransformA.TransformPosition(ClosestA + ThicknessA * Normal);
			const FVec3 ResultNormal = -TransformA.TransformVectorNoScale(Normal);
			const FReal ResultPhi = -Penetration;

			const FReal ExpectedLocationI = SeparationSize;
			const FReal ExpectedNormalI = 1.0f;
			const FReal ExpectedPhi = SeparationSize;

			EXPECT_NEAR(ResultLocation[SeparationAxis], ExpectedLocationI, 1.e-3f) << "Separation " << SeparationSize << " Axis " << SeparationAxis;
			EXPECT_NEAR(ResultNormal[SeparationAxis], ExpectedNormalI, 1.e-4f) << "Separation " << SeparationSize << " Axis " << SeparationAxis;
			EXPECT_NEAR(ResultPhi, ExpectedPhi, 1.e-3f) << "Separation " << SeparationSize << " Axis " << SeparationAxis;
		}
	}

	const FReal BoxBoxGJKDistances[] =
	{
		1.0f,
		1.0f / 2.0f,
		1.0f / 4.0f,
		1.0f / 8.0f,
		1.0f / 16.0f,
		1.0f / 32.0f,
		1.0f / 64.0f,
		1.0f / 128.0f,
		1.0f / 256.0f,
		1.0f / 512.0f,
		1.0f / 1024.0f,
		1.0f / 2048.0f,
		1.0f / 4096.0f,
		1.0f / 8192.0f,
		1.0f / 16384.0f,
		1.0f / 32768.0f,
		1.e-4f,
		1.e-5f,
		1.e-6f,
		1.e-7f,
		1.e-8f,
		0.0f,
	};
	const int32 NumBoxBoxGJKDistances = UE_ARRAY_COUNT(BoxBoxGJKDistances);

	// These tests fails in EPA - we need to cover these cases with SAT
	GTEST_TEST(GJKTests, TestGJKBoxBoxTestFails)
	{
		const FReal Epsilon = 1.e-3f;

		// These are the cases that case EPA to fail out with a degenerate simplex
		GJKBoxBoxZeroMarginSeparationTest(Epsilon, -0.125f, 0);
		GJKBoxBoxZeroMarginSeparationTest(Epsilon, -0.03125, 0);
		GJKBoxBoxZeroMarginSeparationTest(Epsilon, -0.015625, 0);
		GJKBoxBoxZeroMarginSeparationTest(Epsilon, -0.0078125, 0);
		GJKBoxBoxZeroMarginSeparationTest(Epsilon, -0.00390625, 0);
		GJKBoxBoxZeroMarginSeparationTest(Epsilon, -0.001953125, 0);
	}

	// Disabled until we have SAT fallback (see DISABLED_TestGJKBoxBoxTestFails)
	GTEST_TEST(GJKTests, TestGJKBoxBoxNegativeSeparation)
	{
		const FReal Epsilon = 1.e-3f;

		for (int32 DistanceIndex = 0; DistanceIndex < NumBoxBoxGJKDistances; ++DistanceIndex)
		{
			for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
			{
				GJKBoxBoxZeroMarginSeparationTest(Epsilon, -BoxBoxGJKDistances[DistanceIndex], AxisIndex);
			}
		}
	}

	GTEST_TEST(GJKTests, TestGJKBoxBoxPositiveSeparation)
	{
		const FReal Epsilon = 1.e-3f;

		for (int32 DistanceIndex = 0; DistanceIndex < NumBoxBoxGJKDistances; ++DistanceIndex)
		{
			for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
			{
				GJKBoxBoxZeroMarginSeparationTest(Epsilon, BoxBoxGJKDistances[DistanceIndex], AxisIndex);
			}
		}
	}

	// This is a know regression test
	// It is two boxes deeply overlapping in a T-shape
	GTEST_TEST(GJKTests, TestGJKBoxBoxOverlapRegression1)
	{
		FVec3 MinBox = FVec3(-15.839999675750732, -31.840000152587891, -3.8146972691777137e-07);
		FVec3 MaxBox = FVec3(15.840000629425049, 31.840000152587891, 19.200000381469728);
		FVec3 Translation = FVec3(15.999999999999993, 1.9594348786357647e-15, 0.0000000000000000);
		FRotation3 Rotation(UE::Math::TQuat<FReal>(0.0000000000000000, 0.0000000000000000, -0.70710678118654757, -0.70710678118654746));

		FImplicitBox3 ShapeA(MinBox, MaxBox, 0);
		FImplicitBox3 ShapeB(MinBox, MaxBox, 0);
		
		const FRigidTransform3 TransformBtoA = FRigidTransform3(Translation , Rotation);
		const FReal ThicknessA = 0.0f;
		const FReal ThicknessB = 0.0f;

		// Run GJK/EPA
		FReal Penetration;
		FVec3 ClosestA, ClosestBInA, Normal;
		int32 ClosestVertexIndexA, ClosestVertexIndexB;
		bool bSuccess = GJKPenetration<false>(ShapeA, ShapeB, TransformBtoA, Penetration, ClosestA, ClosestBInA, Normal, ClosestVertexIndexA, ClosestVertexIndexB, ThicknessA, ThicknessB, FVec3(1, 0, 0));
		EXPECT_TRUE(bSuccess);
		EXPECT_TRUE(Penetration > 19.0f); // Penetration is the Height of the box
	}

	// This is a know regression test
	// Two boxes clearly overlapping
	// --gtest_filter=*TestGJKBoxBoxOverlapRegression2*
	GTEST_TEST(GJKTests, TestGJKBoxBoxOverlapRegression2)
	{
		
		FVec3 MinBox1 = FVec3(- 112.00000000000000, -256.00000000000000, -8.0000000000000000);
		FVec3 MaxBox1 = FVec3(112.00000000000000, 256.00000000000000, 8.0000000000000000);

		FVec3 MinBox2 = FVec3(-64.000000000000000, -64.000000000000000, -64.000000000000000);
		FVec3 MaxBox2 = FVec3(64.000000000000000, 64.000000000000000, 64.000000000000000);

		FVec3 Translation = FVec3(0.0044999999990977813, -255.99550000000090, -18.000000000000000);
		FRotation3 Rotation(UE::Math::TQuat<FReal>(0.0000000000000000, 0.0000000000000000, 0.0000000000000000,1.0f));

		FImplicitBox3 ShapeA(MinBox1, MaxBox1, 0);
		FImplicitBox3 ShapeB(MinBox2, MaxBox2, 0);

		const FRigidTransform3 TransformBtoA = FRigidTransform3(Translation, Rotation);		

		// Run GJK/EPA
		bool bOverlap = GJKIntersection<FReal>(ShapeA, ShapeB, TransformBtoA, 0.00f, FVec3{ -0.0044999999990977813, 255.99550000000090, 18.000000000000000 });
		EXPECT_TRUE(bOverlap);
	}	

	// Two convex shapes, Shape A on top of Shape B and almost touching. ShapeA is rotated 90 degrees about Z.
	// Check that the contact point lies between Shape A and Shape B with a near zero Phi.
	// This reproduces a bug where GJKPenetrationCore returns points on top of A and at the bottom
	// of B, with a Phi equal to the separation of those points. Resolving this contact
	// would result in Shape B popping to the top of Shape A.
	//
	// The problem was in EPA where the possible set of simplex faces are added to the queue. Here it checks
	// to see if the origin projects to within the face, since if it does not, it cannot the face that is nearest
	// to the origin. However, without a tolerance, this could reject valid faces.
	void GJKConvexConvexEPABoundaryCondition()
	{
		// These verts are those from a rectangular box with bevelled edges
		TArray<FConvex::FVec3Type> CoreShapeVerts =
		{
			{3.54999995f, -1.04999995f, 0.750000000f},
			{3.75000000f, 1.04999995f, 0.549999952f},
			{3.54999995f, 1.04999995f, 0.750000000f},
			{-3.54999995f, 1.04999995f, 0.750000000f},
			{-3.54999995f, 1.25000000f, 0.549999952f},
			{-3.54999995f, 1.25000000f, -0.550000012f},
			{-3.75000000f, 1.04999995f, 0.549999952f},
			{3.54999995f, 1.25000000f, 0.549999952f},
			{3.54999995f, 1.04999995f, -0.750000000f},
			{3.54999995f, 1.25000000f, -0.550000012f},
			{-3.54999995f, 1.04999995f, -0.750000000f},
			{-3.54999995f, -1.04999995f, -0.750000000f},
			{-3.75000000f, 1.04999995f, -0.550000012f},
			{3.54999995f, -1.25000000f, -0.550000012f},
			{3.54999995f, -1.04999995f, -0.750000000f},
			{-3.54999995f, -1.25000000f, 0.549999952f},
			{-3.54999995f, -1.25000000f, -0.550000012f},
			{-3.75000000f, -1.04999995f, -0.550000012f},
			{3.54999995f, -1.25000000f, 0.549999952f},
			{-3.54999995f, -1.04999995f, 0.750000000f},
			{-3.75000000f, -1.04999995f, 0.549999952f},
			{3.75000000f, -1.04999995f, 0.549999952f},
			{3.75000000f, -1.04999995f, -0.550000012f},
			{3.75000000f, 1.04999995f, -0.550000012f},
		};
		const FVec3 Scale = FVec3(50.0f);
		const FReal Margin = 0.75f;

		
		FConvexPtr CoreConvexShapePtr( new FImplicitConvex3(CoreShapeVerts, 0.0f, FConvexBuilder::EBuildMethod::Original));
		const TImplicitObjectScaled<FImplicitConvex3> ShapeA(CoreConvexShapePtr, Scale, Margin);
		const TImplicitObjectScaled<FImplicitConvex3> ShapeB(CoreConvexShapePtr, Scale, Margin);
		const FRigidTransform3 TransformA(FVec3(0.000000000f, 0.000000000f, 182.378937f), FRotation3::FromElements(0.000000000f, 0.000000000f, 0.707106650f, 0.707106888f));	// Top
		const FRigidTransform3 TransformB(FVec3(0.000000000f, 0.000000000f, 107.378944f), FRotation3::FromElements(0.000000000f, 0.000000000f, 0.000000000f, 1.00000000f));		// Bottom

		// Shape Z extents = [50x-0.75, 50x0.75] = [-37.5, 37.5]
		// Shape Z separation = 182.378937 - 107.378944 = 74.999993
		// i.e., the shapes are touching to near float accuracy
		// The top shape is rotated by 90degrees

		const FRigidTransform3 TransformBtoA = TransformB.GetRelativeTransform(TransformA);

		FReal Penetration;
		FVec3 ClosestA, ClosestBInA, Normal;
		int32 ClosestVertexIndexA, ClosestVertexIndexB;
		const FReal Epsilon = 3.e-3f;

		const FReal ThicknessA = 0.0f;
		const FReal ThicknessB = 0.0f;

		const bool bSuccess = GJKPenetration<true>(ShapeA, ShapeB, TransformBtoA, Penetration, ClosestA, ClosestBInA, Normal, ClosestVertexIndexA, ClosestVertexIndexB, ThicknessA, ThicknessB, FVec3(1,0,0), Epsilon);
		EXPECT_TRUE(bSuccess);

		if (bSuccess)
		{
			const FVec3 ContactLocation = TransformA.TransformPosition(ClosestA + ThicknessA * Normal);
			const FVec3 ContactNormal = -TransformA.TransformVectorNoScale(Normal);
			const FReal ContactPhi = -Penetration;

			// Contact should be on bottom of A
			// Normal should point upwards (from B to A)
			// const FReal PreviousIncorrectLocationZ = TransformA.GetTranslation().Z + ShapeA.BoundingBox().Max().Z;
			// const FReal PreviousIncorrectNormalZ = -1.0f;
			const FReal ExpectedContactLocationZ = TransformA.GetTranslation().Z + ShapeA.BoundingBox().Min().Z;
			const FReal ExpectedContactNormalZ = 1.0f;
			const FReal ExpectedContactPhi = (TransformA.GetTranslation().Z + ShapeA.BoundingBox().Min().Z) - (TransformB.GetTranslation().Z + ShapeB.BoundingBox().Max().Z);

			EXPECT_NEAR(ContactLocation.Z, ExpectedContactLocationZ, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(ContactNormal.Z, ExpectedContactNormalZ, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(ContactPhi, ExpectedContactPhi, KINDA_SMALL_NUMBER);
		}
	}

	GTEST_TEST(GJKTests, TestGJKConvexConvexEPABoundaryCondition)
	{
		GJKConvexConvexEPABoundaryCondition();
	}


	void NegativeScaleConvexTest()
	{
		TArray<FConvex::FVec3Type> ConvexVerts =
		{
			{512.000061f, -1279.99988f, -383.999939f},
			{511.999969f, 6.81566016e-05f, 2.23802308e-05f},
			{512.000000f, -255.999939f, 2.23802308e-05f},
			{-2.36513770e-05f, -1.52587909e-05f, -2.84217094e-14f},
			{1.80563184e-05f, -256.000031f, -2.84217094e-14f},
			{2.26354750e-05f, -1024.00000f, -383.999969f},
			{7.96019594e-05f, -1280.00000f, -383.999969f},
			{512.000061f, -1023.99994f, -383.999939f}
		};
		TArray<FConvex::FVec3Type> ConvexVertices(MoveTemp(ConvexVerts));
		FConvexPtr CoreConvex( new FImplicitConvex3(ConvexVertices, 0.0f));
		const TImplicitObjectScaled<FImplicitConvex3> ScaledConvex(CoreConvex, FVec3(-1,1,1), 38.4000015);
		const Chaos::FSphere Sphere(FVec3(0,0,0), 32);
		const FRigidTransform3 StartTM(FVec3( -172.000000, -48.0000000, 52.0000000 ), FRotation3::FromIdentity());


		FVec3 Dir(0, 0, -1);
		FReal Length = 200;
		FVec3 OutNormal;
		FReal OutTime = -1;
		FVec3 OutPos(0, 0, 0);
		int32 OutFaceIdx = -1;
		const bool bSuccess = GJKRaycast2(ScaledConvex, Sphere, StartTM, Dir, Length, OutTime, OutPos, OutNormal, (FReal)0., true);
		EXPECT_TRUE(bSuccess);
	}

	void NegativeScaleConvexTest2()
	{
		//TArray<FVec3> ConvexVerts =
		//{
		//	// subset of verts from above test.
		//	{512.000061, -1279.99988, -383.999939},    // Uncommenting this will cause sweep to miss. Why?
		//	{511.999969, 6.81566016e-05, 2.23802308e-05},
		//	{512.000000, -255.999939, 2.23802308e-05},
		//	{-2.36513770e-05, -1.52587909e-05, -2.84217094e-14},
		//	{1.80563184e-05, -256.000031, -2.84217094e-14},
		//};
		TArray<FConvex::FVec3Type> ConvexVerts =
		{
			// subset of verts from above test.
			FVec3(-512, -1280, -384),
			FVec3(-512, 0, 0),
			FVec3(-512, -256, 0),
			FVec3(0, 0, 0),
			FVec3(0, -256, 0),
		};
		TArray<FConvex::FVec3Type> ConvexVertices(MoveTemp(ConvexVerts));
		FImplicitConvex3 CoreConvex = FImplicitConvex3(ConvexVertices, 38.4000015);
		const Chaos::FSphere Sphere(FVec3(0, 0, 0), 32);
		const FRigidTransform3 StartTM(FVec3(-172.000000, -48.0000000, 52.0000000), FRotation3::FromIdentity());


		FVec3 Dir(0, 0, -1);
		FReal Length = 200;
		FVec3 OutNormal;
		FReal OutTime = -1;
		FVec3 OutPos(0, 0, 0);
		int32 OutFaceIdx = -1;
		const bool bSuccess = GJKRaycast2(CoreConvex, Sphere, StartTM, Dir, Length, OutTime, OutPos, OutNormal, (FReal)0., true);
		EXPECT_TRUE(bSuccess);
	}

	// Disabled until we use different margins for sweeping
	GTEST_TEST(GJKTests, DISABLED_TestGJKConvexNegativeScale)
	{
		NegativeScaleConvexTest();
		NegativeScaleConvexTest2();
	}

	GTEST_TEST(GJKTests, BoxBoxWarmStartTest)
	{
		FAABB3 Box({ -50, -50, -50 }, { 50, 50, 50 });

		FRigidTransform3 ATM = FRigidTransform3::Identity;
		FRigidTransform3 BTM = FRigidTransform3(FVec3(0, 0, 105), FRotation3::FromIdentity());
		FVec3 ClosestA, ClosestB, NormalA, NormalB;
		FReal Penetration;

		FGJKSimplexData WarmStartData;
		FReal SupportDelta = FReal(0);
		int32 VertexIndexA = INDEX_NONE;
		int32 VertexIndexB = INDEX_NONE;

		// Separated (GJK)
		GJKPenetrationWarmStartable(Box, Box, BTM.GetRelativeTransformNoScale(ATM), Penetration, ClosestA, ClosestB, NormalA, NormalB, VertexIndexA, VertexIndexB, WarmStartData, SupportDelta);
		EXPECT_NEAR(Penetration, -5.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ClosestA.Z, (FReal)50.0f, (FReal)KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ClosestB.Z, (FReal)-50.0f, (FReal)KINDA_SMALL_NUMBER);

		GJKPenetrationWarmStartable(Box, Box, BTM.GetRelativeTransformNoScale(ATM), Penetration, ClosestA, ClosestB, NormalA, NormalB, VertexIndexA, VertexIndexB, WarmStartData, SupportDelta);
		EXPECT_NEAR(Penetration, -5.0f, KINDA_SMALL_NUMBER);

		BTM = FRigidTransform3(FVec3(0, 0, 145), FRotation3::FromIdentity());
		GJKPenetrationWarmStartable(Box, Box, BTM.GetRelativeTransformNoScale(ATM), Penetration, ClosestA, ClosestB, NormalA, NormalB, VertexIndexA, VertexIndexB, WarmStartData, SupportDelta);
		EXPECT_NEAR(Penetration, -45.0f, KINDA_SMALL_NUMBER);

		BTM = FRigidTransform3(FVec3(0, 0, 145), FRotation3::FromAxisAngle(FVec3(1, 0, 0), FMath::DegreesToRadians(110.0f)));
		GJKPenetrationWarmStartable(Box, Box, BTM.GetRelativeTransformNoScale(ATM), Penetration, ClosestA, ClosestB, NormalA, NormalB, VertexIndexA, VertexIndexB, WarmStartData, SupportDelta);
		EXPECT_NEAR(Penetration, -30.9144f, KINDA_SMALL_NUMBER);

		FReal Penetration2;
		GJKPenetrationWarmStartable(Box, Box, BTM.GetRelativeTransformNoScale(ATM), Penetration2, ClosestA, ClosestB, NormalA, NormalB, VertexIndexA, VertexIndexB, WarmStartData, SupportDelta);
		EXPECT_NEAR(Penetration2, Penetration, KINDA_SMALL_NUMBER);
	}

	GTEST_TEST(GJKTests, BoxBoxWarmStartDeepTest)
	{
		FAABB3 Box({ -50, -50, -50 }, { 50, 50, 50 });

		FRigidTransform3 ATM = FRigidTransform3::Identity;
		FRigidTransform3 BTM = FRigidTransform3(FVec3(0, 0, 60), FRotation3::FromIdentity());
		FVec3 ClosestA, ClosestB, NormalA, NormalB;
		FReal Penetration;

		FGJKSimplexData WarmStartData;
		FReal SupportDelta = FReal(0);

		int32 VertexIndexA = INDEX_NONE;
		int32 VertexIndexB = INDEX_NONE;

		// Deep (EPA) No Margin
		GJKPenetrationWarmStartable(Box, Box, BTM.GetRelativeTransformNoScale(ATM), Penetration, ClosestA, ClosestB, NormalA, NormalB, VertexIndexA, VertexIndexB,  WarmStartData, SupportDelta);
		EXPECT_NEAR(Penetration, 40.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ClosestA.Z, (FReal)50.0f, (FReal)KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ClosestB.Z, (FReal)-50.0f, (FReal)KINDA_SMALL_NUMBER);

		// Deep (EPA) With Margin
		WarmStartData = FGJKSimplexData();
		TGJKCoreShape<FAABB3> MarginBox(Box, FReal(10));
		GJKPenetrationWarmStartable(MarginBox, MarginBox, BTM.GetRelativeTransformNoScale(ATM), Penetration, ClosestA, ClosestB, NormalA, NormalB,  VertexIndexA, VertexIndexB, WarmStartData, SupportDelta);
		EXPECT_NEAR(Penetration, 40.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ClosestA.Z, (FReal)50.0f, (FReal)KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ClosestB.Z, (FReal)-50.0f, (FReal)KINDA_SMALL_NUMBER);

		// Deep (EPA) With Margin and Relative Rotation
		WarmStartData = FGJKSimplexData();
		ATM = FRigidTransform3(FVec3(0, 0, 0), FRotation3::FromAxisAngle(FVec3(1, 0, 0), FMath::DegreesToRadians(180)));
		GJKPenetrationWarmStartable(MarginBox, MarginBox, BTM.GetRelativeTransformNoScale(ATM), Penetration, ClosestA, ClosestB, NormalA, NormalB,  VertexIndexA, VertexIndexB, WarmStartData, SupportDelta);
		EXPECT_NEAR(Penetration, 40.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ClosestA.Z, (FReal)-50.0f, (FReal)KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ClosestB.Z, (FReal)-50.0f, (FReal)KINDA_SMALL_NUMBER);
	}

	GTEST_TEST(GJKTests, SphereSphereWarmStartTest)
	{
		FImplicitSphere3 Sphere(FVec3(0), FReal(50));

		FRigidTransform3 ATM = FRigidTransform3::Identity;
		FRigidTransform3 BTM = FRigidTransform3(FVec3(0, 0, 105), FRotation3::FromIdentity());
		FVec3 ClosestA, ClosestB, NormalA, NormalB;
		FReal Penetration;

		FGJKSimplexData WarmStartData;
		FReal SupportDelta = FReal(0);
		
		int32 VertexIndexA = INDEX_NONE;
		int32 VertexIndexB = INDEX_NONE;

		GJKPenetrationWarmStartable(Sphere, Sphere, BTM.GetRelativeTransformNoScale(ATM), Penetration, ClosestA, ClosestB, NormalA, NormalB,  VertexIndexA, VertexIndexB, WarmStartData, SupportDelta);
		EXPECT_NEAR(Penetration, (FReal)-5.0f, (FReal)KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ClosestA.Z, (FReal)50.0f, (FReal)KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ClosestB.Z, (FReal)-50.0f, (FReal)KINDA_SMALL_NUMBER);

		BTM = FRigidTransform3(FVec3(0, 0, 105), FRotation3::FromAxisAngle(FVec3(1,0,0), FMath::DegreesToRadians(180)));
		GJKPenetrationWarmStartable(Sphere, Sphere, BTM.GetRelativeTransformNoScale(ATM), Penetration, ClosestA, ClosestB, NormalA, NormalB, VertexIndexA, VertexIndexB, WarmStartData, SupportDelta);
		EXPECT_NEAR(Penetration, (FReal)-5.0f, (FReal)KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ClosestA.Z, (FReal)50.0f, (FReal)KINDA_SMALL_NUMBER);
		EXPECT_NEAR(ClosestB.Z, (FReal)50.0f, (FReal)KINDA_SMALL_NUMBER);

	}


	GTEST_TEST(GJKTests, GJKBug_BadBarycentricCoords)
	{
		FImplicitBox3 BoxA({ -31.999998092651367, -0.73166346549987793, -47.015655517578125 },
			{ 31.999998092651367, 0.73166346549987793, 47.015655517578125 });
		FImplicitBox3 BoxB({ -64.202491760253906, -64.269241333007812, 0.27499961853027344 },
			{ -0.20249176025390625, -0.18923950195312500, 38.524999618530273 });

		const FRigidTransform3 Transform(FVec3(4.28251314, -16.3213539, 32.0828743), FQuat(0.360390902, 0.00000000, 0.00000000, 0.932801366));

		const FVec3 RayDir(0.00000000, 0.672346294, -0.740236819);
		const FRealDouble Length = 37.388404846191406;
		FRealDouble OutTime = 0;
		FVec3 OutPosition(0);
		FVec3 OutNormal(0);
		const FRealDouble Thickness = 0;
		const bool bComputeMTD = true;
		const FVec3 Offset(-4.2825129036202441, -9.4891323727604799, -34.722524278655456);

		bool bResult = GJKRaycast2<FRealDouble, FImplicitBox3, FImplicitBox3>(BoxA, BoxB, Transform, RayDir, Length, OutTime, OutPosition, OutNormal, Thickness, bComputeMTD, Offset, Thickness);
		EXPECT_TRUE(bResult);
		EXPECT_NEAR(OutPosition.X, -13.95, 1e-1);
		EXPECT_NEAR(OutPosition.Y, -0.73, 1e-1);
		EXPECT_NEAR(OutPosition.Z, 14.63, 1e-1);
	}

	GTEST_TEST(GJKTests, GJK_LargeScaledBoxBoxTest)
	{

		TArray<FConvex::FVec3Type> ConvexParticles;
		ConvexParticles.SetNum(8);

		// This is a box with some small deviations
		ConvexParticles[0] = { 500.000000f, -500.000031f, 2.84217094e-14f };
		ConvexParticles[1] = { 500.000000f, 499.999969f, -50.0000153f };
		ConvexParticles[2] = { 500.000000f, -500.000031f, -50.0000153f };
		ConvexParticles[3] = { -500.000183f, 499.999969f, -50.0000153f };
		ConvexParticles[4] = { -500.000183f, -500.000031f, 2.84217094e-14f };
		ConvexParticles[5] = { -500.000183f, -500.000031f, -50.0000153f };
		ConvexParticles[6] = { -500.000183f, 499.999969f, -2.84217094e-14f };
		ConvexParticles[7] = { 500.000000f, 499.999969f, -2.84217094e-14f };

		Chaos::FConvexPtr BigBox( new Chaos::FConvex(ConvexParticles, 0.0f));

		// These two boxes are clearly intersecting each other

		Chaos::TBox<Chaos::FReal, 3> SmallBox({ -3200, -3200, -3200 }, { 3200, 3200, 3200 }, 0);

		TImplicitObjectScaled<Chaos::FConvex> BigBoxScaled(BigBox, FVec3(50, 50, 1));
		const TVector<FReal, 3> Translation{16000, 16000, -500};

		TRigidTransform<Chaos::FReal, 3> BToATM( Translation , TRotation<FReal, 3>::Identity);
		EXPECT_TRUE(GJKIntersection(BigBoxScaled, SmallBox, BToATM, FReal(0), Chaos::TVector<FReal, 3>(-16000, -16000, 500)));		

	}

	// Test known large triangle failure case
	GTEST_TEST(GJKTests, GJK_LargeTriangleCase)
	{
		VectorRegister4Float TriMeshScaleVector = MakeVectorRegister(2000.0f, 2000.0f, 1.0f, 0.0f);

		VectorRegister4Float A = MakeVectorRegister(100.0f, 0.0f, 100.0f, 0.0f);
		VectorRegister4Float B = MakeVectorRegister(0.0f, 100.0f, 100.0f, 0.0f);
		VectorRegister4Float C = MakeVectorRegister(0.0f, 0.0f, 100.0f, 0.0f);

		A = VectorMultiply(A, TriMeshScaleVector);
		B = VectorMultiply(B, TriMeshScaleVector);
		C = VectorMultiply(C, TriMeshScaleVector);

		FTriangleRegister Tri(A, B, C);
		const VectorRegister4Float TriNormal = VectorCross(VectorSubtract(B, A), VectorSubtract(C, A));

		FRealSingle Time;
		VectorRegister4Float OutPositionSimd, OutNormalSimd;
		Chaos::FSphere QueryGeom({ 0, 0, 0 }, 5.0f);
		VectorRegister4Float RotationSimd = MakeVectorRegister(0.0f, 0.0f, 0.0f, 0.0f);

		VectorRegister4Float TranslationSimd = MakeVectorRegister(100000.0f, 100022.820f, 213.002823f, 0.0f);
		VectorRegister4Float RayDirSimd = MakeVectorRegister(-2.22044605e-16f, 0.0f, -1.0f, 0.0f);
		float LengthScale = 1.0f;
		float CurDataLength = 108.002823f;
		const bool bComputeMTD = true;
		bool bFoundIntersection;
		if (Tri.IsTooBigForSinglePrecision())
		{
			VectorRegister4Double OutPositionDouble, OutNormalDouble;
			bFoundIntersection = GJKRaycast2ImplSimd<VectorRegister4Double>(Tri, QueryGeom, VectorRegister4Double(RotationSimd), VectorRegister4Double(TranslationSimd), VectorRegister4Double(RayDirSimd), LengthScale * CurDataLength, Time, OutPositionDouble, OutNormalDouble, bComputeMTD, GlobalVectorConstants::Double1000);
			OutPositionSimd = MakeVectorRegisterFloatFromDouble(OutPositionDouble);
			OutNormalSimd = MakeVectorRegisterFloatFromDouble(OutNormalDouble);
		}
		else
		{
			bFoundIntersection = GJKRaycast2ImplSimd<VectorRegister4Float>(Tri, QueryGeom, RotationSimd, TranslationSimd, RayDirSimd, LengthScale * CurDataLength, Time, OutPositionSimd, OutNormalSimd, bComputeMTD, GlobalVectorConstants::FloatMinusOne);
		}

		EXPECT_FALSE(bFoundIntersection);
	}

	GTEST_TEST(GJKTests, GJK_SweepConvexLargeTriangleInMesh)
	{
		using namespace Chaos;
		TArray<FConvex::FVec3Type> ConvexSurface(
			{
			{5.0, -5.0, -5.0},
			{-5.0, 5.0, -5.0},
			{5.0, 5.0, -5.0},
			{5.0, 5.0, 5.0},
			{-5.0, -5.0, -5.0},
			{-5.0, -5.0, 5.0},
			{5.0, -5.0, 5.0},
			{-5.0, 5.0, 5.0},
			});

		FConvex ConvexBox(MoveTemp(ConvexSurface), 0.0f);

		FTriangleMeshImplicitObject::ParticlesType TrimeshParticles(
			{
				{500.0000000, 500.0000000, 0.0},
				{500.0000000, -500.0000000, 0.0},
				{-500.0000000, 500.0000000, 0.0},
				{-500.0000000, -500.0000000, 0.0}
			});

		TArray<TVec3<int32>> Indices;
		Indices.Emplace(1, 0, 2);
		Indices.Emplace(1, 2, 3);

		TArray<uint16> Materials;
		Materials.Emplace(0);
		Materials.Emplace(0);
		FTriangleMeshImplicitObjectPtr TriangleMesh(new FTriangleMeshImplicitObject(MoveTemp(TrimeshParticles), MoveTemp(Indices), MoveTemp(Materials)));
		TImplicitObjectScaled<FTriangleMeshImplicitObject> ScaledTriangleMesh = TImplicitObjectScaled<FTriangleMeshImplicitObject>(TriangleMesh, FVec3(10000.0, 10000.0, 1.0));

		FQuat Rotation(0.00488796039, 0.00569311855, -0.000786740216, 0.999971569);
		FVec3 Translation(10500.365723, -14500.4132690, 8.2289352);

		TRigidTransform<FReal, 3> Transform(Translation, Rotation);


		FVec3 Dir(-0.00339674903, 5.76980747e-05, -0.999994159);
		FReal Length = 10.0;

		FReal OutTime = -1;
		FVec3 Normal(0.0f);
		FVec3 Position(0.0f);
		int32 FaceIndex = -1;
		FVec3 FaceNormal(0.0);
		bool bResult = ScaledTriangleMesh.LowLevelSweepGeom(ConvexBox, Transform, Dir, Length, OutTime, Position, Normal, FaceIndex, FaceNormal, 0.0f, true);
		EXPECT_TRUE(bResult);
		EXPECT_EQ(Position.Z, 0.0);
		EXPECT_NEAR(Position.X, 10505.365723, 1);
		EXPECT_NEAR(Position.Y, -14505.4132690, 1);
	}

	GTEST_TEST(GJKTests, GJK_SweepBoxLargeTriangleInMesh)
	{
		Chaos::TBox<Chaos::FReal, 3> QueryGeom({ -5, -5, -5 }, { 5, 5, 5 });

		FTriangleMeshImplicitObject::ParticlesType TrimeshParticles(
			{
				{500.0000000, 500.0000000, 0.0},
				{500.0000000, -500.0000000, 0.0},
				{-500.0000000, 500.0000000, 0.0},
				{-500.0000000, -500.0000000, 0.0}
			});

		TArray<TVec3<int32>> Indices;
		Indices.Emplace(1, 0, 2);
		Indices.Emplace(1, 2, 3);

		TArray<uint16> Materials;
		Materials.Emplace(0);
		Materials.Emplace(0);
		FTriangleMeshImplicitObjectPtr TriangleMesh(new FTriangleMeshImplicitObject(MoveTemp(TrimeshParticles), MoveTemp(Indices), MoveTemp(Materials)));
		TImplicitObjectScaled<FTriangleMeshImplicitObject> ScaledTriangleMesh = TImplicitObjectScaled<FTriangleMeshImplicitObject>(TriangleMesh, FVec3(10000.0, 10000.0, 1.0));

		FQuat Rotation(0.00488796039, 0.00569311855, -0.000786740216, 0.999971569);
		FVec3 Translation(10500.365723, -14500.4132690, 8.2289352);

		TRigidTransform<FReal, 3> Transform(Translation, Rotation);


		FVec3 Dir(-0.00339674903, 5.76980747e-05, -0.999994159);
		FReal Length = 10.0;

		FReal OutTime = -1;
		FVec3 Normal(0.0f);
		FVec3 Position(0.0f);
		int32 FaceIndex = -1;
		FVec3 FaceNormal(0.0);
		bool bResult = ScaledTriangleMesh.LowLevelSweepGeom(QueryGeom, Transform, Dir, Length, OutTime, Position, Normal, FaceIndex, FaceNormal, 0.0f, true);
		EXPECT_TRUE(bResult);
		EXPECT_EQ(Position.Z, 0.0);
		EXPECT_NEAR(Position.X, 10505.365723, 1);
		EXPECT_NEAR(Position.Y, -14505.4132690, 1);
	}

	GTEST_TEST(GJKTests, GJK_SweepSphereLargeTriangleInMesh)
	{
		Chaos::FSphere QueryGeom({ 0, 0, 0 }, 5.0f);

		FTriangleMeshImplicitObject::ParticlesType TrimeshParticles(
			{
				{500.0000000, 500.0000000, 0.0},
				{500.0000000, -500.0000000, 0.0},
				{-500.0000000, 500.0000000, 0.0},
				{-500.0000000, -500.0000000, 0.0}
			});

		TArray<TVec3<int32>> Indices;
		Indices.Emplace(1, 0, 2);
		Indices.Emplace(1, 2, 3);

		TArray<uint16> Materials;
		Materials.Emplace(0);
		Materials.Emplace(0);
		FTriangleMeshImplicitObjectPtr TriangleMesh(new FTriangleMeshImplicitObject(MoveTemp(TrimeshParticles), MoveTemp(Indices), MoveTemp(Materials)));
		TImplicitObjectScaled<FTriangleMeshImplicitObject> ScaledTriangleMesh = TImplicitObjectScaled<FTriangleMeshImplicitObject>(TriangleMesh, FVec3(10000.0, 10000.0, 1.0));

		FQuat Rotation(0.00488796039, 0.00569311855, -0.000786740216, 0.999971569);
		FVec3 Translation(10500.365723, -14500.4132690, 8.2289352);

		TRigidTransform<FReal, 3> Transform(Translation, Rotation);


		FVec3 Dir(-0.00339674903, 5.76980747e-05, -0.999994159);
		FReal Length = 10.0;

		FReal OutTime = -1;
		FVec3 Normal(0.0f);
		FVec3 Position(0.0f);
		int32 FaceIndex = -1;
		FVec3 FaceNormal(0.0);
		bool bResult = ScaledTriangleMesh.LowLevelSweepGeom(QueryGeom, Transform, Dir, Length, OutTime, Position, Normal, FaceIndex, FaceNormal, 0.0f, true);
		EXPECT_TRUE(bResult);
		EXPECT_EQ(Position.Z, 0.0);
		EXPECT_NEAR(Position.X, 10500.365723, 1e-1);
		EXPECT_NEAR(Position.Y, -14500.4132690, 1e-1);
	}

	GTEST_TEST(GJKTests, GJK_SweepCapsuleLargeTriangleInMesh)
	{
		Chaos::FCapsule QueryGeom({ 0, 0, 1.0 }, { 0, 0, -1.0 }, 5.0f);

		FTriangleMeshImplicitObject::ParticlesType TrimeshParticles(
			{
				{500.0000000, 500.0000000, 0.0},
				{500.0000000, -500.0000000, 0.0},
				{-500.0000000, 500.0000000, 0.0},
				{-500.0000000, -500.0000000, 0.0}
			});

		TArray<TVec3<int32>> Indices;
		Indices.Emplace(1, 0, 2);
		Indices.Emplace(1, 2, 3);

		TArray<uint16> Materials;
		Materials.Emplace(0);
		Materials.Emplace(0);
		FTriangleMeshImplicitObjectPtr TriangleMesh(new FTriangleMeshImplicitObject(MoveTemp(TrimeshParticles), MoveTemp(Indices), MoveTemp(Materials)));
		TImplicitObjectScaled<FTriangleMeshImplicitObject> ScaledTriangleMesh = TImplicitObjectScaled<FTriangleMeshImplicitObject>(TriangleMesh, FVec3(10000.0, 10000.0, 1.0));

		FQuat Rotation(0.00488796039, 0.00569311855, -0.000786740216, 0.999971569);
		FVec3 Translation(10500.365723, -14500.4132690, 8.2289352);

		TRigidTransform<FReal, 3> Transform(Translation, Rotation);


		FVec3 Dir(-0.00339674903, 5.76980747e-05, -0.999994159);
		FReal Length = 10.0;

		FReal OutTime = -1;
		FVec3 Normal(0.0f);
		FVec3 Position(0.0f);
		int32 FaceIndex = -1;
		FVec3 FaceNormal(0.0);
		bool bResult = ScaledTriangleMesh.LowLevelSweepGeom(QueryGeom, Transform, Dir, Length, OutTime, Position, Normal, FaceIndex, FaceNormal, 0.0f, true);
		EXPECT_TRUE(bResult);
		EXPECT_EQ(Position.Z, 0.0);
		EXPECT_NEAR(Position.X, 10500.365723, 1e-1);
		EXPECT_NEAR(Position.Y, -14500.4132690, 1e-1);
	}

	GTEST_TEST(GJKTests, GJK_SweepSphereBox)
	{
		Chaos::FSphere QueryGeom({ 0, 0, 0 }, 12.0f);

		Chaos::TBox<Chaos::FReal, 3> Box(FVec3(-240.742279/2.0, -16.483643/2.0, -17.078735/2.0), FVec3(240.742279 / 2.0, 16.483643 / 2.0, 17.078735 / 2.0));

		FVec3 BoxWorldTranslation(791.002986, -13679.957812, 7977.850960);
		FQuat BoxWorldRotation = FQuat(0.006953, -0.013390, 0.887363, -0.460825);
		TRigidTransform<FReal, 3> BoxWorldTransform(BoxWorldTranslation, BoxWorldRotation);
		
		FVec3 Start(715.18582, -13666.121838, 7860.814454);
		FVec3 End(727.595994, -13746.006452, 8140.664592);
				
		TRigidTransform<FReal, 3> TransformSphere(Start, FQuat::Identity);

		TRigidTransform<FReal, 3> FromBoxToSphere1 = TRigidTransform<FReal, 3>::MultiplyNoScale(BoxWorldTransform, TransformSphere.Inverse());
		TRigidTransform<FReal, 3> FromBoxToSphere2 = BoxWorldTransform.GetRelativeTransform(TransformSphere);


		FVec3 Dir(0.042604, -0.274241, 0.960717);
		const FVec3 LocalDir = TransformSphere.InverseTransformVectorNoScale(Dir);
		FVec3 Ray = End - Start;
		FReal Length = Ray.Length();

		FReal OutTime = -1;
		FVec3 Normal(0.0f);
		FVec3 Position(0.0f);
		int32 FaceIndex = -1;
		FVec3 FaceNormal(0.0);
		bool bResult = GJKRaycast2(QueryGeom, Box, FromBoxToSphere2, Dir, Length, OutTime, Position, Normal, 0.0, true);

		Position = TransformSphere.TransformPositionNoScale(Position);
		EXPECT_FALSE(bResult);
	}

	GTEST_TEST(GJKTests, GJK_SweepSphereTriangleMesh)
	{
		{
			FImplicitSphere3 Sphere{ FVec3(0.0f, 0.0f, 0.0f), 8.16201973f };
			VectorRegister4Float A = MakeVectorRegister(755.133118f, 69.0732956f, 184.725311f, 0.0f);
			VectorRegister4Float B = MakeVectorRegister(713.519043f, 205.570648f, 172.950577f, 0.0f);
			VectorRegister4Float C = MakeVectorRegister(686.582520f, 252.185165f, 365.669464f, 0.0f);
			FTriangleRegister Tri{ A,B,C };

			VectorRegister4Float StartRotation = MakeVectorRegister(0.0f, 0.0f, 0.0f, 1.0f);
			VectorRegister4Float StartPosition = MakeVectorRegister(727.730835f, 221.037231f, 363.199402f, 0.0f);
			VectorRegister4Float RayDirection = MakeVectorRegister(0.0423882306f, 0.0380843617f, -0.998375118f, 0.0f);

			FRealSingle RayLength = 200.0;
			FRealSingle OutTime;
			VectorRegister4Float OutPosition;
			VectorRegister4Float OutNormal;

			const bool bDidHit = GJKRaycast2ImplSimd<VectorRegister4Float>(Tri, Sphere, StartRotation, StartPosition, RayDirection, RayLength, OutTime, OutPosition, OutNormal, true, GlobalVectorConstants::Float1000);
			EXPECT_FALSE(bDidHit);
		}
		{
			FImplicitSphere3 Sphere{ FVec3(0.0f, 0.0f, 0.0f), 8.16201973f };
			VectorRegister4Float A = MakeVectorRegister(755.133118f, 69.0732956f, 184.725311f, 0.0f);
			VectorRegister4Float B = MakeVectorRegister(713.519043f, 205.570648f, 172.950577f, 0.0f);
			VectorRegister4Float C = MakeVectorRegister(686.582520f, 252.185165f, 365.669464f, 0.0f);
			FTriangleRegister Tri{ A,B,C };

			VectorRegister4Double StartRotation = MakeVectorRegister(0.0f, 0.0f, 0.0f, 1.0f);
			VectorRegister4Double StartPosition = MakeVectorRegister(727.730835f, 221.037231f, 363.199402f, 0.0f);
			VectorRegister4Double RayDirection = MakeVectorRegister(0.0423882306f, 0.0380843617f, -0.998375118f, 0.0f);

			FRealSingle RayLength = 200.0;
			FRealSingle OutTime;
			VectorRegister4Double OutPosition;
			VectorRegister4Double OutNormal;

			const bool bDidHit = GJKRaycast2ImplSimd<VectorRegister4Double>(Tri, Sphere, StartRotation, StartPosition, RayDirection, RayLength, OutTime, OutPosition, OutNormal, true, GlobalVectorConstants::Float1000);
			EXPECT_FALSE(bDidHit);
		}
	}

	GTEST_TEST(GJKTests, GJK_SweepSphereConvexMesh)
	{
		// Test case bug that was not working on XSX, and maybe on Switch
		const TArray<Chaos::FConvex::FVec3Type> ConvexVertices
		{
			{ -1320.50159f, 461.919220f, 18.5806618f },
			{ -1321.64062f, 464.953125f, -81.3668213f },
			{ -723.913879f, 585.305603f, -65.5020828f },
			{ -1026.47400f, -493.330170f, -13.7665596f },
			{ -1027.61304f, -490.296295f, -113.714035f },
			{ -723.348694f, 578.755981f, 34.2816010f },
			{ -668.640442f, -417.582031f, -31.4263611f },
			{ -669.205688f, -411.032410f, -131.210037f }
		};

		const Chaos::FConvexPtr ConvexPtr(new Chaos::FConvex(ConvexVertices, 0.0f));
		const Chaos::TImplicitObjectInstanced<Chaos::FConvex> Convex(ConvexPtr);
		const Chaos::FSphere Sphere(Chaos::FVec3::ZeroVector, 40.0);

		const UE::Math::TQuat BToARotation{ -0.0135237072, -0.149320737, 0.0850137547, 0.985034525 };
		const Chaos::FVec3 StartPoint{ -2305.07544, -225.373749, 362.317993 };
		const Chaos::FVec3 RayDir{ 0.958641946, 0.168052554, -0.229703903 };
		const Chaos::FReal RayLength{ 1214.51257 };
		const bool bComputeMTD{ true };
		const Chaos::FVec3 InitialDir{ 270.538879, -2088.81055, -1029.13525 };

		Chaos::FReal TimeOffImpact{ };
		Chaos::FVec3 Position{ };
		Chaos::FVec3 Normal{ };
		bool bHit = Chaos::GJKRaycast2<Chaos::FReal>(Convex, Sphere, Chaos::FRigidTransform3(UE::Math::TTransform<double>(BToARotation, StartPoint)), RayDir, RayLength, TimeOffImpact, Position, Normal, 0, true, InitialDir, 0.0);
		EXPECT_FALSE(bHit);
	}

	GTEST_TEST(GJKTests, GJK_SweepConvexMeshConvexMesh)
	{
		const TArray<Chaos::FConvex::FVec3Type> ConvexVerticesA
		{
			{ 372.876678f, 1253.23181f, 1937.96875f },
			{ -516.649231f, 1253.23181f, 1937.96875f },
			{ -516.649231f, 1253.23181f, 1873.81213f },
			{ -516.649231f, 1253.23181f, 1873.81213f },
			{ 372.876678f, 1253.23181f, 1873.81213f },
			{ 372.876678f, -2157.82104f, 2122.00000f },
			{ 372.876678f, -2157.82104f, 2057.84326f },
			{ -516.649231f, -2157.82104f, 2057.84326f },
			{ -516.649231f, -2157.82104f, 2122.00000f }
		};
		const Chaos::FConvexPtr ConvexPtrA(new Chaos::FConvex(ConvexVerticesA, 0.0f));
		const Chaos::TImplicitObjectInstanced<Chaos::FConvex> ConvexA(ConvexPtrA);

		const TArray<Chaos::FConvex::FVec3Type> ConvexVerticesB
		{
			{ 73.4703674f, -61.1765442f, -0.19269731f },
			{ -58.2119789f, -43.0196724f, 150.211761f },
			{ 56.7236938f, -43.0196762f, 150.211761f },
			{ 73.4703674f, 87.8292542f, -0.19269731f },
			{ -73.4703674f, 87.8292542f, -0.19269731f },
			{ -73.4703751f, -61.1765366f, -0.19269731f },
			{ -45.8896980f, 52.8858337f, 150.211761f },
			{ 44.4014816f, 52.8858109f, 150.211761f }
		};
		const Chaos::FConvexPtr ConvexPtrB(new Chaos::FConvex(ConvexVerticesB, 0.0f));
		const Chaos::TImplicitObjectInstanced<Chaos::FConvex> ConvexB(ConvexPtrB);
		{
			// Test case bug that was fixed on PS5
			const UE::Math::TQuat BToARotation{ -5.54229738e-017, -3.12762759e-018, 0.02172332, 0.99976402 };
			const Chaos::FVec3 StartPoint{ -57.6924553, -1033.34961, 2064.99536 };
			const Chaos::FVec3 RayDir{ -0.00042043, 0.99854773, -0.05387306 };
			const Chaos::FReal RayLength{ 5.3171567916870117 };
			const Chaos::FVec3 InitialDir{ 173.583862, 1020.29828, -2064.99536 };
			const bool bComputeMTD{ true };
		
			Chaos::FReal Time{ };
			Chaos::FVec3 Position{ };
			Chaos::FVec3 Normal{ };
			bool bHit{ Chaos::GJKRaycast2<Chaos::FReal>(ConvexA, ConvexB, Chaos::FRigidTransform3(UE::Math::TTransform<double>(BToARotation, StartPoint)), RayDir, RayLength, Time, Position, Normal, 0, true, InitialDir, 0.0) };
			EXPECT_TRUE(bHit);
			EXPECT_NEAR(-0.000022, Time, 1e-5);
		}
		{
			// Test case falling on PS5
			const UE::Math::TQuat BToARotation{ -5.54229738e-017, -3.12762759e-018, 0.02172332, 0.99976402 };
			const Chaos::FVec3 StartPoint{ -57.4445953, -974.640625, 2061.82788 };
			const Chaos::FVec3 RayDir{ -0.00025997, 0.99854779, -0.05387305 };
			const Chaos::FReal RayLength{ 2.4696500301361084 };
			const Chaos::FVec3 InitialDir{ 166.732468, 961.989868, -2061.82788 };
			const bool bComputeMTD{ true };

			Chaos::FReal Time{ };
			Chaos::FVec3 Position{ };
			Chaos::FVec3 Normal{ };
			bool bHit{ Chaos::GJKRaycast2<Chaos::FReal>(ConvexA, ConvexB, Chaos::FRigidTransform3(UE::Math::TTransform<double>(BToARotation, StartPoint)), RayDir, RayLength, Time, Position, Normal, 0, true, InitialDir, 0.0) };
			EXPECT_TRUE(bHit);
			EXPECT_NEAR(0.0, Time, 1e-3);
		}
	}
}