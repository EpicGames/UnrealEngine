// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestGJK.h"
#include "HeadlessChaos.h"

#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/GJK.h"
#include "Chaos/ImplicitObjectScaled.h"

namespace ChaosTest
{
	using namespace Chaos;

	void GJKSphereSphereDistanceTest()
	{
		const FReal Tolerance = (FReal)1e-3;

		FVec3 NearestA = { 0,0,0 };
		FVec3 NearestB = { 0,0,0 };
		FReal Distance = 0;
		FVec3 Normal = {0, 0, 1};
		// Fail - overlapping
		{
			Chaos::FSphere A(FVec3(12, 0, 0), 5);
			Chaos::FSphere B(FVec3(4, 0, 0), 2);
			const FRigidTransform3 BToATm = FRigidTransform3(FVec3(2, 0, 0), FRotation3::FromIdentity());
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKCoreShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			EXPECT_NE(Result, EGJKDistanceResult::Separated);
		}

		// Success - not overlapping
		{
			Chaos::FSphere A(FVec3(12, 0, 0), 5);
			Chaos::FSphere B(FVec3(4, 0, 0), 2);
			const FRigidTransform3 BToATm = FRigidTransform3::Identity;
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKCoreShape(A), 
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (FReal)1, Tolerance);
			EXPECT_NEAR(NearestA.X, (FReal)7, Tolerance);
			EXPECT_NEAR(NearestA.Y, (FReal)0, Tolerance);
			EXPECT_NEAR(NearestA.Z, (FReal)0, Tolerance);
			EXPECT_NEAR(NearestB.X, (FReal)6, Tolerance);
			EXPECT_NEAR(NearestB.Y, (FReal)0, Tolerance);
			EXPECT_NEAR(NearestB.Z, (FReal)0, Tolerance);
		}

		// Success - not overlapping
		{
			Chaos::FSphere A(FVec3(0, 0, 0), 2);
			Chaos::FSphere B(FVec3(0, 0, 0), 2);
			FVec3 BPos = FVec3(3, 3, 0);
			const FRigidTransform3 BToATm = FRigidTransform3(BPos, FRotation3::FromIdentity());
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKCoreShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			FVec3 CenterDelta = (B.GetCenterf() + BPos) - A.GetCenterf();
			FVec3 CenterDir = CenterDelta.GetSafeNormal();
			EXPECT_NEAR(Distance, CenterDelta.Size() - (A.GetRadiusf() + B.GetRadiusf()), Tolerance);
			EXPECT_NEAR(NearestA.X, A.GetCenterf().X + A.GetRadiusf() * CenterDir.X, Tolerance);
			EXPECT_NEAR(NearestA.Y, A.GetCenterf().Y + A.GetRadiusf() * CenterDir.Y, Tolerance);
			EXPECT_NEAR(NearestA.Z, A.GetCenterf().Z + A.GetRadiusf() * CenterDir.Z, Tolerance);
			EXPECT_NEAR(NearestB.X, B.GetCenterf().X - B.GetRadiusf() * CenterDir.X, Tolerance);
			EXPECT_NEAR(NearestB.Y, B.GetCenterf().Y - B.GetRadiusf() * CenterDir.Y, Tolerance);
			EXPECT_NEAR(NearestB.Z, B.GetCenterf().Z - B.GetRadiusf() * CenterDir.Z, Tolerance);
		}

		// Success - very close not overlapping
		{
			Chaos::FSphere A(FVec3(12, 0, 0), 5);
			Chaos::FSphere B(FVec3(4, 0, 0), 2);
			FVec3 BPos = FVec3(0.99, 0, 0);
			const FRigidTransform3 BToATm = FRigidTransform3(BPos, FRotation3::FromIdentity());
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKCoreShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (FReal)1 - BPos.X, Tolerance);
			EXPECT_NEAR(NearestA.X, (FReal)7, Tolerance);
			EXPECT_NEAR(NearestA.Y, (FReal)0, Tolerance);
			EXPECT_NEAR(NearestA.Z, (FReal)0, Tolerance);
			EXPECT_NEAR(NearestB.X, (FReal)6, Tolerance);
			EXPECT_NEAR(NearestB.Y, (FReal)0, Tolerance);
			EXPECT_NEAR(NearestB.Z, (FReal)0, Tolerance);
		}
	}

	TEST(TestGJKDistance, GJKSphereSphereDistanceTest)
	{
		GJKSphereSphereDistanceTest();
	}

		void GJKBoxSphereDistanceTest()
	{
		const FReal Tolerance = (FReal)2e-3;

		FVec3 NearestA = { 0,0,0 };
		FVec3 NearestB = { 0,0,0 };
		FReal Distance = 0;
		FVec3 Normal = { 0, 0, 1 };

		// Fail - overlapping
		{
			FAABB3 A(FVec3(5, -2, -2), FVec3(8, 2, 2));
			Chaos::FSphere B(FVec3(2, 0, 0), 2);
			const FRigidTransform3 BToATm = FRigidTransform3(FVec3(2, 0, 0), FRotation3::FromIdentity());
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			EXPECT_NE(Result, EGJKDistanceResult::Separated);
		}

		// Success - not overlapping - mid-face near point
		{
			FAABB3 A(FVec3(5, -2, -2), FVec3(8, 2, 2));
			Chaos::FSphere B(FVec3(2, 0, 0), 2);
			const FRigidTransform3 BToATm = FRigidTransform3::Identity;
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (FReal)1, Tolerance);
			EXPECT_NEAR(NearestA.X, (FReal)5, Tolerance);
			EXPECT_NEAR(NearestA.Y, (FReal)0, Tolerance);
			EXPECT_NEAR(NearestA.Z, (FReal)0, Tolerance);
			EXPECT_NEAR(NearestB.X, (FReal)4, Tolerance);
			EXPECT_NEAR(NearestB.Y, (FReal)0, Tolerance);
			EXPECT_NEAR(NearestB.Z, (FReal)0, Tolerance);
		}
		// Other way round
		{
			FAABB3 A(FVec3(5, -2, -2), FVec3(8, 2, 2));
			Chaos::FSphere B(FVec3(2, 0, 0), 2);
			const FRigidTransform3 BToATm = FRigidTransform3::Identity;
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (FReal)1, Tolerance);
			EXPECT_NEAR(NearestA.X, (FReal)5, Tolerance);
			EXPECT_NEAR(NearestA.Y, (FReal)0, Tolerance);
			EXPECT_NEAR(NearestA.Z, (FReal)0, Tolerance);
			EXPECT_NEAR(NearestB.X, (FReal)4, Tolerance);
			EXPECT_NEAR(NearestB.Y, (FReal)0, Tolerance);
			EXPECT_NEAR(NearestB.Z, (FReal)0, Tolerance);
		}

		// Success - not overlapping - vertex near point
		{
			FAABB3 A(FVec3(5, 2, 2), FVec3(8, 4, 4));
			Chaos::FSphere B(FVec3(2, 0, 0), 2);
			const FRigidTransform3 BToATm = FRigidTransform3::Identity;
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			FVec3 NearPointOnA = A.Min();
			FVec3 SphereNearPointDir = (NearPointOnA - B.GetCenterf()).GetSafeNormal();
			FVec3 NearPointOnB = B.GetCenterf() + SphereNearPointDir * B.GetRadiusf();
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (NearPointOnA - NearPointOnB).Size(), Tolerance);
			EXPECT_NEAR(NearestA.X, NearPointOnA.X, Tolerance);
			EXPECT_NEAR(NearestA.Y, NearPointOnA.Y, Tolerance);
			EXPECT_NEAR(NearestA.Z, NearPointOnA.Z, Tolerance);
			EXPECT_NEAR(NearestB.X, NearPointOnB.X, Tolerance);
			EXPECT_NEAR(NearestB.Y, NearPointOnB.Y, Tolerance);
			EXPECT_NEAR(NearestB.Z, NearPointOnB.Z, Tolerance);
		}
		// Other way round
		{
			FAABB3 A(FVec3(5, 2, 2), FVec3(8, 4, 4));
			Chaos::FSphere B(FVec3(2, 0, 0), 2);
			const FRigidTransform3 BToATm = FRigidTransform3::Identity;
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			FVec3 NearPointOnA = A.Min();
			FVec3 SphereNearPointDir = (NearPointOnA - B.GetCenterf()).GetSafeNormal();
			FVec3 NearPointOnB = B.GetCenterf() + SphereNearPointDir * B.GetRadiusf();
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (NearPointOnA - NearPointOnB).Size(), Tolerance);
			EXPECT_NEAR(NearestA.X, NearPointOnA.X, Tolerance);
			EXPECT_NEAR(NearestA.Y, NearPointOnA.Y, Tolerance);
			EXPECT_NEAR(NearestA.Z, NearPointOnA.Z, Tolerance);
			EXPECT_NEAR(NearestB.X, NearPointOnB.X, Tolerance);
			EXPECT_NEAR(NearestB.Y, NearPointOnB.Y, Tolerance);
			EXPECT_NEAR(NearestB.Z, NearPointOnB.Z, Tolerance);
		}

		// Rotated
		{
			FAABB3 A(FVec3(-2, -2, -2), FVec3(4, 4, 4));
			Chaos::FSphere B(FVec3(0, 0, 0), 2);
			FRigidTransform3 BToATm = FRigidTransform3(FVec3(8, 0, 0), FRotation3::FromAxisAngle(FVec3(0, 1, 0), FMath::DegreesToRadians(45)));	// Rotation won't affect contact depth, but does affect local contact position
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			FVec3 NearPointOnA = FVec3(4, 0, 0);
			FVec3 BPos = BToATm.TransformPositionNoScale(FVec3(B.GetCenterf()));
			FVec3 NearPointDir = (NearPointOnA - BPos).GetSafeNormal();
			FVec3 NearPointOnB = BPos + NearPointDir * B.GetRadiusf();
			FVec3 NearPointOnBLocal = BToATm.InverseTransformPositionNoScale(NearPointOnB);
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (NearPointOnA - NearPointOnB).Size(), Tolerance);
			EXPECT_NEAR(NearestA.X, NearPointOnA.X, Tolerance);
			EXPECT_NEAR(NearestA.Y, NearPointOnA.Y, Tolerance);
			EXPECT_NEAR(NearestA.Z, NearPointOnA.Z, Tolerance);
			EXPECT_NEAR(NearestB.X, NearPointOnBLocal.X, Tolerance);
			EXPECT_NEAR(NearestB.Y, NearPointOnBLocal.Y, Tolerance);
			EXPECT_NEAR(NearestB.Z, NearPointOnBLocal.Z, Tolerance);
		}
		// Other way round
		{
			FAABB3 A(FVec3(-2, -2, -2), FVec3(4, 4, 4));
			Chaos::FSphere B(FVec3(0, 0, 0), 2);
			FRigidTransform3 BToATm = FRigidTransform3(FVec3(-8, 0, 0), FRotation3::FromAxisAngle(FVec3(0, 1, 0), FMath::DegreesToRadians(45)));
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKCoreShape(B),
				TGJKShapeTransformed(A, BToATm),
				GJKDistanceInitialVFromRelativeTransform(B, A, BToATm),
				Distance, NearestB, NearestA, Normal);
			FVec3 NearPointOnA = FVec3(4, 0, 4);
			FVec3 BPos = BToATm.InverseTransformPositionNoScale(FVec3(B.GetCenterf()));
			FVec3 NearPointDir = (NearPointOnA - BPos).GetSafeNormal();
			FVec3 NearPointOnB = BPos + NearPointDir * B.GetRadiusf();
			FVec3 NearPointOnBLocal = BToATm.TransformPositionNoScale(NearPointOnB);
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (NearPointOnA - NearPointOnB).Size(), Tolerance);
			EXPECT_NEAR(NearestA.X, NearPointOnA.X, Tolerance);
			EXPECT_NEAR(NearestA.Y, NearPointOnA.Y, Tolerance);
			EXPECT_NEAR(NearestA.Z, NearPointOnA.Z, Tolerance);
			EXPECT_NEAR(NearestB.X, NearPointOnBLocal.X, Tolerance);
			EXPECT_NEAR(NearestB.Y, NearPointOnBLocal.Y, Tolerance);
			EXPECT_NEAR(NearestB.Z, NearPointOnBLocal.Z, Tolerance);
		}
	
		// Success - specific test case that initially failed (using incorrect initialization of V which works for Overlap but not Distance)
		{
			FAABB3 A(FVec3(5, -2, 2), FVec3(8, 2, 4));
			Chaos::FSphere B(FVec3(2, 0, 0), 2);

			bool bOverlap = GJKIntersection<FReal>(A, B, FRigidTransform3::Identity);
			EXPECT_FALSE(bOverlap);

			const FRigidTransform3 BToATm = FRigidTransform3::Identity;
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			FVec3 NearPointOnA = FVec3(5, 0, 2);
			FVec3 NearPointDir = (NearPointOnA - B.GetCenterf()).GetSafeNormal();
			FVec3 NearPointOnB = B.GetCenterf() + NearPointDir * B.GetRadiusf();
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (NearPointOnA - NearPointOnB).Size(), Tolerance);
			EXPECT_NEAR(NearestA.X, NearPointOnA.X, Tolerance);
			EXPECT_NEAR(NearestA.Y, NearPointOnA.Y, Tolerance);
			EXPECT_NEAR(NearestA.Z, NearPointOnA.Z, Tolerance);
			EXPECT_NEAR(NearestB.X, NearPointOnB.X, Tolerance);
			EXPECT_NEAR(NearestB.Y, NearPointOnB.Y, Tolerance);
			EXPECT_NEAR(NearestB.Z, NearPointOnB.Z, Tolerance);
		}
	}

	TEST(TestGJKDistance, GJKBoxSphereDistanceTest)
	{
		GJKBoxSphereDistanceTest();
	}

	void GJKBoxCapsuleDistanceTest()
	{
		FVec3 NearestA = { 0,0,0 };
		FVec3 NearestB = { 0,0,0 };
		FReal Distance = 0;
		FVec3 Normal = { 0, 0, 1 };

		// Fail - overlapping
		{
			FAABB3 A(FVec3(5, -2, -2), FVec3(8, 2, 2));
			FCapsule B(FVec3(2, -2, 0), FVec3(2, 2, 0), 2);
			const FRigidTransform3 BToATm = FRigidTransform3(FVec3(2, 0, 0), FRotation3::FromIdentity());
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			EXPECT_NE(Result, EGJKDistanceResult::Separated);
		}

		// Success - not overlapping, capsule axis parallel to nearest face (near points on cylinder and box face)
		{
			FAABB3 A(FVec3(5, -2, -2), FVec3(8, 2, 2));
			FCapsule B(FVec3(2, 0, -1), FVec3(2, 0, 2), 2);
			const FRigidTransform3 BToATm = FRigidTransform3::Identity;
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			const FReal Tolerance = (FReal)2e-3;
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (FReal)1, Tolerance);
			EXPECT_NEAR(NearestA.X, (FReal)5, Tolerance);
			EXPECT_NEAR(NearestA.Y, (FReal)0, Tolerance);
			EXPECT_GT(NearestA.Z, (FReal)-2-Tolerance);
			EXPECT_LT(NearestA.Z, (FReal)2+Tolerance);
			EXPECT_NEAR(NearestB.X, (FReal)4, Tolerance);
			EXPECT_NEAR(NearestB.Y, (FReal)0, Tolerance);
			EXPECT_GT(NearestB.Z, (FReal)-1-Tolerance);
			EXPECT_LT(NearestB.Z, (FReal)2+Tolerance);
		}

		// Success - not overlapping, capsule axis at angle to nearest face (near points on end-cap and box edge)
		{
			FAABB3 A(FVec3(5, -2, -2), FVec3(8, 2, 2));
			FCapsule B(FVec3(-2, 0, 3), FVec3(2, 0, -3), 2);
			const FRigidTransform3 BToATm = FRigidTransform3::Identity;
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			FVec3 ExpectedNearestA = FVec3(5, 0, -2);
			FVec3 ExpectedDir = (ExpectedNearestA - B.GetX2f()).GetSafeNormal();
			FVec3 ExpectedNearestB = B.GetX2f() + ExpectedDir * B.GetRadiusf();

			const FReal Tolerance = (FReal)2e-3;
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (ExpectedNearestB - ExpectedNearestA).Size(), Tolerance);
			EXPECT_NEAR(NearestA.X, (FReal)ExpectedNearestA.X, Tolerance);
			EXPECT_NEAR(NearestA.Y, (FReal)ExpectedNearestA.Y, Tolerance);
			EXPECT_NEAR(NearestA.Z, (FReal)ExpectedNearestA.Z, Tolerance);
			EXPECT_NEAR(NearestB.X, (FReal)ExpectedNearestB.X, Tolerance);
			EXPECT_NEAR(NearestB.Y, (FReal)ExpectedNearestB.Y, Tolerance);
			EXPECT_NEAR(NearestB.Z, (FReal)ExpectedNearestB.Z, Tolerance);
		}

		// Success - not overlapping, near point partway down wall of capsule
		{
			FCapsule A(FVec3(4, 0, -1), FVec3(4, 0, -7), 1);
			FAABB3 B(FVec3(-2, -2, -2), FVec3(2, 2, 2));
			const FRigidTransform3 BToATm = FRigidTransform3::Identity;
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKCoreShape(A),
				TGJKShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			FVec3 ExpectedNearestA = FVec3(3, 0, (FReal)-1.5);
			FVec3 ExpectedNearestB = FVec3(2, 0, (FReal)-1.5);

			const FReal Tolerance = (FReal)2e-3;
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (FReal)1, Tolerance);
			EXPECT_NEAR(NearestA.X, (FReal)ExpectedNearestA.X, Tolerance);
			EXPECT_NEAR(NearestA.Y, (FReal)ExpectedNearestA.Y, Tolerance);
			EXPECT_LT(NearestA.Z, (FReal)ExpectedNearestA.Z + (FReal)0.5 + Tolerance);
			EXPECT_GT(NearestA.Z, (FReal)ExpectedNearestA.Z - (FReal)0.5 - Tolerance);
			EXPECT_NEAR(NearestB.X, (FReal)ExpectedNearestB.X, Tolerance);
			EXPECT_NEAR(NearestB.Y, (FReal)ExpectedNearestB.Y, Tolerance);
			EXPECT_NEAR(NearestB.Z, (FReal)NearestA.Z, Tolerance);
		}

		// Success - not overlapping, near point partway down wall of capsule.
		// Same result as above, but using transform rather than the shape's built-in offsets.
		{
			FCapsule A(FVec3(0, 0, -3), FVec3(0, 0, 3), 1);
			FAABB3 B(FVec3(-2, -2, -2), FVec3(2, 2, 2));
			FRigidTransform3 BToATm = FRigidTransform3(FVec3(-4, 0, 4), FRotation3::FromIdentity());
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKCoreShape(A),
				TGJKShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			FVec3 ExpectedNearestA = FVec3(-1, 0, (FReal)2);
			FVec3 ExpectedNearestB = FVec3(2, 0, (FReal)-2);

			const FReal Tolerance = (FReal)2e-3;
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (FReal)1, Tolerance);
			EXPECT_NEAR(NearestA.X, (FReal)ExpectedNearestA.X, Tolerance);
			EXPECT_NEAR(NearestA.Y, (FReal)ExpectedNearestA.Y, Tolerance);
			EXPECT_LT(NearestA.Z, (FReal)ExpectedNearestA.Z + (FReal)0.5 + Tolerance);
			EXPECT_GT(NearestA.Z, (FReal)ExpectedNearestA.Z - (FReal)0.5 - Tolerance);
			EXPECT_NEAR(NearestB.X, (FReal)ExpectedNearestB.X, Tolerance);
			EXPECT_NEAR(NearestB.Y, (FReal)ExpectedNearestB.Y, Tolerance);
			EXPECT_NEAR(NearestB.Z + BToATm.GetTranslation().Z, (FReal)NearestA.Z, Tolerance);
		}
	}


	TEST(TestGJKDistance, GJKBoxCapsuleDistanceTest)
	{
		GJKBoxCapsuleDistanceTest();
	}


	void GJKBoxCapsuleDistanceIterationCountTest()
	{
		FVec3 NearestA = { 0,0,0 };
		FVec3 NearestB = { 0,0,0 };
		FReal Distance = 0;
		FVec3 Normal = { 0, 0, 1 };

		// Capsule-box takes number of iterations at the moment (we can improve that with a better the choice of Initial V)
		// so test that we still get an approximate answer with less iterations
		{
			FAABB3 A(FVec3(5, -2, -2), FVec3(8, 2, 2));
			FCapsule B(FVec3(-2, 0, 3), FVec3(2, 0, -3), 2);
			FReal Epsilon = (FReal)1e-6;
			int32 MaxIts = 5;
			const FRigidTransform3 BToATm = FRigidTransform3::Identity;
			EGJKDistanceResult Result = GJKDistance<FReal>(
				TGJKShape(A),
				TGJKCoreShapeTransformed(B, BToATm),
				GJKDistanceInitialVFromRelativeTransform(A, B, BToATm),
				Distance, NearestA, NearestB, Normal);
			FVec3 ExpectedNearestA = FVec3(5, 0, -2);
			FVec3 ExpectedDir = (ExpectedNearestA - B.GetX2f()).GetSafeNormal();
			FVec3 ExpectedNearestB = B.GetX2f() + ExpectedDir * B.GetRadiusf();

			const FReal Tolerance = (FReal)0.3;
			EXPECT_EQ(Result, EGJKDistanceResult::Separated);
			EXPECT_NEAR(Distance, (ExpectedNearestB - ExpectedNearestA).Size(), Tolerance);
			EXPECT_NEAR(NearestA.X, (FReal)ExpectedNearestA.X, Tolerance);
			EXPECT_NEAR(NearestA.Y, (FReal)ExpectedNearestA.Y, Tolerance);
			EXPECT_NEAR(NearestA.Z, (FReal)ExpectedNearestA.Z, Tolerance);
			EXPECT_NEAR(NearestB.X, (FReal)ExpectedNearestB.X, Tolerance);
			EXPECT_NEAR(NearestB.Y, (FReal)ExpectedNearestB.Y, Tolerance);
			EXPECT_NEAR(NearestB.Z, (FReal)ExpectedNearestB.Z, Tolerance);
		}
	}


	TEST(TestGJKDistance, GJKBoxCapsuleDistanceIterationCountTest)
	{
		GJKBoxCapsuleDistanceIterationCountTest();
	}

}