// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"
#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/Island/IslandManager.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDNullConstraints.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/Utilities.h"
#include "Modules/ModuleManager.h"
#include "HAL/ConsoleManager.h"
#include "Chaos/Framework/Parallel.h"

namespace ChaosTest
{
	using namespace Chaos;

	class GraphEvolutionTests : public ::testing::TestWithParam<bool>
	{
		void SetUp()
		{
			SetPartialIslandSleepUnitTestDefaults();
		}

		void TearDown()
		{
			RestorePartialIslandSleepEditorDefaults();
		}

		void SetPartialIslandSleepUnitTestDefaults()
		{
			CVarPostStepWake = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.PostStepWakeThreshold"), false);
			check(CVarPostStepWake && CVarPostStepWake->IsVariableFloat());
			PostStepWake_EditorDefault = CVarPostStepWake->GetFloat();
			CVarPostStepWake->Set(PostStepWake_UnitTestDefault);

			CVarMomentumPropagation = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.MomentumPropagation"), false);
			check(CVarMomentumPropagation && CVarMomentumPropagation->IsVariableFloat());
			MomentumPropagation_EditorDefault = CVarMomentumPropagation->GetFloat();
			CVarMomentumPropagation->Set(MomentumPropagation_UnitTestDefault);

			CVarPartialSleepCollisionConstraintsOnly = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.CollisionConstraintsOnly"), false);
			check(CVarPartialSleepCollisionConstraintsOnly && CVarPartialSleepCollisionConstraintsOnly->IsVariableInt());
			PartialSleepCollisionConstraintsOnly_EditorDefault = CVarPartialSleepCollisionConstraintsOnly->GetInt();
			CVarPartialSleepCollisionConstraintsOnly->Set(PartialSleepCollisionConstraintsOnly_UnitTestDefault);

			CVarPartialWakePreIntegration = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.PartialWakePreIntegration"), false);
			check(CVarPartialWakePreIntegration && CVarPartialWakePreIntegration->IsVariableBool());
			bPartialWakePreIntegration_EditorDefault = CVarPartialWakePreIntegration->GetBool();
			CVarPartialWakePreIntegration->Set(bPartialWakePreIntegration_UnitTestDefault);

			CVarMinMotionlessRatio = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.MinimumMotionlessRatio"), false);
			check(CVarMinMotionlessRatio && CVarMinMotionlessRatio->IsVariableFloat());
			MinimumMotionlessThreshold_EditorDefault = CVarMinMotionlessRatio->GetFloat();
			CVarMinMotionlessRatio->Set(MinimumMotionlessThreshold_UnitTestDefault);

			CVarLinearWakeThresholdMultiplier = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.LinearWakeThresholdMultiplier"), false);
			check(CVarLinearWakeThresholdMultiplier && CVarLinearWakeThresholdMultiplier->IsVariableFloat());
			LinearWakeThresholdMultiplier_EditorDefault = CVarLinearWakeThresholdMultiplier->GetFloat();
			CVarLinearWakeThresholdMultiplier->Set(LinearWakeThresholdMultiplier_UnitTestDefault);

			CVarAngularWakeThresholdMultiplier = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.AngularWakeThresholdMultiplier"), false);
			check(CVarAngularWakeThresholdMultiplier && CVarAngularWakeThresholdMultiplier->IsVariableFloat());
			AngularWakeThresholdMultiplier_EditorDefault = CVarAngularWakeThresholdMultiplier->GetFloat();
			CVarAngularWakeThresholdMultiplier->Set(AngularWakeThresholdMultiplier_UnitTestDefault);

			CVarValidateConstraintSleepState = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.ValidateConstraintSleepState"), false);
			check(CVarValidateConstraintSleepState && CVarValidateConstraintSleepState->IsVariableBool());
			bValidateConstraintSleepState_EditorDefault = CVarValidateConstraintSleepState->GetBool();
			CVarValidateConstraintSleepState->Set(bValidateConstraintSleepState_UnitTestDefault);
		}

		void RestorePartialIslandSleepEditorDefaults()
		{
			CVarPostStepWake->Set(PostStepWake_EditorDefault);
			CVarMomentumPropagation->Set(MomentumPropagation_EditorDefault);
			CVarPartialSleepCollisionConstraintsOnly->Set(PartialSleepCollisionConstraintsOnly_EditorDefault);
			CVarPartialWakePreIntegration->Set(bPartialWakePreIntegration_EditorDefault);
			CVarMinMotionlessRatio->Set(MinimumMotionlessThreshold_EditorDefault);
			CVarLinearWakeThresholdMultiplier->Set(LinearWakeThresholdMultiplier_EditorDefault);
			CVarAngularWakeThresholdMultiplier->Set(AngularWakeThresholdMultiplier_EditorDefault);
			CVarValidateConstraintSleepState->Set(bValidateConstraintSleepState_EditorDefault);
		}

		// Default values for unit testing
		FRealSingle PostStepWake_UnitTestDefault = 0.0f;
		FRealSingle MomentumPropagation_UnitTestDefault = 0.0f;
		int32 PartialSleepCollisionConstraintsOnly_UnitTestDefault = 0;
		bool bPartialWakePreIntegration_UnitTestDefault = false;
		FRealSingle MinimumMotionlessThreshold_UnitTestDefault = 0.0f;
		FRealSingle LinearWakeThresholdMultiplier_UnitTestDefault = 1.0f;
		FRealSingle AngularWakeThresholdMultiplier_UnitTestDefault = 1.0f;
		bool bValidateConstraintSleepState_UnitTestDefault = true;

		// Storage for default values editor hard-coded in IslandManager.cpp
		FRealSingle PostStepWake_EditorDefault;
		FRealSingle MomentumPropagation_EditorDefault;
		int32 PartialSleepCollisionConstraintsOnly_EditorDefault;
		bool bPartialWakePreIntegration_EditorDefault;
		FRealSingle MinimumMotionlessThreshold_EditorDefault;
		FRealSingle LinearWakeThresholdMultiplier_EditorDefault;
		FRealSingle AngularWakeThresholdMultiplier_EditorDefault;
		bool bValidateConstraintSleepState_EditorDefault;

	public:
		// Storage of console variable pointer
		IConsoleVariable* CVarPostStepWake;
		IConsoleVariable* CVarMomentumPropagation;
		IConsoleVariable* CVarPartialSleepCollisionConstraintsOnly;
		IConsoleVariable* CVarPartialWakePreIntegration;
		IConsoleVariable* CVarMinMotionlessRatio;
		IConsoleVariable* CVarLinearWakeThresholdMultiplier;
		IConsoleVariable* CVarAngularWakeThresholdMultiplier;
		IConsoleVariable* CVarValidateConstraintSleepState;
	};

	class FGraphEvolutionTest
	{
	public:
		FGraphEvolutionTest(const int32 NumParticles, const bool bPartialSleeping)
			: UniqueIndices()
			, Particles(UniqueIndices)
			, PhysicalMaterials()
			, Evolution(Particles, PhysicalMaterials)
			, Constraints()
			, TickCount(0)
			, CVarPartialSleeping()
		{
			IslandManager = &Evolution.GetIslandManager();

			// Bind the constraints to the evolution
			Evolution.AddConstraintContainer(Constraints);

			Evolution.GetGravityForces().SetAcceleration(FVec3(0), 0);

			// Create some particles and constraints connecting them in a chain: 0-1-2-3-...-N
			ParticleHandles = Evolution.CreateDynamicParticles(NumParticles);

			for (FGeometryParticleHandle* ParticleHandle : ParticleHandles)
			{
				Evolution.EnableParticle(ParticleHandle);
			}

			CVarPartialSleeping = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep"), false);
			check(CVarPartialSleeping && CVarPartialSleeping->IsVariableBool());
			PartialSleepDefault = CVarPartialSleeping->GetBool();
			CVarPartialSleeping->Set(bPartialSleeping);
		}

		~FGraphEvolutionTest()
		{
			CVarPartialSleeping->Set(PartialSleepDefault);
		}

		// Connect all the particles in a chain
		void MakeChain()
		{
			for (int32 ParticleIndex = 0; ParticleIndex < ParticleHandles.Num() - 1; ++ParticleIndex)
			{
				ConstraintHandles.Add(Constraints.AddConstraint({ ParticleHandles[ParticleIndex], ParticleHandles[ParticleIndex + 1] }));
			}
		}

		// Treat particle0 like a kinematic floor with all the other particles sat on it
		void MakeFloor()
		{
			Evolution.SetParticleObjectState(ParticleHandles[0], EObjectStateType::Kinematic);

			for (int32 ParticleIndex = 0; ParticleIndex < ParticleHandles.Num() - 1; ++ParticleIndex)
			{
				ConstraintHandles.Add(Constraints.AddConstraint({ ParticleHandles[0], ParticleHandles[ParticleIndex + 1] }));
			}
		}

		void Advance()
		{
			Evolution.AdvanceOneTimeStep(FReal(1.0 / 60.0));
			++TickCount;
		}

		void AdvanceUntilSleeping()
		{
			const int32 MaxIterations = 50;
			const int32 MaxTickCount = TickCount + MaxIterations;
			bool bIsSleeping = false;
			while (!bIsSleeping && (TickCount < MaxTickCount))
			{
				Advance();

				bIsSleeping = true;
				for (FPBDRigidParticleHandle* ParticleHandle : ParticleHandles)
				{
					if (ParticleHandle->IsDynamic() && !ParticleHandle->IsSleeping())
					{
						bIsSleeping = false;
					}
				}
			}

			EXPECT_TRUE(bIsSleeping);
			EXPECT_LT(TickCount, MaxTickCount);
		}

		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles;
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution;

		FPBDNullConstraints Constraints;
		TArray<FPBDRigidParticleHandle*> ParticleHandles;
		TArray<FPBDNullConstraintHandle*> ConstraintHandles;
		Private::FPBDIslandManager* IslandManager;

		int32 TickCount;

		IConsoleVariable* CVarPartialSleeping;
		bool PartialSleepDefault;
	};

	// Veryify that the Null Constraint mockup is working as intended. We can create the container and constraints, and they are correctly bound to the evolution
	TEST_P(GraphEvolutionTests, TestNullConstraint)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeChain();

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
	}

	// Start with a kinematic connected to a dynamic. Verify that removing the
	// constraint removes both particles.
	// This version explicitly removes the constraint from the graph.
	// 
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ak - Bd
	// =>	{}
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_KinematicDynamic_Remove)
	{
		FGraphEvolutionTest Test(2, GetParam());
		Test.MakeChain();

		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);

		Test.Advance();

		// Should have 1 island
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());

		// Disable the constraint and remove it from the graph
		Test.ConstraintHandles[0]->SetEnabled(false);
		Test.IslandManager->RemoveConstraint(Test.ConstraintHandles[0]);

		Test.Advance();

		// Should have no islands and all particles should have been removed
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 0);
		EXPECT_FALSE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_FALSE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_FALSE(Test.ConstraintHandles[0]->IsInConstraintGraph());
	}

	// Start with a kinematic connected to a dynamic. Verify that removing the
	// constraint removes both particles.
	// This version has the constraint removed by making all particles kinematic
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ak - Bd
	// =>	Ak - Bk
	// =>	{}
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_KinematicDynamic_Remove2)
	{
		FGraphEvolutionTest Test(2, GetParam());
		Test.MakeChain();

		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);

		Test.Advance();

		// Should have 1 island
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());

		// Make the other particle kinematic
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Kinematic);

		Test.Advance();

		// Should have no islands and all particles should have been removed
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 0);
		EXPECT_FALSE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_FALSE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_FALSE(Test.ConstraintHandles[0]->IsInConstraintGraph());
	}

	// Start with a kinematic connected to a sleeping dynamic. Verify that removing the
	// kinematic removes both particles from the graph (because we do not keep islands 
	// unless there are constraints in them), but the dynamic particle is now awake.
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ak - Bs
	// =>	{} (B is dynamic/awake but the graph is empty)
	//
	// This tests a bug where we were not waking a particle if we removed all other particles 
	// from its island. We now defer island destruction to after sleep handling.
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_KinematicDynamic_RemoveKinematic)
	{
		FGraphEvolutionTest Test(2, GetParam());
		Test.MakeChain();

		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);

		Test.AdvanceUntilSleeping();

		// Should have 1 island
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());

		// B should be asleep
		EXPECT_TRUE(Test.ParticleHandles[1]->IsSleeping());

		// Remove A
		Test.Evolution.DisableParticle(Test.ParticleHandles[0]);

		// A and the constraint should have been removed from the graph
		EXPECT_FALSE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_FALSE(Test.ConstraintHandles[0]->IsInConstraintGraph());

		// B will be removed from the graph because we do not track islands without
		// constraints, but not until the next tick. For now it will still be in
		// the graph and still asleep.
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[1]->IsSleeping());

		// Tick physics. This will update the graph, waking B's island and therefore B.
		// B's island will then be destroyed because it has no constraints.
		Test.Advance();

		// Graph should be empty
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 0);
		EXPECT_FALSE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_FALSE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_FALSE(Test.ConstraintHandles[0]->IsInConstraintGraph());

		// B should be awake
		EXPECT_FALSE(Test.ParticleHandles[1]->IsSleeping());
	}

	// Start with an island containing 4 particle connected in a chain, then make the second one kinematic.
	// Check that the island splits.
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ad - Bd - Cd - Dd 
	// =>	Ad - Bk   Bk - Cd - Dd
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_ToKinematic)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeChain();

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);

		// Convert particle B to kinematic to split the islands: A-B, B-C-D
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Kinematic);

		Test.Advance();

		// Should now have 2 islands
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 2);

		// A should be in its own island
		EXPECT_EQ(Test.IslandManager->GetParticleIsland(Test.ParticleHandles[0])->GetNumParticles(), 1);

		// C and D should be in same island
		EXPECT_EQ(Test.IslandManager->GetParticleIsland(Test.ParticleHandles[2]), Test.IslandManager->GetParticleIsland(Test.ParticleHandles[3]));

		// B should be in 2 islands
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 2);
	}

	// Start with an island containing 2 particle connected in a chain, then invalidate one of the particles.
	// Check that the particles and constraints are removed from the graph and then re-added at the next tick.
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ad - Bd
	// =>	Ad   Bd
	// =>	Ad - Bd
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_Invalidate)
	{
		FGraphEvolutionTest Test(2, GetParam());
		Test.MakeChain();

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());

		// Invalidate B
		Test.Evolution.InvalidateParticle(Test.ParticleHandles[1]);

		// Constraint was kicked from the graph, but particles remain until
		// explicitly removed or they have no constraint on next update
		EXPECT_TRUE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_FALSE(Test.ConstraintHandles[0]->IsInConstraintGraph());

		Test.Advance();

		// Everything was added back to the graph
		EXPECT_TRUE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
	}

	// An isolated stationary particle with no gravity should go to sleep
	TEST_P(GraphEvolutionTests, TestConstraintGraph_ParticleSleep_Isolated)
	{
		FGraphEvolutionTest Test(1, GetParam());
		Test.Evolution.GetGravityForces().SetAcceleration(FVec3(0), 0);

		// Make all the particles sleep
		Test.AdvanceUntilSleeping();

		// Particle should be asleep and it should have taken 21 ticks (ChaosSolverCollisionDefaultSleepCounterThreshold)
		EXPECT_TRUE(Test.ParticleHandles[0]->IsSleeping());
		EXPECT_EQ(Test.TickCount, 21);
	}

	// Wait for all particles to go to sleep naturally (i.e., as part of the tick and not by explicitly setting the state)
	// then check that the islands are preserved.
	TEST_P(GraphEvolutionTests, TestConstraintGraph_ParticleSleep_Natural)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeChain();

		Test.Advance();

		// All constraints in graph in 1 island that is awake
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsInConstraintGraph());
		EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());

		// Make all the particles sleep
		Test.AdvanceUntilSleeping();

		// Island should be asleep
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
	}

	// Sleep all particles in the scene and ensure that the island manager puts the island to sleep
	// but retains all the constraints and particles in the island.
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_ParticleSleep_Manual)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeChain();

		Test.Advance();

		// All constraints in graph in 1 island that is awake
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsInConstraintGraph());
		EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());

		// Make all the particles sleep
		for (FPBDRigidParticleHandle* ParticleHandle : Test.ParticleHandles)
		{
			Test.Evolution.SetParticleObjectState(ParticleHandle, EObjectStateType::Sleeping);
		}

		Test.Advance();

		// Island should be asleep
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
	}

	// Start with an island containing 4 particle connected in a chain, then make the middle two kinematic.
	// This makes the B-C constraint kinematic which means it does not belong in any island and is kicked 
	// out of the graph (the edge is deleted).
	// Check that the island manager handles kinematic-kinematic constraints
	//		A-B-C-D 
	// =>	A-B  C-D
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_KinematicKinematic)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeChain();

		Test.Advance();

		// All constraints in graph in 1 island
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsInConstraintGraph());

		// Convert a particle B and C to kinematic to split the islands: A-B, C-D
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Kinematic);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[2], EObjectStateType::Kinematic);

		Test.Advance();

		// Constraint[1] was kicked from the graph
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
		EXPECT_FALSE(Test.ConstraintHandles[1]->IsInConstraintGraph());	// Not in graph
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsInConstraintGraph());

		// Should now have 2 islands
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 2);
	}


	// Same as TestConstraintGraph_KinematicKinematic but islands are sleeping when the change is made.
	// 
	// (d=dynamic, s=sleeping, k=kinematic)
	//		As - Bs - Cs - Ds
	// =>	As - Bk   Ck - Ds
	// =>	Ad - Bk   Ck - Ds
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_KinematicKinematic_Sleeping)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeChain();

		Test.Advance();

		// All constraints in graph in 1 island that is awake
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsInConstraintGraph());

		// Wait for the sleep state
		Test.AdvanceUntilSleeping();

		// Island should be asleep but still contain all the particles and constraints
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsInConstraintGraph());

		// Convert a particle B and C to kinematic to split the islands: A-B, C-D
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Kinematic);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[2], EObjectStateType::Kinematic);

		Test.Advance();

		// The kinematic-kinematic constraint was kicked from graph
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
		EXPECT_FALSE(Test.ConstraintHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsInConstraintGraph());

		// Island will have split into two sleeping islands
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 2);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_TRUE(Test.IslandManager->GetIsland(1)->IsSleeping());

		// Wake a dynamic particle
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Dynamic);

		Test.Advance();

		// Should now have 2 islands and only one awake particle
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 2);
		EXPECT_FALSE(Test.ParticleHandles[0]->IsSleeping());
		EXPECT_TRUE(Test.ParticleHandles[3]->IsSleeping());
	}

	// 3 objects sat on the floor awake. Make the floor dynamic.
	// This tests what happens when a kinematic in multiple islands gets converted to a dynamic.
	// 
	// (d=dynamic, s=sleeping, k=kinematic)
	// Bd   Cd   Dd          Bd   Cd   Dd
	//  \   |   /	   =>     \   |   /
	//      Ak		              Ad
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_KinematicToDynamic)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeFloor();

		Test.Advance();

		// Each particle in its own island (kinematic will be in all 3)
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

		// Convert A to dynamic which should merge all the islands into 1
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Dynamic);

		Test.Advance();

		// All particles in same island
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);
	}

	// 3 objects sat on the floor asleep. Move the kinematic floor by setting its transform.
	// This tests what happens when a non-moving kinematic in multiple islands starts moving.
	// 
	// (d=dynamic, s=sleeping, k=kinematic, km=kinematic, moving)
	// Bs   Cs   Ds          Bd   Cd   Dd
	//  \   |   /	   =>     \   |   /
	//      Ak		              Akm
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_MoveKinematicFloor)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeFloor();

		Test.AdvanceUntilSleeping();

		// Each particle in its own island (kinematic will be in all 3)
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

		// Teleport a particle by setting its transform explicitly
		const bool bIsTeleport = true;
		Test.Evolution.SetParticleKinematicTarget(Test.ParticleHandles[0], FKinematicTarget::MakePositionTarget(FVec3(0, 3, 0), FRotation3()));

		Test.Advance();

		// Each particle in its own island (kinematic will be in all 3)
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

		// All particles awake
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
	}

	// 3 objects sat on the floor asleep. Make the floor dynamic.
	// This tests what happens when a kinematic in multiple sleeping islands gets converted to a dynamic.
	// 
	// (d=dynamic, s=sleeping, k=kinematic)
	// Bs   Cs   Ds          Bd   Cd   Dd
	//  \   |   /	   =>     \   |   /
	//      Ak		              Ad
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_KinematicToDynamic_WithSleep1)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeFloor();

		Test.AdvanceUntilSleeping();

		// Each particle in its own island (kinematic will be in all 3)
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

		// Convert A to dynamic which should merge all the islands into 1
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Dynamic);

		Test.Advance();

		// All particles in one awake island
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

		// All particles awake
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
	}

	// 3 Objects sat on the floor asleep. Make the floor dynamic and asleep.
	// This tests that adding a sleeping body to an island does not wake it.
	// This is required for streaming to work which adds bodies over multiple frames.
	// 
	// (d=dynamic, s=sleeping, k=kinematic)
	// Bs   Cs   Ds          Bs   Cs   Ds
	//  \   |   /      =>     \   |   /
	//      Ak                    As
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_KinematicToDynamic_WithSleep2)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeFloor();

		Test.AdvanceUntilSleeping();

		// Each particle in its own island (kinematic will be in all 3)
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

		// Convert A to dynamic sleeping which should merge all the islands into 1 but leave it asleep
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Sleeping);

		Test.Advance();

		// All particles in one asleep island
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

		// All particles asleep
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());

	}

	// 3 Objects sat on the floor, 2 asleep and 1 awake. Make the floor dynamic and asleep.
	// In this case we should get 1 awake island and all particles should wake.
	// 
	// Island sleeping: (d=dynamic, s=sleeping, k=kinematic)
	// Bs   Cs   Dd          Bd   Cd   Dd
	//  \   |   /      =>     \   |   /
	//      Ak	                  Ad
	//
	// Partial island sleeping: (d=dynamic, s=sleeping, k=kinematic)
	// Bs   Cs   Dd          Bs   Cs   Dd
	//  \   |   /      =>     \   |   /
	//      Ak	                  As
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_KinematicToDynamic_WithSleep3)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeFloor();

		Test.AdvanceUntilSleeping();

		// Wake D
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[3], EObjectStateType::Dynamic);

		Test.Advance();

		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());

		// Each particle in its own island (kinematic will be in all 3)
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

		// Convert A to dynamic sleeping
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Sleeping);

		Test.Advance();

		if (!Test.CVarPartialSleeping->GetBool()) // island sleeping
		{
			// All particles in one awake island (D was awake so it would wake the island)
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 1);
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

			// All particles awake
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		}
		else // partial island sleeping
		{
			// All particles are asleep except for D which remains awake
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 1);
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

			// All particles except for D asleep
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		}
	}

	// 3 Objects sat on the floor, 2 asleep and 1 awake. Make the floor dynamic and asleep.
	// Same as TestConstraintGraph_KinematicToDynamic_WithSleep3 except we wake a different particle to 
	// be sure we weren't just lucky above (when we make a kinematic into a dynamic we add it to one
	// of the islands it is in. This is testing that this is ok).
	// 
	// Island sleeping (d=dynamic, s=sleeping, k=kinematic)
	// Bd   Cs   Ds          Bd   Cd   Dd
	//  \   |   /      =>     \   |   /
	//      Ak	                  Ad
	//
	// Partial island sleeping (d=dynamic, s=sleeping, k=kinematic)
	// Bd   Cs   Ds          Bd   Cs   Ds
	//  \   |   /      =>     \   |   /
	//      Ak	                  As
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_KinematicToDynamic_WithSleep4)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeFloor();

		Test.AdvanceUntilSleeping();

		// Wake B
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Dynamic);

		Test.Advance();

		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());

		// Each particle in its own island (kinematic will be in all 3)
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 3);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
		EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

		// Convert A to dynamic sleeping
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Sleeping);

		Test.Advance();

		if (!Test.CVarPartialSleeping->GetBool()) // island sleeping
		{
			// All particles in one awake island (B was awake so it would wake the island)
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 1);
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

			// All particles awake
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		}
		else // partial island sleeping
		{
			// All particles are asleep except for B which remains awake
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[0]).Num(), 1);
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[1]).Num(), 1);
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[2]).Num(), 1);
			EXPECT_EQ(Test.IslandManager->FindParticleIslands(Test.ParticleHandles[3]).Num(), 1);

			// All particles except for B asleep
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		}
	}

	// Start with an island containing 4 awake particles connected in a chain. Then sleep the island
	// by explicitly putting all particles to sleep
	//
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ad - Bd - Cd - Dd  =>	As - Bs - Cs - Ds
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_SleepIsland)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeChain();

		Test.Advance();

		// All particles and costraints are awake
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		EXPECT_FALSE(Test.ConstraintHandles[0]->IsSleeping());
		EXPECT_FALSE(Test.ConstraintHandles[1]->IsSleeping());
		EXPECT_FALSE(Test.ConstraintHandles[2]->IsSleeping());
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);

		// Put all of the particles to sleep
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Sleeping);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Sleeping);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[2], EObjectStateType::Sleeping);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[3], EObjectStateType::Sleeping);

		Test.Advance();

		// Island and all particles and constraints should now be asleep
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		if (Test.IslandManager->GetNumIslands() == 1)
		{
			EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
		}
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());

		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsInConstraintGraph());

		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
	}

	// @todo(chaos): Implement this. TestConstraintGraph_SleepIsland is intended to reproduce a bug 
	// exp[osed by collision constraints where collisions were being destroyed on particles that were 
	// explicitly put to sleep. However that bug was a result of how collision constraints are 
	// destroyed (i.e., when they are not updated this tick) and the NullConstraints don't have that same 
	// behaviour. We need a unit testing constraint that can reproduce that behaviour. 
	TEST_P(GraphEvolutionTests, DISABLED_TestConstraintGraph_SleepIsland_Collisions)
	{
	}

	// Add a constraint between a sleeping and a kinematic body and tick.
	// Nothing should change.
	//
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ak - Bs    =>    Ak - Bs
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_SleepingKinematicConstraint)
	{
		FGraphEvolutionTest Test(2, GetParam());

		// Make A kinematic, B sleeping
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Sleeping);

		// Add a constraint A-B
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[1] }));

		Test.Advance();

		// Everything asleep
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
	}

	// Add a constraint between a sleeping and a kinematic body, one tick after the bodies were added.
	// 
	// This differs from TestConstraintGraph_SleepingKinematicConstraint in that we tick the scene one
	// time before adding the constraint, which means the particles are already in separate islands.
	// Nothing should wake and the constraint should be flagged as sleeping.
	// 
	// This behaviour is required for streaming to work since scene creation may
	// be amortized over multiple frames and constraints may be made betweens
	// sleeping particles in a later tick.
	//
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ak  Bs    =>    Ak - Bs
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_SleepingKinematicConstraint2)
	{
		FGraphEvolutionTest Test(2, GetParam());

		// Make A kinematic, B sleeping
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Sleeping);

		Test.Advance();
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());

		// Add a constraint A-B
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[1] }));

		Test.Advance();

		// B still asleep
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
	}

	// Add a constraint between two sleeping particles.
	// Nothing should wake and the constraint should be flagged as sleeping.
	// This behaviour is required for streaming to work since scene creation may
	// be amortized over multiple frames and constraints may be made betweens
	// sleeping particles in a later tick.
	// In this case, A and B start sleeping and get merged into a single 
	// still-sleeping island when we add the constraint.
	//
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		As  Bs    =>    As - Bs
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_SleepingSleepingConstraint)
	{
		FGraphEvolutionTest Test(2, GetParam());

		// Make A and B sleeping
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Sleeping);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Sleeping);

		Test.Advance();

		// Particles without any constraints are not in the graph
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 0);
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());

		// Add a constraint A-B
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[1] }));

		Test.Advance();

		// A and B still asleep but now in an island with the constraint
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
	}

	// Similar to TestConstraintGraph_SleepingKinematicConstraint, but we are adding a constraint 
	// between sleeping and kinematic particles that are already in an existing sleeping island
	// with multiple sleeping constraints.
	//
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ak - Bs - Cs  =>    Ak - Bs - Cs
	//		                     ^--------^
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_SleepingKinematicConstraint_SameIsland)
	{
		FGraphEvolutionTest Test(3, GetParam());

		// Chains the particles and make the first one kinematic
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);
		Test.MakeChain();

		// Wait for sleep
		Test.AdvanceUntilSleeping();
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());

		// Add a constraint A - C
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[2] }));

		Test.Advance();

		// All still asleep, including the new constraint
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
	}

	// Similar to TestConstraintGraph_SleepingKinematicConstraint, but we are adding a constraint 
	// between two sleeping particles in different island, but where each island already contains
	// sleeping constraints.
	//
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		As - Bs   Cs - Ds  =>    As - Bs - Cs - Ds
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_SleepingSleepingConstraint_MergeIslands)
	{
		FGraphEvolutionTest Test(4, GetParam());

		// Add constraints A-B and C-D
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[1] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[2], Test.ParticleHandles[3] }));

		// Wait for sleep
		Test.AdvanceUntilSleeping();
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 2);
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());

		// Add a constraint B - C
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[1], Test.ParticleHandles[2] }));

		Test.Advance();

		// All still asleep, including the new constraint
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
	}

	// Add constraints beween objects on the tick where their island goes to sleep, and one either side just to be sure.
	// i.e., The SleepCounter does not get reset when we add a constraint between two particles.
	//
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ad   Bd  =>    As - Bs
	//
	// @todo(chaos): this test would fail because we do not transfer isolated particle
	// sleep counts to the graph when we add a constraint to them. We could fix this
	// but probably not worth worrying about
	//
	TEST_P(GraphEvolutionTests, DISABLED_TestConstraintGraph_SleepingSleepingConstraint_Timing_Isolated)
	{
		// Count how many frames it takes the simulation to sleep
		FGraphEvolutionTest SleepTest(2, GetParam());
		SleepTest.AdvanceUntilSleeping();
		const int32 TicksToSleep = SleepTest.TickCount;

		// Create a new simulation up to the sleep tick +/- a tick
		// Verify that adding a constraint on that tick leaves the scene as expected
		for (int32 SleepRelativeTickCount = -1; SleepRelativeTickCount < 2; ++SleepRelativeTickCount)
		{
			FGraphEvolutionTest Test(2, GetParam());
			for (int32 Frame = 0; Frame < TicksToSleep + SleepRelativeTickCount; ++Frame)
			{
				Test.Advance();
			}
			const bool bExpectSleep = (SleepRelativeTickCount >= 0);

			// Should have no islands (because we have no constraints)
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 0);
			EXPECT_EQ(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping(), bExpectSleep);
			EXPECT_EQ(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping(), bExpectSleep);

			// Add a constraint A-B and tick
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[1] }));
			Test.Advance();

			// Should now have 1 island and it should be asleep
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());

			// Constraint should also be asleep
			EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
		}
	}

	// Add constraints beween objects on the tick where their island goes to sleep, and one either side just to be sure.
	// i.e., The SleepCounter does not get reset when we add a constraint between two particles.
	//
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ad - Bd   Cd - Dd  =>    As - Bs - Cs - Ds
	//
	// NOTE: this one works where DISABLED_TestConstraintGraph_SleepingSleepingConstraint_Timing_Isolated
	// would fail because we retain SleepCounter when merging islands (but not when adding isloated
	// particles that have their own sleep counter)
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_SleepingSleepingConstraint_Timing)
	{
		// Count how many frames it takes the simulation to sleep
		int32 TicksToSleep = 0;
		{
			FGraphEvolutionTest SleepTest(4, GetParam());
			SleepTest.ConstraintHandles.Add(SleepTest.Constraints.AddConstraint({ SleepTest.ParticleHandles[0], SleepTest.ParticleHandles[1] }));
			SleepTest.ConstraintHandles.Add(SleepTest.Constraints.AddConstraint({ SleepTest.ParticleHandles[2], SleepTest.ParticleHandles[3] }));

			SleepTest.AdvanceUntilSleeping();
			TicksToSleep = SleepTest.TickCount;
		}

		// Create a new simulation up to the sleep tick +/- a tick
		// Verify that adding a constraint on that tick leaves the scene as expected
		for (int32 SleepRelativeTickCount = -1; SleepRelativeTickCount < 2; ++SleepRelativeTickCount)
		{
			FGraphEvolutionTest Test(4, GetParam());
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[1] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[2], Test.ParticleHandles[3] }));

			for (int32 Frame = 0; Frame < TicksToSleep + SleepRelativeTickCount; ++Frame)
			{
				Test.Advance();
			}
			const bool bExpectSleep = (SleepRelativeTickCount >= 0);

			// Should have no islands (because we have no constraints)
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 2);
			EXPECT_EQ(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping(), bExpectSleep);
			EXPECT_EQ(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping(), bExpectSleep);
			EXPECT_EQ(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping(), bExpectSleep);
			EXPECT_EQ(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping(), bExpectSleep);

			// Add a constraint B-C and tick
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[1], Test.ParticleHandles[2] }));
			Test.Advance();

			// Should now have 1 island and it should be asleep
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());

			// Constraints should also be asleep
			EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
		}
	}

	// Test an edge case bug that is probably easy to accidentally reintroduce. This would leave
	// a dangling pointer in the constraint graph due to a collision constraint being deleted while
	// in a sleeping island.
	// 
	// The fix was to ensure that we build the Island particle and constraint lists for islands that
	// have just been put to sleep (we still don't bother for thoise that were already asleep) so
	// that we can visit all the particles and constraints to set the sleep state.
	// 
	// 1: A dynamic particle is in its own awake island
	// - Tick
	// 2a: The particle is manually put to sleep
	// 2b: A constraint is added between the particle and a kinematic
	// - Tick
	// During the graph update on this tick, the particle's island is put to sleep in UpdateGraph
	// because all particles in it are asleep. However, the constraint was added this tick as well,
	// but when it was added the island was awake, so the constraint starts in the awake state.
	// 
	// Verify that the constraint does actually get put to sleep at some point in the graph update.
	//
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ak   Bd  =>    As - Bs
	//
	// NOTE: the transition to sleep is by a user call, not the automatic sleep-when-not-moving system
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_SleepingSleepingConstraint_Timing2)
	{
		FGraphEvolutionTest Test(2, GetParam());

		// Make A kinematic, B dynamic
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Dynamic);

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 0);

		// Explicitly put B to sleep
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Sleeping);

		// Add a constraint A-B. B is asleep
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[1] }));

		Test.Advance();

		// Everything should be asleep
		// The bug was that the constraint was still flagged as awake, but in a sleeping island.
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
	}

	// This is a very similar test to TestConstraintGraph_SleepingSleepingConstraint_Timing2
	// in that is exposes the same bug where an island that is implicitly put to sleep because
	// all its particles were explicitly put to sleep did not put its constraints to sleep.
	//
	//		(d=dynamic, s=sleeping, k=kinematic)
	//		Ad -  Bd  =>    As - Bs
	// 
	// NOTE: the transition to sleep is by a user call, not the automatic sleep-when-not-moving system
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_SleepingSleepingConstraint_Timing3)
	{
		FGraphEvolutionTest Test(2, GetParam());

		// Make A, B dynamic
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Dynamic);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Dynamic);

		// Add a constraint A-B
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[1] }));

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());

		// Explicitly put both particles (and therefore their island) to sleep
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Sleeping);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Sleeping);

		Test.Advance();

		// Everything should be asleep
		// The bug was that the constraint was still flagged as awake, but in a sleeping island.
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
	}

	// Test isolated particles are not present in the graph
	TEST_P(GraphEvolutionTests, TestConstraintGraph_KinematicRemoveFromGraph)
	{
		// Create a scene with 3 dynamic particles
		FGraphEvolutionTest Test(3, GetParam());

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 0);
		EXPECT_EQ(Test.IslandManager->GetNumParticles(), 0);

		// Change a particle to kinematic
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Kinematic);

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 0);
		EXPECT_EQ(Test.IslandManager->GetNumParticles(), 0);

		Test.Advance();

		// State should not have changed with a second tick
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 0);
		EXPECT_EQ(Test.IslandManager->GetNumParticles(), 0);
	}

	// Test the conditions for a kinematic particle waking an island
	// If a kinematic is being animated by velocity or by setting a target
	// position the island should wake but only if the target velocity is
	// non-zero or the target transform is different from the identity
	TEST_P(GraphEvolutionTests, TestConstraintGraph_KinematicWakeIslandConditions)
	{
		FGraphEvolutionTest Test(4, GetParam());
		Test.MakeChain();

		// Set the root of the chain to be kinematic and the rest to be sleeping
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);
		for (int32 ParticleIndex = 1; ParticleIndex < Test.ParticleHandles.Num(); ++ParticleIndex)
		{
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[ParticleIndex], EObjectStateType::Sleeping);
		}

		Test.Advance();

		// Expect one sleeping island
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_NE(Test.ConstraintHandles[0]->GetConstraintGraphEdge(), nullptr);
		EXPECT_NE(Test.ConstraintHandles[1]->GetConstraintGraphEdge(), nullptr);
		EXPECT_NE(Test.ConstraintHandles[2]->GetConstraintGraphEdge(), nullptr);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());

		// Set to velocity mode and animate
		FKinematicGeometryParticleHandle* KinematicParticle = Test.ParticleHandles[0]->CastToKinematicParticle();
		ASSERT_NE(KinematicParticle, nullptr);
		KinematicParticle->KinematicTarget().SetVelocityMode();

		Test.Advance();

		// Expect one sleeping island as the velocity of the kinematic particle is still zero
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());

		KinematicParticle->SetV(FVec3(10.0f, 0.0f, 0.0f));

		Test.Advance();

		// Expect one awake island
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());

		// Put all particles back to sleep and now set angular velocity
		KinematicParticle->SetV(FVec3(0.0f, 0.0f, 0.0f));
		for (int32 ParticleIndex = 1; ParticleIndex < Test.ParticleHandles.Num(); ++ParticleIndex)
		{
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[ParticleIndex], EObjectStateType::Sleeping);
		}

		Test.Advance();

		// Check we've put the island back to sleep
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());

		// Now set angular velocity. Island should wake
		KinematicParticle->SetW(FVec3(0.0f, 1.0f, 0.0f));

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());

		// Put all particles back to sleep
		KinematicParticle->SetW(FVec3(0.0f, 0.0f, 0.0f));
		for (int32 ParticleIndex = 1; ParticleIndex < Test.ParticleHandles.Num(); ++ParticleIndex)
		{
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[ParticleIndex], EObjectStateType::Sleeping);
		}

		Test.Advance();

		// Check we've put the island back to sleep
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());

		// Now set to position mode. Initially the island should stay sleeping as the target
		// transform is the identity
		FKinematicTarget KinematicTarget;
		KinematicTarget.SetTargetMode(FRigidTransform3::Identity);
		KinematicParticle->SetKinematicTarget(KinematicTarget);

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());

		// Set a non-zero position target. Should cause the island to wake
		KinematicTarget.SetTargetMode(FVec3(10.0f, 0.0f, 0.0f), FRotation3::Identity);
		KinematicParticle->SetKinematicTarget(KinematicTarget);

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());

		// Put all particles back to sleep
		KinematicTarget.SetTargetMode(FRigidTransform3::Identity);
		KinematicParticle->SetKinematicTarget(KinematicTarget);
		for (int32 ParticleIndex = 1; ParticleIndex < Test.ParticleHandles.Num(); ++ParticleIndex)
		{
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[ParticleIndex], EObjectStateType::Sleeping);
		}

		Test.Advance();

		// Check we've put the island back to sleep
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());

		// Set a non-identity rotation target. Should cause the island to wake
		FRigidTransform3 TargetTransform(FVec3(0.0f, 0.0f, 0.0f), FQuat::MakeFromEuler(FVec3(1.0f, 0.0f, 2.0f)));
		KinematicTarget.SetTargetMode(TargetTransform);
		KinematicParticle->SetKinematicTarget(KinematicTarget);

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());

	}

	// Test that island merging works if we remove the last constraint
	// in an island that was already queued for merge (PLAY-6440).
	// 
	// (d=dynamic, s=sleeping, k=kinematic)
	//		Ad - Bd   Cd - Dd
	// =>	Ad - Bd - Cd   Dd
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_IslandMerge_EnableDisable)
	{
		FGraphEvolutionTest Test(4, GetParam());

		// Create constraints in a chain but disable the middle constraint so we have two islands {A-B} and {C-D}
		Test.MakeChain();
		Test.ConstraintHandles[1]->SetEnabled(false);

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 2);
		EXPECT_FALSE(Test.ConstraintHandles[1]->IsInConstraintGraph());

		// Enable the constraint B-C.
		// NOTE: In the implementation, the enable will add the constraint to one of the islands
		// and queue the two islands to be merged, but the actual merging happens in Advance().
		Test.ConstraintHandles[1]->SetEnabled(true);

		// Disable the constraint C-D.
		// NOTE: This will leave the second island without any constraints, but it is
		// not destroyed immediately because it is queued for merging and, even though
		// it has no constraints, it still contains particle C which needs to be
		// copied to the new merged island. Particle D would have been removed because
		// it does not have any constraints.
		// Issue (PLAY-6440) was caused by the island being destroyed because it was
		// empty, but it was still queued to be merged.
		Test.ConstraintHandles[2]->SetEnabled(false);

		Test.Advance();

		// We should now only have 1 island and D should not be in the graph
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsInConstraintGraph());
		EXPECT_FALSE(Test.ConstraintHandles[2]->IsInConstraintGraph());	// Not in graph
		EXPECT_TRUE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[2]->IsInConstraintGraph());
		EXPECT_FALSE(Test.ParticleHandles[3]->IsInConstraintGraph());	// Not in graph
	}


	// Test the sparse array repeatable index assignment. See TSparseArray::SortFreeList()
	TEST_P(GraphEvolutionTests, SparseArrayTests_TestSortFreeList)
	{
		TSparseArray<int32> Values;

		// The first time we add objects, they should be in consecutive indices starting from 0
		Values.Add(0);
		Values.Add(1);
		Values.Add(2);
		EXPECT_EQ(Values[0], 0);
		EXPECT_EQ(Values[1], 1);
		EXPECT_EQ(Values[2], 2);

		// Remove a couple items in the same order we added them
		Values.RemoveAt(1);
		Values.RemoveAt(2);

		// Add the items again, they will end up in reverse order
		// We don't rely on this behaviour but I'm testing here because if this changes
		// in the future then we may be able to remove our calls to SortFreeList in the graph.
		Values.Add(1);
		Values.Add(2);
		EXPECT_EQ(Values[0], 0);
		EXPECT_EQ(Values[1], 2);	// Swapped
		EXPECT_EQ(Values[2], 1);	// Swapped


		// Now do the same as above on a new array, but call SortFreeList before reusing it
		TSparseArray<int32> Values2;

		Values2.Add(0);
		Values2.Add(1);
		Values2.Add(2);
		Values2.RemoveAt(1);
		Values2.RemoveAt(2);

		// Rebuild the free list
		Values2.SortFreeList();

		// We should now get the same order as the first time we added items
		Values2.Add(1);
		Values2.Add(2);
		EXPECT_EQ(Values2[0], 0);
		EXPECT_EQ(Values2[1], 1);
		EXPECT_EQ(Values2[2], 2);
	}


	// Test validating that a sleeping island wakes up when teleporting one of its particles.
	// Teleporting means explicitly updating the transform of the particle.
	TEST_P(GraphEvolutionTests, TestConstraintGraph_TeleportSleeping)
	{
		const bool bPartialWakePreIntegrationDefault = CVarPartialWakePreIntegration->GetBool();
		CVarPartialWakePreIntegration->Set(true);

		FGraphEvolutionTest Test(5, GetParam());
		Test.MakeChain();
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);

		Test.AdvanceUntilSleeping();

		// Should have 1 island and it should be asleep
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_TRUE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[2]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[3]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[4]->IsInConstraintGraph());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());

		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[3]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[3]->IsSleeping());

		// Teleport a particle by setting its transform explicitly
		const bool bIsTeleport = true;
		const bool bWakeUp = true;
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[4], EObjectStateType::Dynamic);
		Test.Evolution.SetParticleTransform(Test.ParticleHandles[4], FVec3(0, 3, 0), FRotation3::Identity, bIsTeleport, bWakeUp);

		Test.Advance();

		// Should wake up the entire island
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());

		if (!GetParam())
		{
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
			EXPECT_FALSE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[2]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[3]->IsSleeping());
		}
		else // partial island sleep
		{
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
			EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[2]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[3]->IsSleeping());
		}

		CVarPartialWakePreIntegration->Set(bPartialWakePreIntegrationDefault);
	}

	// Test validating that a sleeping island wakes up when adding an impulse, acceleration, etc. to one particle.
	// NOTE: This code emulates what happens if dynamics data is updated in @PushToPhysicsStateImp (SingleParticlePhysicsProxy.cpp).
	TEST_P(GraphEvolutionTests, TestConstraintGraph_SetVelocityOfSleeping)
	{
		FGraphEvolutionTest Test(3, GetParam());
		Test.MakeChain();
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);

		Test.AdvanceUntilSleeping();

		// Should have 1 island and it should be asleep
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_TRUE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[2]->IsInConstraintGraph());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());

		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());

		// Explicitly set a non-zero velocity
		FParticleDynamics Dynamics;
		Dynamics.SetAcceleration(FVec3(0, 0, 0));
		Dynamics.SetAngularAcceleration(FVec3(0, 0, 0));
		Dynamics.SetLinearImpulseVelocity(FVec3(50, 50, 0));
		Dynamics.SetAngularImpulseVelocity(FVec3(0, 0, 0));

		// This emulates what happens in @PushToPhysicsStateImp (SingleParticlePhysicsProxy.cpp).
		Test.Evolution.SetParticleDynamics(Test.ParticleHandles[2], Dynamics);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[2], EObjectStateType::Dynamic);

		Test.Advance();

		// Should wake up the entire island
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_FALSE(Test.ConstraintHandles[0]->IsSleeping());
		EXPECT_FALSE(Test.ConstraintHandles[1]->IsSleeping());
	}

	// Test validating the wake-up propagation throughout a sleeping island if adding an awake particle to the top of the stack.
	// 1) For island sleeping, the entire island should wake up in a single tick.
	// (d=dynamic, s=sleeping, k=kinematic)
	//		Ak - Bs - Cs - Ds
	// =>	Ak - Bd - Cd - Dd - Ed
	// 2) For partial island sleeping, the awake particle will be added to the island and only wake the particle closest to it. 
	// (d=dynamic, s=sleeping, k=kinematic)
	//		Ak - Bs - Cs - Ds
	// =>	Ak - Bs - Cs - Dd - Ed
	//
	TEST_P(GraphEvolutionTests, TestConstraintGraph_WakeUpPropagation_AddToStackTop)
	{
		FGraphEvolutionTest Test(4, GetParam());

		if (Test.CVarPartialSleeping->GetBool())
		{
			// Turn off mommentum propagation to test level-based waking only
			const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
			CVarMomentumPropagation->Set(0);

			Test.MakeChain();
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);

			Test.AdvanceUntilSleeping();

			// Should have 1 island and it should be asleep
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_TRUE(Test.ParticleHandles[0]->IsInConstraintGraph());
			EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
			EXPECT_TRUE(Test.ParticleHandles[2]->IsInConstraintGraph());
			EXPECT_TRUE(Test.ParticleHandles[3]->IsInConstraintGraph());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());

			EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
			EXPECT_TRUE(Test.ConstraintHandles[1]->IsInConstraintGraph());
			EXPECT_TRUE(Test.ConstraintHandles[2]->IsInConstraintGraph());
			EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());

			// Add another particle and connect it to the top of the chain
			Test.ParticleHandles.Add(Test.Evolution.CreateDynamicParticles(1)[0]);
			Test.Evolution.EnableParticle(Test.ParticleHandles.Last());
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles.Last(1), Test.ParticleHandles.Last() }));

			// NOTE: Set a non-zero velocity to make the particle move and trigger a wake-up event.
			FParticleDynamics Dynamics;
			Dynamics.SetAcceleration(FVec3(0, 0, 0));
			Dynamics.SetAngularAcceleration(FVec3(0, 0, 0));
			Dynamics.SetLinearImpulseVelocity(FVec3(50, 50, 0));
			Dynamics.SetAngularImpulseVelocity(FVec3(0, 0, 0));
			Test.Evolution.SetParticleDynamics(Test.ParticleHandles.Last(), Dynamics);

			Test.Advance();

			EXPECT_EQ(Test.IslandManager->GetNumParticles(), 5);
			EXPECT_EQ(Test.IslandManager->GetNumConstraints(), 4);

			// Will wake up the top particle of the original stack
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
			EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[2]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[3]->IsSleeping());

			CVarMomentumPropagation->Set(MomentumPropagationDefault);
		}
	}

	// Test validating the wake-up propagation throughout a sleeping island if adding an awake particle to the top of the stack.
	// Partial island sleeping only: the awake particle will be added to the island and only wake the particle closest to it. 
	// (d=dynamic, s=sleeping, k=kinematic)
	//		Ak - Bs - Cs - Ds
	// =>	Ak - Bs - Cd - Dd	
	//                |
	//                Ed
	TEST_P(GraphEvolutionTests, TestConstraintGraph_WakeUpPropagation_AddToStackCenter)
	{
		FGraphEvolutionTest Test(4, GetParam());

		if (Test.CVarPartialSleeping->GetBool())
		{
			// Turn off mommentum propagation to test level-based waking only
			const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
			CVarMomentumPropagation->Set(0);

			Test.MakeChain();
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);

			Test.AdvanceUntilSleeping();

			// Should have 1 island and it should be asleep
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_TRUE(Test.ParticleHandles[0]->IsInConstraintGraph());
			EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
			EXPECT_TRUE(Test.ParticleHandles[2]->IsInConstraintGraph());
			EXPECT_TRUE(Test.ParticleHandles[3]->IsInConstraintGraph());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());

			EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
			EXPECT_TRUE(Test.ConstraintHandles[1]->IsInConstraintGraph());
			EXPECT_TRUE(Test.ConstraintHandles[2]->IsInConstraintGraph());
			EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());

			// Add another particle and connect it to the top of the chain
			Test.ParticleHandles.Add(Test.Evolution.CreateDynamicParticles(1)[0]);
			Test.Evolution.EnableParticle(Test.ParticleHandles.Last());
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles.Last(2), Test.ParticleHandles.Last() }));

			// NOTE: Set a non-zero velocity to make the particle move and trigger a wake-up event.
			FParticleDynamics Dynamics;
			Dynamics.SetAcceleration(FVec3(0, 0, 0));
			Dynamics.SetAngularAcceleration(FVec3(0, 0, 0));
			Dynamics.SetLinearImpulseVelocity(FVec3(50, 50, 0));
			Dynamics.SetAngularImpulseVelocity(FVec3(0, 0, 0));
			Test.Evolution.SetParticleDynamics(Test.ParticleHandles.Last(), Dynamics);

			Test.Advance();

			EXPECT_EQ(Test.IslandManager->GetNumParticles(), 5);
			EXPECT_EQ(Test.IslandManager->GetNumConstraints(), 4);

			// Will wake up the top particle of the original stack
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
			EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[2]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[3]->IsSleeping());

			CVarMomentumPropagation->Set(MomentumPropagationDefault);
		}
	}

	// Test validating the wake-up propagation for a triangular brick wall with lateral spacing.
	// We add another particle near the top of the wall (partial island sleeping only).
	//				    Gs
	//				  /	  \
	//		        Es     Fs
	//             / \	  / \
	//            Bs   Cs   Ds
	//             \   |    /
	//                 Ak
	//  =>
	//			 Hd     Gd
	//			  \	  /	  \
	//		        Ed     Fs
	//             / \	  / \
	//            Bs   Cs   Ds
	//             \   |    /
	//                 Ak
	TEST_P(GraphEvolutionTests, TestConstraintGraph_WakeUpPropagation_AddToWallWithSpacing)
	{
		FGraphEvolutionTest Test(7, GetParam());
		if (Test.CVarPartialSleeping->GetBool())
		{
			// Turn off mommentum propagation to test level-based waking only
			const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
			CVarMomentumPropagation->Set(0);

			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[1] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[2] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[3] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[1], Test.ParticleHandles[4] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[2], Test.ParticleHandles[4] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[2], Test.ParticleHandles[5] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[3], Test.ParticleHandles[5] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[4], Test.ParticleHandles[6] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[5], Test.ParticleHandles[6] }));

			Test.AdvanceUntilSleeping();

			// Should have 1 island and it should be asleep
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[6])->Sleeping());
			EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[3]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[4]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[5]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[6]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[7]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[8]->IsSleeping());

			// Add another particle near the top of the wall
			Test.ParticleHandles.Add(Test.Evolution.CreateDynamicParticles(1)[0]);
			Test.Evolution.EnableParticle(Test.ParticleHandles.Last());
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles.Last(3), Test.ParticleHandles.Last() }));

			// NOTE: Set a non-zero velocity to make the particle move and trigger a wake-up event.
			FParticleDynamics Dynamics;
			Dynamics.SetAcceleration(FVec3(0, 0, 0));
			Dynamics.SetAngularAcceleration(FVec3(0, 0, 0));
			Dynamics.SetLinearImpulseVelocity(FVec3(50, 50, 0));
			Dynamics.SetAngularImpulseVelocity(FVec3(0, 0, 0));
			Test.Evolution.SetParticleDynamics(Test.ParticleHandles.Last(), Dynamics);

			Test.Advance();

			EXPECT_EQ(Test.IslandManager->GetNumParticles(), 8);
			EXPECT_EQ(Test.IslandManager->GetNumConstraints(), 10);

			// Will wake up the two particles connected by the new constraint and the top of the wall
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[6])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[7])->Sleeping());
			EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[3]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[4]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[5]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[6]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[7]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[8]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[9]->IsSleeping());

			CVarMomentumPropagation->Set(MomentumPropagationDefault);
		}
	}

	// Test validating the wake-up propagation for a triangular brick wall without lateral spacing.
	// We add another particle near the top of the wall (partial island sleeping only).
	//				    Gs
	//				  /	  \
	//		        Es  -  Fs
	//             / \	  / \
	//            Bs - Cs - Ds
	//             \   |    /
	//                 Ak
	//  =>
	//			 Hd     Gd
	//			  \	  /	  \
	//		        Ed  -  Fs
	//             / \	  / \
	//            Bs - Cs - Ds
	//             \   |    /
	//                 Ak
	TEST_P(GraphEvolutionTests, TestConstraintGraph_WakeUpPropagation_AddToWallWithoutSpacing)
	{
		FGraphEvolutionTest Test(7, GetParam());
		if (Test.CVarPartialSleeping->GetBool())
		{
			// Turn off mommentum propagation to test level-based waking only
			const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
			CVarMomentumPropagation->Set(0);

			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[1] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[2] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[3] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[1], Test.ParticleHandles[2] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[2], Test.ParticleHandles[3] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[1], Test.ParticleHandles[4] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[2], Test.ParticleHandles[4] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[2], Test.ParticleHandles[5] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[3], Test.ParticleHandles[5] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[4], Test.ParticleHandles[5] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[4], Test.ParticleHandles[6] }));
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[5], Test.ParticleHandles[6] }));

			Test.AdvanceUntilSleeping();

			// Should have 1 island and it should be asleep
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[6])->Sleeping());
			EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[3]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[4]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[5]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[6]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[7]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[8]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[9]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[10]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[11]->IsSleeping());

			// Add another particle near the top of the wall
			Test.ParticleHandles.Add(Test.Evolution.CreateDynamicParticles(1)[0]);
			Test.Evolution.EnableParticle(Test.ParticleHandles.Last());
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles.Last(3), Test.ParticleHandles.Last() }));

			// NOTE: Set a non-zero velocity to make the particle move and trigger a wake-up event.
			FParticleDynamics Dynamics;
			Dynamics.SetAcceleration(FVec3(0, 0, 0));
			Dynamics.SetAngularAcceleration(FVec3(0, 0, 0));
			Dynamics.SetLinearImpulseVelocity(FVec3(50, 50, 0));
			Dynamics.SetAngularImpulseVelocity(FVec3(0, 0, 0));
			Test.Evolution.SetParticleDynamics(Test.ParticleHandles.Last(), Dynamics);

			Test.Advance();

			EXPECT_EQ(Test.IslandManager->GetNumParticles(), 8);
			EXPECT_EQ(Test.IslandManager->GetNumConstraints(), 13);

			// Will wake up the two particles connected by the new constraint and the top of the wall
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[6])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[7])->Sleeping());
			EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[3]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[4]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[5]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[6]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[7]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[8]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[9]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[10]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[11]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[12]->IsSleeping());

			CVarMomentumPropagation->Set(MomentumPropagationDefault);
		}
	}

	// Test validating the wake-up propagation for a rectangular block wall without lateral spacing.
	// We add another constraint to connect two particles diagonally.
	//			  Hs - Ks - Ls
	//			  |    |    |
	//		      Es - Fs - Gs
	//            |    |    |
	//            Bs - Cs - Ds
	//             \   |    /
	//                 Ak
	//  =>
	//			  Hs - Ks - Ls
	//			  |  \ |    |
	//		      Es - Fs - Gs
	//            |    |    |
	//            Bs - Cs - Ds
	//             \   |    /
	//                 Ak
	TEST_P(GraphEvolutionTests, TestConstraintGraph_WakeUpPropagation_AddToBlockWithoutSpacingSleeping)
	{
		// Turn off mommentum propagation to test level-based waking only
		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(0);

		FGraphEvolutionTest Test(10, GetParam());
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);

		// Constraints with the ground
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[1] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[2] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[3] }));
		// Horizontal constraints
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[1], Test.ParticleHandles[2] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[2], Test.ParticleHandles[3] }));
		// Vertical constraints
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[1], Test.ParticleHandles[4] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[2], Test.ParticleHandles[5] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[3], Test.ParticleHandles[6] }));
		// Horizontal constraints
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[4], Test.ParticleHandles[5] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[5], Test.ParticleHandles[6] }));
		// Vertical constraints
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[4], Test.ParticleHandles[7] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[5], Test.ParticleHandles[8] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[6], Test.ParticleHandles[9] }));
		// Horizontal constraints
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[7], Test.ParticleHandles[8] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[8], Test.ParticleHandles[9] }));

		Test.AdvanceUntilSleeping();

		// Should have 1 island and it should be asleep
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[6])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[7])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[8])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[9])->Sleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[3]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[4]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[5]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[6]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[7]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[8]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[9]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[10]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[11]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[12]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[13]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[14]->IsSleeping());

		// Add another constraint to diagonally connect to particles in the block
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[5], Test.ParticleHandles[7] }));

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumParticles(), 10);
		EXPECT_EQ(Test.IslandManager->GetNumConstraints(), 16);

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[6])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[7])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[8])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[9])->Sleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[3]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[4]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[5]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[6]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[7]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[8]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[9]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[10]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[11]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[12]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[13]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[14]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[15]->IsSleeping());

		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}

	// Test validating the wake-up propagation for a rectangular block wall without lateral spacing.
	// We add another constraint to connect two particles diagonally, wake particle H and add an impulse to it.
	//			  Hs - Ks - Ls
	//			  |    |    |
	//		      Es - Fs - Gs
	//            |    |    |
	//            Bs - Cs - Ds
	//             \   |    /
	//                 Ak
	//  1) For full island sleeping, the entire island will wake up.
	// (d=dynamic, s=sleeping, k=kinematic)
	//			  Hd - Kd - Ld
	//			  |  \ |    |
	//		      Ed - Fd - Gd
	//            |    |    |
	//            Bd - Cd - Dd
	//             \   |    /
	//                 Ak
	//  2) For partial island sleeping, the state change in particle H will wake up its immediate neighbors and all particles at a higher level.
	// (d=dynamic, s=sleeping, k=kinematic)
	//			  Hd - Kd - Ls
	//			  |  \ |    |
	//		      Ed - Fd - Gs
	//            |    |    |
	//            Bs - Cs - Ds
	//             \   |    /
	//                 As
	TEST_P(GraphEvolutionTests, TestConstraintGraph_WakeUpPropagation_AddToBlockWithoutSpacingAwake)
	{
		// Turn off mommentum propagation to test level-based waking only
		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(0);

		FGraphEvolutionTest Test(10, GetParam());
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);

		// Constraints with the ground
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[1] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[2] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[0], Test.ParticleHandles[3] }));
		// Horizontal constraints
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[1], Test.ParticleHandles[2] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[2], Test.ParticleHandles[3] }));
		// Vertical constraints
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[1], Test.ParticleHandles[4] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[2], Test.ParticleHandles[5] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[3], Test.ParticleHandles[6] }));
		// Horizontal constraints
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[4], Test.ParticleHandles[5] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[5], Test.ParticleHandles[6] }));
		// Vertical constraints
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[4], Test.ParticleHandles[7] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[5], Test.ParticleHandles[8] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[6], Test.ParticleHandles[9] }));
		// Horizontal constraints
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[7], Test.ParticleHandles[8] }));
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[8], Test.ParticleHandles[9] }));

		Test.AdvanceUntilSleeping();

		// Should have 1 island and it should be asleep
		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[6])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[7])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[8])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[9])->Sleeping());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[3]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[4]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[5]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[6]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[7]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[8]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[9]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[10]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[11]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[12]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[13]->IsSleeping());
		EXPECT_TRUE(Test.ConstraintHandles[14]->IsSleeping());

		// Add another constraint to diagonally connect to particles in the block
		Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles[5], Test.ParticleHandles[7] }));
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[7], EObjectStateType::Dynamic);

		// NOTE: Set a non-zero velocity to make the particle move and trigger a wake-up event.
		FParticleDynamics Dynamics;
		Dynamics.SetAcceleration(FVec3(0, 0, 0));
		Dynamics.SetAngularAcceleration(FVec3(0, 0, 0));
		Dynamics.SetLinearImpulseVelocity(FVec3(50, 50, 0));
		Dynamics.SetAngularImpulseVelocity(FVec3(0, 0, 0));
		Test.Evolution.SetParticleDynamics(Test.ParticleHandles[7], Dynamics);

		Test.Advance();

		EXPECT_EQ(Test.IslandManager->GetNumParticles(), 10);
		EXPECT_EQ(Test.IslandManager->GetNumConstraints(), 16);

		if (!Test.CVarPartialSleeping->GetBool()) // island sleeping
		{
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[6])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[7])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[8])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[9])->Sleeping());
			EXPECT_FALSE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[2]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[3]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[4]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[5]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[6]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[7]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[8]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[9]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[10]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[11]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[12]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[13]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[14]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[15]->IsSleeping());
		}
		else // partial island sleeping
		{
			EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
			EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[6])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[7])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[8])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[9])->Sleeping());
			EXPECT_TRUE(Test.ConstraintHandles[0]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[2]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[3]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[4]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[5]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[6]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[7]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[8]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[9]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[10]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[11]->IsSleeping());
			EXPECT_TRUE(Test.ConstraintHandles[12]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[13]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[14]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[15]->IsSleeping());
		}

		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}

	TEST_P(GraphEvolutionTests, TestConstraintGraph_SleepCounterReset)
	{
		// Partial island sleeping only
		if (GetParam())
		{	
			FGraphEvolutionTest Test(2, GetParam());
			Test.MakeChain();

			// Number of steps is a few less than the sleep counter threshold
			IConsoleVariable* CVarSleepCounterThreshold = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.Defaults.SleepCounterThreshold"), false);
			check(CVarSleepCounterThreshold && CVarSleepCounterThreshold->IsVariableInt());
			const int32 SleepCounterThreshold = CVarSleepCounterThreshold->GetInt();
			const int32 Steps = SleepCounterThreshold - 2;

			for (int32 I = 0; I < Steps; ++I)
			{
				Test.Advance();
			}

			for (FPBDRigidParticleHandle* Particle : Test.ParticleHandles)
			{
				EXPECT_FALSE(Particle->IsSleeping());
				const int32 SleepCounter = Particle->SleepCounter();
				EXPECT_EQ(SleepCounter, Steps);
			}

			// Set a non-zero velocity to make the particle move.
			FParticleDynamics Dynamics;
			Dynamics.SetAcceleration(FVec3(0, 0, 0));
			Dynamics.SetAngularAcceleration(FVec3(0, 0, 0));
			Dynamics.SetLinearImpulseVelocity(FVec3(50, 50, 0));
			Dynamics.SetAngularImpulseVelocity(FVec3(0, 0, 0));
			for (FPBDRigidParticleHandle* Particle : Test.ParticleHandles)
			{
				Test.Evolution.SetParticleDynamics(Particle, Dynamics);
			}

			// Step one more time, the sleep counter should be reset to 0 because the particles are moving
		    Test.Advance();

			for (FPBDRigidParticleHandle* Particle : Test.ParticleHandles)
			{
				EXPECT_FALSE(Particle->IsSleeping());
				const int32 SleepCounter = Particle->SleepCounter();
				EXPECT_EQ(SleepCounter, 0);
			}
		}
	}

	// Test would fail since introducing Island->Flags.bHasSleepingNode.
	// When the entire island is already awake, we don't perform partial island waking (FPBDIslandManager::ProcessPartialIslandWake)
	// but just sync the island sleep state to all particles and constraints (FPBDIslandManager::PropagateIslandSleep).
	// Thus, the particle sleep counter is not reset if there is a wake event and the entire island is already awake
	// Since this is difficult to fix and there is no known practical issue, we leave this as is for now and disable the test. 
	TEST_P(GraphEvolutionTests, DISABLED_TestConstraintGraph_PreventSleepDuringWakeEvent)
	{
		// Partial island sleeping only
		if (GetParam())
		{
			FGraphEvolutionTest Test(2, GetParam());
			Test.MakeChain();

			// Number of steps is the sleep counter threshold
			IConsoleVariable* CVarSleepCounterThreshold = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.Defaults.SleepCounterThreshold"), false);
			check(CVarSleepCounterThreshold && CVarSleepCounterThreshold->IsVariableInt());
			const int32 SleepCounterThreshold = CVarSleepCounterThreshold->GetInt();
			const int32 Steps = SleepCounterThreshold;

			for (int32 I = 0; I < Steps; ++I)
			{
				Test.Advance();
			}

			for (FPBDRigidParticleHandle* Particle : Test.ParticleHandles)
			{
				EXPECT_FALSE(Particle->IsSleeping());
				const int32 SleepCounter = Particle->SleepCounter();
				EXPECT_EQ(SleepCounter, Steps);
			}

			// Add another particle and connect it to the top of the chain
			Test.ParticleHandles.Add(Test.Evolution.CreateDynamicParticles(1)[0]);
			Test.Evolution.EnableParticle(Test.ParticleHandles.Last());
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles.Last(1), Test.ParticleHandles.Last() }));

			// NOTE: Set a non-zero velocity to make the particle move and trigger a wake-up event.
			FParticleDynamics Dynamics;
			Dynamics.SetAcceleration(FVec3(0, 0, 0));
			Dynamics.SetAngularAcceleration(FVec3(0, 0, 0));
			Dynamics.SetLinearImpulseVelocity(FVec3(50, 50, 0));
			Dynamics.SetAngularImpulseVelocity(FVec3(0, 0, 0));
			Test.Evolution.SetParticleDynamics(Test.ParticleHandles.Last(), Dynamics);

			// Step one more time, the particles should not sleep since a wake-up event just happened.
			Test.Advance();

			for (FPBDRigidParticleHandle* Particle : Test.ParticleHandles)
			{
				EXPECT_FALSE(Particle->IsSleeping());
			}

			// Resting particles
			EXPECT_EQ(Test.ParticleHandles[0]->SleepCounter(), 1);
			EXPECT_EQ(Test.ParticleHandles[1]->SleepCounter(), 1);
			// Moving particle
			EXPECT_EQ(Test.ParticleHandles[2]->SleepCounter(), 0);
		}
	}

	TEST_P(GraphEvolutionTests, TestConstraintGraph_PartialSleepAllowedParticleFlag)
	{
		if (GetParam())
		{
			FGraphEvolutionTest Test(4, GetParam());
			Test.MakeChain();

			// Explicitly put two particles asleep
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Sleeping);
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Sleeping);

			Test.Advance();

			// Partial sleeping should work as usual
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());

			// Disallow partial sleeping for one of the sleeping particles
			Test.ParticleHandles[0]->SetPartialIslandSleepAllowed(false);

			Test.Advance();

			// All particles wake up
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());

			// Explicitly put two particles asleep
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Sleeping);
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Sleeping);

			// Allow partial sleeping again of the particles
			Test.ParticleHandles[0]->SetPartialIslandSleepAllowed(true);

			Test.Advance();

			// Partial sleeping should work as usual
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());

			// Disallow partial sleeping for one of the awake particles
			Test.ParticleHandles[2]->SetPartialIslandSleepAllowed(false);

			Test.Advance();

			// All particles wake up
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());

			Test.AdvanceUntilSleeping();

			// All particles are asleep (using island sleeping)
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		}
	}

	TEST_P(GraphEvolutionTests, TestConstraintGraph_PartialSleepAllowedParticleFlag_2)
	{
		if (GetParam())
		{
			FGraphEvolutionTest Test(4, GetParam());
			Test.MakeChain();

			// Explicitly put two particles asleep
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Sleeping);
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Sleeping);

			Test.Advance();

			// Partial sleeping should work as usual
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());

			// Add another particle which does not allow partial sleep
			Test.ParticleHandles.Add(Test.Evolution.CreateDynamicParticles(1)[0]);
			Test.Evolution.EnableParticle(Test.ParticleHandles.Last());
			Test.ConstraintHandles.Add(Test.Constraints.AddConstraint({ Test.ParticleHandles.Last(1), Test.ParticleHandles.Last() }));
			Test.ParticleHandles.Last()->SetPartialIslandSleepAllowed(false);

			Test.Advance();

			// All particles wake up
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());

			// Explicitly put two particles asleep
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Sleeping);
			Test.Evolution.SetParticleObjectState(Test.ParticleHandles[1], EObjectStateType::Sleeping);

			// Disconnect the particle which does not allow partial sleep
			Test.ConstraintHandles.Last()->SetEnabled(false);

			Test.Advance();

			// Partial sleeping should work as usual
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
		}
	}

	TEST_P(GraphEvolutionTests, TestConstraintGraph_MomentumPropagation)
	{
		const bool bEnablePartialSleep = true;
		const FRealSingle MomentumPropagation = GetParam() ? 0.5f : 0.025f;

		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(MomentumPropagation);

		FGraphEvolutionTest Test(5, bEnablePartialSleep);
		Test.MakeChain();
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);

		Test.AdvanceUntilSleeping();

		// Explicitly wake up one of the top particle
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[4], EObjectStateType::Dynamic);
		Test.Advance();

		// One the particle explicitly woken and the particle next to it are awake.
		// Momentum of the woken particle is zero, i.e. no momentum propagation.
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());

		Test.AdvanceUntilSleeping();

		// Explicitly wake up one of the particles and add an impulse
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[4], EObjectStateType::Dynamic);
		FParticleDynamics Dynamics;
		Dynamics.SetAcceleration(FVec3(0, 0, 0));
		Dynamics.SetAngularAcceleration(FVec3(0, 0, 0));
		Dynamics.SetLinearImpulseVelocity(FVec3(0, 0.1, 0));
		Dynamics.SetAngularImpulseVelocity(FVec3(0, 0, 0));
		Test.Evolution.SetParticleDynamics(Test.ParticleHandles[4], Dynamics);

		Test.Advance();

		if (GetParam())
		{
			// All particles wake up, momentum is propagated through
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
		}
		else
		{
			// Only the first two particles wake up, momentum is too small to be propagated through
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
		}
		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}

	TEST_P(GraphEvolutionTests, TestConstraintGraph_MinimumMotionlessRatio)
	{
		const bool bEnablePartialSleep = true;
		const FRealSingle MinMotionlessRatio = GetParam() ? 0.5f : 0.1f;

		const FRealSingle MinMotionlessRatioDefault = CVarMinMotionlessRatio->GetFloat();
		CVarMinMotionlessRatio->Set(MinMotionlessRatio);

		const IConsoleVariable* CVarSleepModeSwitchCounterThreshold = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.SleepModeSwitchCounterThreshold"), false);
		check(CVarSleepModeSwitchCounterThreshold && CVarSleepModeSwitchCounterThreshold->IsVariableInt());
		const int32 SleepModeSwitchCounterThreshold = CVarSleepModeSwitchCounterThreshold->GetInt();
		const int32 Steps = SleepModeSwitchCounterThreshold;

		FGraphEvolutionTest Test(6, bEnablePartialSleep);
		Test.MakeChain();
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[0], EObjectStateType::Kinematic);

		Test.AdvanceUntilSleeping();

		FParticleDynamics Dynamics;
		Dynamics.SetAcceleration(FVec3(0, 0, 0));
		Dynamics.SetAngularAcceleration(FVec3(0, 0, 0));
		Dynamics.SetLinearImpulseVelocity(FVec3(0, 0.1, 0));
		Dynamics.SetAngularImpulseVelocity(FVec3(0, 0, 0));

		// Explicitly wake up one of the top 3 particles and apply and impulse
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[3], EObjectStateType::Dynamic);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[4], EObjectStateType::Dynamic);
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[5], EObjectStateType::Dynamic);
		Test.Evolution.SetParticleDynamics(Test.ParticleHandles[3], Dynamics);
		Test.Evolution.SetParticleDynamics(Test.ParticleHandles[4], Dynamics);
		Test.Evolution.SetParticleDynamics(Test.ParticleHandles[5], Dynamics);
		
		for (int32 I = 0; I < Steps; ++I)
		{
			Test.Advance();
		}

		// The particles explicitly woken and the particle next to it are awake.
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping()); // sleeping
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping()); // awake, motionless
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping()); // awake, moving
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping()); // awake, moving
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping()); // awake, moving

		// Partial island sleep remains enabled because particles have not moved enough steps
		EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsPartialSleepEnabled());

		Test.Advance();
		
		if (MinMotionlessRatio == 0.5f)
		{
			// Partial island sleeping is internally disabled since 2/5 = 40% of dynamic particles are motionless.
			EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsPartialSleepEnabled());
		}
		else
		{
			// Partial island sleeping is internally enabled since 2/5 = 40% of dynamic particles are motionless.
			EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsPartialSleepEnabled());
		}

		Test.Advance();

		if (MinMotionlessRatio == 0.5f)
		{
			// All particles wake up
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping());
		}
		else
		{
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping()); // sleeping
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping()); // awake, motionless
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping()); // awake, moving
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping()); // awake, moving
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping()); // awake, moving
		}

		Test.CVarPartialSleeping->Set(false);
		Test.Advance();

		// All particles wake up when disabling partial island sleep explicitly (CVar).
		EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[0])->IsKinematic());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[4])->Sleeping());
		EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[5])->Sleeping());

		CVarMinMotionlessRatio->Set(MinMotionlessRatioDefault);
	}

	TEST_P(GraphEvolutionTests, TestConstraintGraph_MinimumMotionlessRatio_Extremes)
	{
		const bool bEnablePartialSleep = true;
		// MinMotionlessRatio == 0.0f => 0% of particles need to be motionless for partial island sleep to be active
		// MinMotionlessRatio == 1.0f => 100% of particles need to be motionless for partial island sleep to be active
		const FRealSingle MinMotionlessRatio = GetParam() ? 1.0f : 0.0f;

		const FRealSingle MinMotionlessRatioDefault = CVarMinMotionlessRatio->GetFloat();
		CVarMinMotionlessRatio->Set(MinMotionlessRatio);

		const IConsoleVariable* CVarSleepModeSwitchCounterThreshold = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.SleepModeSwitchCounterThreshold"), false);
		check(CVarSleepModeSwitchCounterThreshold && CVarSleepModeSwitchCounterThreshold->IsVariableInt());
		const int32 SleepModeSwitchCounterThreshold = CVarSleepModeSwitchCounterThreshold->GetInt();
		const int32 Steps = SleepModeSwitchCounterThreshold + 1;

		FGraphEvolutionTest Test(5, bEnablePartialSleep);
		Test.MakeChain();

		FParticleDynamics Dynamics;
		Dynamics.SetAcceleration(FVec3(0, 0, 0));
		Dynamics.SetAngularAcceleration(FVec3(0, 0, 0));
		if (MinMotionlessRatio == 0.0f)
		{
			Dynamics.SetLinearImpulseVelocity(FVec3(0, 0.1, 0));
		}
		else // MinMotionlessRatio == 1.0f
		{
			Dynamics.SetLinearImpulseVelocity(FVec3(0, 0, 0));
		}
		Dynamics.SetAngularImpulseVelocity(FVec3(0, 0, 0));

		for (FPBDRigidParticleHandle* Particle : Test.ParticleHandles)
		{
			Test.Evolution.SetParticleObjectState(Particle, EObjectStateType::Dynamic);
			Test.Evolution.SetParticleDynamics(Particle, Dynamics);
		}

		for (int32 I = 0; I < Steps; ++I)
		{
			Test.Advance();

			EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsPartialSleepEnabled());
		}

		if (MinMotionlessRatio == 0.0f)
		{
			// Partial island sleeping remains enabled since all particles are in motion
			EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsPartialSleepEnabled());
		}
		else // MinMotionlessRatio == 1.0f
		{
			// Partial island sleeping remains enabled since all particles are in motionless
			EXPECT_TRUE(Test.IslandManager->GetIsland(0)->IsPartialSleepEnabled());
		}

		CVarMinMotionlessRatio->Set(MinMotionlessRatioDefault);
	}

	// Multithreaded access to the island manager is not supported.
	// This test attempts to remove particles from the constraint graph in parallel
	// which will corrupt the data stored in the island manager.
	// This is triggers similar crashes as in UE-215521, UE-296295, UE-296297, UE-297690, UE-314497.
	// The test needs to be disabled by default and helps testing the debug access detectors added to the island manager.
	TEST_P(GraphEvolutionTests, DISABLED_TestConstraintGraph_MultithreadedAccess)
	{
		const bool bEnablePartialSleep = true;
		const bool bInParallel = GetParam();

		FGraphEvolutionTest Test(10, bEnablePartialSleep);
		Test.MakeChain();
		Test.Advance();

		auto IslandManagerRemoveParticle = [](FPBDIslandManager* IslandManager,
			FGeometryParticleHandle* ParticleHandle)
			{
				IslandManager->RemoveParticle(ParticleHandle);
			};

		if (bInParallel)
		{
			// Multithreaded access not supported, this will crash in debug and development due to access detectors.
			PhysicsParallelFor(Test.ParticleHandles.Num(),
				[&](const int32 Index)
				{
					IslandManagerRemoveParticle(Test.IslandManager, Test.ParticleHandles[Index]);
				});
		}
		else
		{
			for (int32 Index = 0; Index < Test.ParticleHandles.Num(); ++Index)
			{
				IslandManagerRemoveParticle(Test.IslandManager, Test.ParticleHandles[Index]);
			}
		}
	}

	// Make sure any particle whose sleep type is NeverSleep will keep levels above awake.
	TEST_P(GraphEvolutionTests, TestConstraintGraph_NeverSleepFlag)
	{
		FGraphEvolutionTest Test(5, GetParam());

		Test.MakeChain();
		Test.Evolution.SetParticleObjectState(Test.ParticleHandles[4], EObjectStateType::Kinematic);

		// Tick once to create constraint graph and set the nodes' sleep settings
		Test.Advance();

		// Update the sleep type
		// Node sleep threshold won't be updated automatically
		Test.ParticleHandles[1]->SetSleepType(Chaos::ESleepType::NeverSleep);

		for (int32 Tick = 0; Tick < 200; ++Tick)
		{
			Test.Advance();
		}

		EXPECT_EQ(Test.IslandManager->GetNumIslands(), 1);
		EXPECT_FALSE(Test.IslandManager->GetIsland(0)->IsSleeping());
		EXPECT_TRUE(Test.ParticleHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[2]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[3]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ParticleHandles[3]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[0]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[1]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[2]->IsInConstraintGraph());
		EXPECT_TRUE(Test.ConstraintHandles[3]->IsInConstraintGraph());

		if (Test.CVarPartialSleeping->GetBool())
		{
			// All particles farther away from the kinematic than the never sleeping one should be awake (ParticleHandles[0]).
			// Level-based partial island wake will also wake up one level closer to the kinematic (ParticleHandles[2]). 
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[4])->IsKinematic());
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());

			EXPECT_TRUE(Test.ConstraintHandles[3]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[2]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[0]->IsSleeping());
		}
		else
		{
			EXPECT_TRUE(FConstGenericParticleHandle(Test.ParticleHandles[4])->IsKinematic());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[3])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[2])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[1])->Sleeping());
			EXPECT_FALSE(FConstGenericParticleHandle(Test.ParticleHandles[0])->Sleeping());

			EXPECT_FALSE(Test.ConstraintHandles[3]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[2]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[1]->IsSleeping());
			EXPECT_FALSE(Test.ConstraintHandles[0]->IsSleeping());
		}
	}

	INSTANTIATE_TEST_SUITE_P(, GraphEvolutionTests,	testing::Bool());
}
