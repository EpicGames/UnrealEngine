// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestHandles.h"
#include "HeadlessChaos.h"
#include "Chaos/ChaosArchive.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Chaos/HandleArray.h"

namespace ChaosTest
{
	using namespace Chaos;

	namespace Handles
	{
		struct TempStruct
		{
			TempStruct()
				: TempStruct(-1.0f, -1)
			{

			}

			TempStruct(FReal InFVal, int32 InIVal)
				: FVal(InFVal)
				, IVal(InIVal)
			{

			}

			FReal FVal;
			int32 IVal;

			friend FArchive operator <<(FArchive& Ar, TempStruct& InVal)
			{
				Ar << InVal.FVal << InVal.IVal;
				return Ar;
			}
		};

		template<typename ContainerType>
		int32 SumArrayWithHandles(ContainerType& InArray, TArray<typename ContainerType::FHandle>& InHandles)
		{
			int32 Result = 0;
			for(typename ContainerType::FHandle& CurrHandle : InHandles)
			{
				if(TempStruct* Inner = InArray.Get(CurrHandle))
				{
					Result += Inner->IVal;
				}
			}

			return Result;
		}

		template<typename ContainerType>
		int32 SumArray(ContainerType& InArray)
		{
			int32 Result = 0;
			const int32 NumEntries = InArray.Num();
			for(int32 i = 0; i < NumEntries; ++i)
			{
				if(const TempStruct* Inner = InArray.Get(InArray.GetConstHandle(i)))
				{
					Result += Inner->IVal;
				}
			}

			return Result;
		}

		void HandleArrayTest()
		{
			TArray<THandleArray<TempStruct>::FHandle> TempHandles;
			TempHandles.Reserve(1000);
			THandleArray<TempStruct> HandleArray0(10);
			
			for(int32 i = 0; i < 1000; ++i)
			{
				TempHandles.Add(HandleArray0.Create((FReal)i, i));
			}

			for(int32 i = 99; i < 1000; i += 100)
			{
				HandleArray0.Destroy(TempHandles[i]);
			}

			for(int32 i = 810; i < 860; ++i)
			{
				HandleArray0.Destroy(TempHandles[i]);
			}
			// Add a few to test the free list
			for(int32 i = 0; i < 10; ++i)
			{
				TempHandles.Add(HandleArray0.Create((FReal)i, i));
			}

			// Test that handles are tracked correctly and invalidate on removal
			int32 Sum0WithHandles = SumArrayWithHandles(HandleArray0, TempHandles);
			int32 Sum0NoHandles = SumArray(HandleArray0);
			EXPECT_EQ(Sum0WithHandles, Sum0NoHandles);

			// Test copy construction
			THandleArray<TempStruct> HandleArray1(HandleArray0);
			EXPECT_EQ(HandleArray0.Num(), HandleArray1.Num());
			EXPECT_EQ(HandleArray0.GetNumActive(), HandleArray1.GetNumActive());
			EXPECT_EQ(HandleArray0.GetCapacity(), HandleArray1.GetCapacity());
			EXPECT_EQ(SumArray(HandleArray0), SumArray(HandleArray1));

			// Test Move construction
			THandleArray<TempStruct> Moved = MoveTemp(HandleArray0);
			EXPECT_EQ(Moved.Num(), HandleArray1.Num());
			EXPECT_EQ(Moved.GetNumActive(), HandleArray1.GetNumActive());
			EXPECT_EQ(Moved.GetCapacity(), HandleArray1.GetCapacity());
			EXPECT_EQ(SumArray(Moved), SumArray(HandleArray1));

			EXPECT_NE(HandleArray0.Num(), HandleArray1.Num());
			EXPECT_NE(HandleArray0.GetNumActive(), HandleArray1.GetNumActive());
			EXPECT_NE(HandleArray0.GetCapacity(), HandleArray1.GetCapacity());
			EXPECT_NE(SumArray(HandleArray0), SumArray(HandleArray1));
		}

		void HandleHeapTest()
		{
			TArray<THandleHeap<TempStruct>::FHandle> TempHandles;
			TempHandles.Reserve(1000);
			THandleHeap<TempStruct> HandleArray0(10);

			for(int32 i = 0; i < 1000; ++i)
			{
				TempHandles.Add(HandleArray0.Create((FReal)i, i));
			}

			for(int32 i = 99; i < 1000; i += 100)
			{
				HandleArray0.Destroy(TempHandles[i]);
			}

			for(int32 i = 810; i < 860; ++i)
			{
				HandleArray0.Destroy(TempHandles[i]);
			}

			// Add a few to test the free list
			for(int32 i = 0; i < 10; ++i)
			{
				TempHandles.Add(HandleArray0.Create((FReal)i, i));
			}

			// Test that handles are tracked correctly and invalidate on removal
			int32 Sum0WithHandles = SumArrayWithHandles(HandleArray0, TempHandles);
			int32 Sum0NoHandles = SumArray(HandleArray0);
			EXPECT_EQ(Sum0WithHandles, Sum0NoHandles);

			// Test copy construction
			THandleHeap<TempStruct> HandleArray1(HandleArray0);
			EXPECT_EQ(HandleArray0.Num(), HandleArray1.Num());
			EXPECT_EQ(HandleArray0.GetNumActive(), HandleArray1.GetNumActive());
			EXPECT_EQ(SumArray(HandleArray0), SumArray(HandleArray1));

			// Test Move construction
			THandleHeap<TempStruct> Moved = MoveTemp(HandleArray0);
			EXPECT_EQ(Moved.Num(), HandleArray1.Num());
			EXPECT_EQ(Moved.GetNumActive(), HandleArray1.GetNumActive());
			EXPECT_EQ(SumArray(Moved), SumArray(HandleArray1));

			EXPECT_NE(HandleArray0.Num(), HandleArray1.Num());
			EXPECT_NE(HandleArray0.GetNumActive(), HandleArray1.GetNumActive());
			EXPECT_NE(SumArray(HandleArray0), SumArray(HandleArray1));
		}

		void HandleSerializeTest()
		{
			THandleArray<TempStruct> AsArray;
			THandleHeap<TempStruct> AsHeap;
			TArray<THandleArray<TempStruct>::FHandle> ArrayHandles;
			TArray<THandleHeap<TempStruct>::FHandle> HeapHandles;

			for(int i = 0; i < 500; ++i)
			{
				ArrayHandles.Add(AsArray.Create((FReal)i, i));
				HeapHandles.Add(AsHeap.Create((FReal)i, i));
			}

			// Make some holes
			for(int i = 0; i < 50; ++i)
			{
				AsArray.Destroy(ArrayHandles[100 + i]);
				AsArray.Destroy(ArrayHandles[300 + i]);
				AsHeap.Destroy(HeapHandles[100 + i]);
				AsHeap.Destroy(HeapHandles[300 + i]);
			}

			// Fill in to use the freelist
			for(int i = 0; i < 25; ++i)
			{
				ArrayHandles.Add(AsArray.Create((FReal)i, i));
				HeapHandles.Add(AsHeap.Create((FReal)i, i));
			}

			TArray<uint8> Bytes;
			FMemoryWriter Writer(Bytes);
			FChaosArchive WriteAr(Writer);

			WriteAr << AsArray << AsHeap << ArrayHandles << HeapHandles;

			FMemoryReader Reader(Bytes);
			FChaosArchive ReadAr(Reader);

			THandleArray<TempStruct> AsArrayRead;
			THandleHeap<TempStruct> AsHeapRead;
			TArray<THandleArray<TempStruct>::FHandle> ArrayHandlesRead;
			TArray<THandleHeap<TempStruct>::FHandle> HeapHandlesRead;

			ReadAr << AsArrayRead << AsHeapRead << ArrayHandlesRead << HeapHandlesRead;

			// Old Handles
			EXPECT_EQ(SumArrayWithHandles(AsArray, ArrayHandles), SumArrayWithHandles(AsArrayRead, ArrayHandles));
			EXPECT_EQ(SumArrayWithHandles(AsHeap, HeapHandles), SumArrayWithHandles(AsHeapRead, HeapHandles));

			// New Handles
			EXPECT_EQ(SumArrayWithHandles(AsArray, ArrayHandlesRead), SumArrayWithHandles(AsArrayRead, ArrayHandlesRead));
			EXPECT_EQ(SumArrayWithHandles(AsHeap, HeapHandlesRead), SumArrayWithHandles(AsHeapRead, HeapHandlesRead));

			// No Handles
			EXPECT_EQ(SumArray(AsArray), SumArray(AsArrayRead));
			EXPECT_EQ(SumArray(AsHeap), SumArray(AsHeapRead));

			// Basic data
			EXPECT_EQ(AsArray.Num(), AsArrayRead.Num());
			EXPECT_EQ(AsArray.GetNumActive(), AsArrayRead.GetNumActive());
			EXPECT_EQ(AsArray.GetCapacity(), AsArrayRead.GetCapacity());

			EXPECT_EQ(AsHeap.Num(), AsHeapRead.Num());
			EXPECT_EQ(AsHeap.GetNumActive(), AsHeapRead.GetNumActive());
		}

		void HandleArrayIteratorTest()
		{
			THandleArray<int> HandleArray(10);
			using FHandle = THandleArray<int>::FHandle;
			TArray<FHandle> Handles;

			for (int32 I = 0; I < 10; ++I)
			{
				Handles.Add(HandleArray.Create(I));
			}

			HandleArray.Destroy(Handles[0]);
			HandleArray.Destroy(Handles[1]);
			HandleArray.Destroy(Handles[5]);
			HandleArray.Destroy(Handles[8]);
			HandleArray.Destroy(Handles[9]);

			const TArray<int> ExpectedValues
			{
				2,
				3,
				4,
				6,
				7,
			};
			const TArray<FHandle> ExpectedHandles
			{
				Handles[2],
				Handles[3],
				Handles[4],
				Handles[6],
				Handles[7],
			};

			TArray<int> ActualValues;
			TArray<FHandle> ActualHandles;
			for (THandleArray<int>::TRangedForIterator It = HandleArray.begin(); It != HandleArray.end(); ++It)
			{
				ActualHandles.Add(It.GetHandle());
				int& Value = *It;
				ActualValues.Add(Value);
			}
			EXPECT_EQ(ActualHandles, ExpectedHandles);
			EXPECT_EQ(ActualValues, ExpectedValues);

			// Test the const iterator
			ActualHandles.Reset();
			ActualValues.Reset();
			const THandleArray<int>& ConstHandleArray = HandleArray;
			for (THandleArray<int>::TRangedForConstIterator It = ConstHandleArray.begin(); It != ConstHandleArray.end(); ++It)
			{
				ActualHandles.Add(It.GetHandle());
				const int& Value = *It;
				ActualValues.Add(Value);
			}
			EXPECT_EQ(ActualHandles, ExpectedHandles);
			EXPECT_EQ(ActualValues, ExpectedValues);
		}
	}
}