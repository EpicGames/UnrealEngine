// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ContactModification.h"
#include "ChaosSolversModule.h"
#include "HeadlessChaosTestUtility.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Algo/Count.h"

namespace ChaosTest
{
	using namespace Chaos;

	class FCollisionReportingCallback : public TSimCallbackObject<FSimCallbackNoInput, FSimCallbackNoOutput, Chaos::ESimCallbackOptions::Presimulate | Chaos::ESimCallbackOptions::ContactModification>
	{
	public:
		TArray<TTuple<FUniqueIdx, FUniqueIdx>> CollisionPairs;

	private:
		virtual void OnPreSimulate_Internal() override {}

		virtual void OnContactModification_Internal(Chaos::FCollisionContactModifier& Modifier) override
		{
			for(FContactPairModifier& PairModifier : Modifier)
			{
				TVec2<FGeometryParticleHandle*> Particles = PairModifier.GetParticlePair();

				if(Particles[0] && Particles[1])
				{
					FUniqueIdx Ids[2] = {Particles[0]->UniqueIdx(), Particles[1]->UniqueIdx()};
					if(Ids[1] < Ids[0])
					{
						Swap(Ids[0], Ids[1]);
					}
					CollisionPairs.AddUnique({ Ids[0], Ids[1] });
				}
			}
		}
	};

	void FlushSolver(FPBDRigidsSolver* InSolver)
	{
		InSolver->AdvanceAndDispatch_External(0);
		InSolver->WaitOnPendingTasks_External();

		// Populate the spacial acceleration
		FPBDRigidsSolver::FPBDRigidsEvolution* Evolution = InSolver->GetEvolution();

		if(Evolution)
		{
			Evolution->FlushSpatialAcceleration();
		}
	}

	void RunSolver(FPBDRigidsSolver* InSolver, float InDt, int32 NumSteps)
	{
		for(int Step = 0; Step < NumSteps; ++Step)
		{
			InSolver->AdvanceAndDispatch_External(InDt);
			InSolver->UpdateGameThreadStructures();
		}
	}

	GTEST_TEST(AllTraits, IgnoreManager_IgnoresCollisions)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// create a static floor and two boxes falling onto it.
		// One box has collisions ignored and should fall through, one should collide.

		// simulated cube with downward velocity,should collide with floor and not fall through.
		FSingleParticlePhysicsProxy* CollidingCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		FRigidBodyHandle_External& CollidingCubeParticle = CollidingCubeProxy->GetGameThreadAPI();
		Chaos::FImplicitObjectPtr CollidingCubeGeom(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		CollidingCubeParticle.SetGeometry(CollidingCubeGeom);
		Solver->RegisterObject(CollidingCubeProxy);
		CollidingCubeParticle.SetGravityEnabled(false);
		CollidingCubeParticle.SetV(FVec3(0, 0, -100));
		CollidingCubeParticle.SetX(FVec3(200, 0, 500));
		SetCubeInertiaTensor(CollidingCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ CollidingCubeProxy->GetParticle_LowLevel() });

		// Simulated cube with downward velocity, collision ignore disables collision with floor, should fall through.
		FSingleParticlePhysicsProxy* NonCollidableProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		FRigidBodyHandle_External& NonCollidableParticle = NonCollidableProxy->GetGameThreadAPI();
		Chaos::FImplicitObjectPtr NonCollidableCubeGeom(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		NonCollidableParticle.SetGeometry(NonCollidableCubeGeom);
		Solver->RegisterObject(NonCollidableProxy);
		NonCollidableParticle.SetGravityEnabled(false);
		NonCollidableParticle.SetV(FVec3(0, 0, -100));
		NonCollidableParticle.SetX(FVec3(-200, 0, 500));
		SetCubeInertiaTensor(NonCollidableParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ NonCollidableProxy->GetParticle_LowLevel() });

		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		FRigidBodyHandle_External& FloorParticle = FloorProxy->GetGameThreadAPI();
		Chaos::FImplicitObjectPtr FloorGeom(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });

		FCollisionReportingCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FCollisionReportingCallback>();

		// Named Ids for the particles
		const FUniqueIdx CollidableId = CollidingCubeParticle.UniqueIdx();
		const FUniqueIdx NonCollidableId = NonCollidableParticle.UniqueIdx();
		const FUniqueIdx FloorId = FloorParticle.UniqueIdx();

		// Flush the solver
		FlushSolver(Solver);

		// Set up the ignore manager
		FIgnoreCollisionManager& IgnoreManager = Solver->GetEvolution()->GetBroadPhase().GetIgnoreCollisionManager();
		IgnoreManager.AddIgnoreCollisions(NonCollidableProxy->GetHandle_LowLevel(), FloorProxy->GetHandle_LowLevel());

		RunSolver(Solver, 1.0f, 10);

		auto CountPred = [Callback](FUniqueIdx Idx) -> int32
		{
			return Algo::CountIf(Callback->CollisionPairs, [Idx](const TTuple<FUniqueIdx, FUniqueIdx>& Hit)
			{
				return Hit.Get<0>() == Idx || Hit.Get<1>() == Idx;
			});
		};

		TMap<FUniqueIdx, int32> Hits;
		Hits.Add({ CollidableId, CountPred(CollidableId) });
		Hits.Add({ NonCollidableId, CountPred(NonCollidableId) });
		Hits.Add({ FloorId, CountPred(FloorId) });

		EXPECT_EQ(Hits[CollidableId], 1);		// Collidable hit the floor
		EXPECT_EQ(Hits[FloorId], 1);			// Collidable hit the floor
		EXPECT_EQ(Hits[NonCollidableId], 0);	// Non collidable ignored all collisions with the floor

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(CollidingCubeProxy);
		Solver->UnregisterObject(NonCollidableProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, IgnoreManager_IgnoresCollisions_MultipleSources)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		// create a static floor and two boxes falling onto it.
		// One box has collisions ignored and should fall through, one should collide.

		// simulated cube with downward velocity,should collide with floor and not fall through.
		FSingleParticlePhysicsProxy* CollidingCubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		FRigidBodyHandle_External& CollidingCubeParticle = CollidingCubeProxy->GetGameThreadAPI();
		Chaos::FImplicitObjectPtr CollidingCubeGeom(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		CollidingCubeParticle.SetGeometry(CollidingCubeGeom);
		Solver->RegisterObject(CollidingCubeProxy);
		CollidingCubeParticle.SetGravityEnabled(false);
		CollidingCubeParticle.SetV(FVec3(0, 0, -100));
		CollidingCubeParticle.SetX(FVec3(200, 0, 500));
		SetCubeInertiaTensor(CollidingCubeParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ CollidingCubeProxy->GetParticle_LowLevel() });

		// Simulated cube with downward velocity, collision ignore disables collision with floor, should fall through.
		FSingleParticlePhysicsProxy* NonCollidableProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		FRigidBodyHandle_External& NonCollidableParticle = NonCollidableProxy->GetGameThreadAPI();
		Chaos::FImplicitObjectPtr NonCollidableCubeGeom(new TBox<FReal, 3>(FVec3(-100), FVec3(100)));
		NonCollidableParticle.SetGeometry(NonCollidableCubeGeom);
		Solver->RegisterObject(NonCollidableProxy);
		NonCollidableParticle.SetGravityEnabled(false);
		NonCollidableParticle.SetV(FVec3(0, 0, -10));
		NonCollidableParticle.SetX(FVec3(-200, 0, 110));
		SetCubeInertiaTensor(NonCollidableParticle, /*Dimension=*/200, /*Mass=*/1);
		ChaosTest::SetParticleSimDataToCollide({ NonCollidableProxy->GetParticle_LowLevel() });

		// static floor at origin, occupying Z = [-100,0]
		FSingleParticlePhysicsProxy* FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		FRigidBodyHandle_External& FloorParticle = FloorProxy->GetGameThreadAPI();
		Chaos::FImplicitObjectPtr FloorGeom(new TBox<FReal, 3>(FVec3(-500, -500, -100), FVec3(500, 500, 0)));
		FloorParticle.SetGeometry(FloorGeom);
		Solver->RegisterObject(FloorProxy);
		FloorParticle.SetX(FVec3(0, 0, 0));
		ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });

		FCollisionReportingCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FCollisionReportingCallback>();

		// Named Ids for the particles
		const FUniqueIdx CollidableId = CollidingCubeParticle.UniqueIdx();
		const FUniqueIdx NonCollidableId = NonCollidableParticle.UniqueIdx();
		const FUniqueIdx FloorId = FloorParticle.UniqueIdx();

		// Flush the solver
		FlushSolver(Solver);

		// Set up the ignore manager (Twice as if two systems requested it)
		FIgnoreCollisionManager& IgnoreManager = Solver->GetEvolution()->GetBroadPhase().GetIgnoreCollisionManager();
		IgnoreManager.AddIgnoreCollisions(NonCollidableProxy->GetHandle_LowLevel(), FloorProxy->GetHandle_LowLevel());
		IgnoreManager.AddIgnoreCollisions(NonCollidableProxy->GetHandle_LowLevel(), FloorProxy->GetHandle_LowLevel());

		const float Dt = 1.0f;
		const int32 Steps = 10;
		for(int Step = 0; Step < Steps; ++Step)
		{
			Solver->AdvanceAndDispatch_External(Dt);
			Solver->UpdateGameThreadStructures();
		}

		auto CountPred = [Callback](FUniqueIdx Idx) -> int32
		{
			return Algo::CountIf(Callback->CollisionPairs, [Idx](const TTuple<FUniqueIdx, FUniqueIdx>& Hit)
			{
				return Hit.Get<0>() == Idx || Hit.Get<1>() == Idx;
			});
		};

		TMap<FUniqueIdx, int32> Hits;
		Hits.Add({ CollidableId, CountPred(CollidableId) });
		Hits.Add({ NonCollidableId, CountPred(NonCollidableId) });
		Hits.Add({ FloorId, CountPred(FloorId) });

		EXPECT_EQ(Hits[CollidableId], 1);		// Collidable hit the floor
		EXPECT_EQ(Hits[FloorId], 1);			// Collidable hit the floor
		EXPECT_EQ(Hits[NonCollidableId], 0);	// Non collidable ignored all collisions with the floor

		// Remove one source
		IgnoreManager.RemoveIgnoreCollisions(NonCollidableProxy->GetHandle_LowLevel(), FloorProxy->GetHandle_LowLevel());

		Callback->CollisionPairs.Reset();
		RunSolver(Solver, 1.0f, 10);

		Hits[CollidableId] = CountPred(CollidableId);
		Hits[NonCollidableId] = CountPred(NonCollidableId);
		Hits[FloorId] = CountPred(FloorId);

		EXPECT_EQ(Hits[CollidableId], 1);		// Collidable hit the floor
		EXPECT_EQ(Hits[FloorId], 1);			// Collidable hit the floor
		EXPECT_EQ(Hits[NonCollidableId], 0);	// Non collidable ignored all collisions with the floor

		// Remove final source
		IgnoreManager.RemoveIgnoreCollisions(NonCollidableProxy->GetHandle_LowLevel(), FloorProxy->GetHandle_LowLevel());

		Callback->CollisionPairs.Reset();
		RunSolver(Solver, 1.0f, 10);

		Hits[CollidableId] = CountPred(CollidableId);
		Hits[NonCollidableId] = CountPred(NonCollidableId);
		Hits[FloorId] = CountPred(FloorId);

		EXPECT_EQ(Hits[CollidableId], 1);		// Collidable hit the floor
		EXPECT_EQ(Hits[FloorId], 2);			// Collidable and NonCollidable hit the floor
		EXPECT_EQ(Hits[NonCollidableId], 1);	// Non collidable is now collidable and hits the floor

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);
		Solver->UnregisterObject(CollidingCubeProxy);
		Solver->UnregisterObject(NonCollidableProxy);
		Solver->UnregisterObject(FloorProxy);
		Module->DestroySolver(Solver);
	}
}