// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Box.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/Matrix.h"
#include "Chaos/MassProperties.h"
#include "Chaos/TriangleMesh.h"
#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Modules/ModuleManager.h"

namespace ChaosTest 
{
	using namespace Chaos;

	// Box inertia is correct
	GTEST_TEST(ImplicitMassPropertiesTests, TestImplicitBoxMassPropeerties)
	{
		// Offset box
		const FReal ImplicitMass = 100.0f;
		const FVec3 ImplicitMin = FVec3(100, 200, 300);
		const FVec3 ImplicitMax = FVec3(200, 400, 600);
		const FImplicitBox3 Implicit(ImplicitMin, ImplicitMax);

		const FMatrix33 ImplicitInertia = Implicit.GetInertiaTensor(ImplicitMass);
		const FVec3 ImplicitCoM = Implicit.GetCenterOfMass();
		const FRotation3 ImplicitRoM = Implicit.GetRotationOfMass();

		const FVec3 ExpectedCoM = 0.5f * (ImplicitMin + ImplicitMax);
		const FRotation3 ExpectedRoM = FRotation3::FromIdentity();
		EXPECT_NEAR(ExpectedCoM.X, ImplicitCoM.X, 1.e-2f);
		EXPECT_NEAR(ExpectedCoM.Y, ImplicitCoM.Y, 1.e-2f);
		EXPECT_NEAR(ExpectedCoM.Z, ImplicitCoM.Z, 1.e-2f);
	}


	// Convex box inertia is correct
	GTEST_TEST(ImplicitMassPropertiesTests, TestImplicitConvexBoxMassPropeerties)
	{
		// Offset box
		const FReal ImplicitMass = 100.0f;
		const FVec3 ImplicitMin = FVec3(100, 200, 300);
		const FVec3 ImplicitMax = FVec3(200, 400, 600);
		const FImplicitConvex3 Implicit = CreateConvexBox(ImplicitMin, ImplicitMax, 0.0f);

		const FMatrix33 ImplicitInertia = Implicit.GetInertiaTensor(ImplicitMass);
		const FVec3 ImplicitCoM = Implicit.GetCenterOfMass();
		const FRotation3 ImplicitRoM = Implicit.GetRotationOfMass();

		const FImplicitBox3 Box(ImplicitMin, ImplicitMax, 0.0f);
		const FVec3 ExpectedCoM = Box.GetCenterOfMass();
		const FRotation3 ExpectedRoM = Box.GetRotationOfMass();
		const FMatrix33 ExectedInertia = Box.GetInertiaTensor(ImplicitMass);
		EXPECT_NEAR(ExpectedCoM.X, ImplicitCoM.X, 1.e-2f);
		EXPECT_NEAR(ExpectedCoM.Y, ImplicitCoM.Y, 1.e-2f);
		EXPECT_NEAR(ExpectedCoM.Z, ImplicitCoM.Z, 1.e-2f);
		EXPECT_NEAR(ImplicitInertia.M[0][0], ExectedInertia.M[0][0], 1.e-1f);
		EXPECT_NEAR(ImplicitInertia.M[1][1], ExectedInertia.M[1][1], 1.e-1f);
		EXPECT_NEAR(ImplicitInertia.M[2][2], ExectedInertia.M[2][2], 1.e-1f);
	}


	// Scaled convex box inertia is correct
	GTEST_TEST(ImplicitMassPropertiesTests, TestImplicitScaledConvexBoxMassPropeerties)
	{
		// Offset box
		const FReal ImplicitMass = 100.0f;
		const FVec3 ImplicitSize = FVec3(100, 200, 300);
		const FVec3 ImplicitScale = FVec3(1.5f, 1.8f, 1.1f);
		const TImplicitObjectScaled<FImplicitConvex3> Implicit = CreateScaledConvexBox(ImplicitSize, ImplicitScale, 0.0f);

		const FMatrix33 ImplicitInertia = Implicit.GetInertiaTensor(ImplicitMass);
		const FVec3 ImplicitCoM = Implicit.GetCenterOfMass();
		const FRotation3 ImplicitRoM = Implicit.GetRotationOfMass();

		const FImplicitBox3 Box(-0.5f * ImplicitScale * ImplicitSize, 0.5f * ImplicitScale * ImplicitSize, 0.0f);
		const FVec3 ExpectedCoM = Box.GetCenterOfMass();
		const FRotation3 ExpectedRoM = Box.GetRotationOfMass();
		const FMatrix33 ExectedInertia = Box.GetInertiaTensor(ImplicitMass);
		EXPECT_NEAR(ExpectedCoM.X, ImplicitCoM.X, 1.e-2f);
		EXPECT_NEAR(ExpectedCoM.Y, ImplicitCoM.Y, 1.e-2f);
		EXPECT_NEAR(ExpectedCoM.Z, ImplicitCoM.Z, 1.e-2f);
		EXPECT_NEAR(ImplicitInertia.M[0][0], ExectedInertia.M[0][0], 1.f);
		EXPECT_NEAR(ImplicitInertia.M[1][1], ExectedInertia.M[1][1], 1.f);
		EXPECT_NEAR(ImplicitInertia.M[2][2], ExectedInertia.M[2][2], 1.f);
	}


	// @todo(chaos): Convex shape that is not a box

	// @todo(chaos): Compound transformed scaled shape inertia

	// Transformed Scaled convex box inertia is correct
	// @chaos(todo): Cannot be tested unless we make GetInertiaTensor virtual on 
	// ImplicitObject (required to make GetInertiaTensor compile on Transformed Implicit)
#if 0
	GTEST_TEST(ImplicitMassPropertiesTests, TestImplicitTransformedConvexBoxMassPropeerties)
	{
		// Offset box
		const FReal ImplicitMass = 100.0f;
		const FVec3 ImplicitCenter = FVec3(100, 100, 100);
		const FVec3 ImplicitSize = FVec3(100, 200, 300);
		const FVec3 ImplicitScale = FVec3(1.5f, 1.8f, 1.1f);
		TImplicitObjectScaled<FImplicitConvex3> Convex = CreateScaledConvexBox(ImplicitSize, ImplicitScale, 0.0f);
		const TImplicitObjectTransformed<FReal, 3, false> Implicit(&Convex, FRigidTransform3(ImplicitCenter, FRotation3::FromIdentity()));

		const FMatrix33 ImplicitInertia = Implicit.GetInertiaTensor(ImplicitMass);
		const FVec3 ImplicitCoM = Implicit.GetCenterOfMass();
		const FRotation3 ImplicitRoM = Implicit.GetRotationOfMass();

		const FImplicitBox3 Box(ImplicitCenter - 0.5f * ImplicitScale * ImplicitSize, ImplicitCenter + 0.5f * ImplicitScale * ImplicitSize, 0.0f);
		const FVec3 ExpectedCoM = Box.GetCenterOfMass();
		const FRotation3 ExpectedRoM = Box.GetRotationOfMass();
		const FMatrix33 ExectedInertia = Box.GetInertiaTensor(ImplicitMass);
		EXPECT_NEAR(ExpectedCoM.X, ImplicitCoM.X, 1.e-2f);
		EXPECT_NEAR(ExpectedCoM.Y, ImplicitCoM.Y, 1.e-2f);
		EXPECT_NEAR(ExpectedCoM.Z, ImplicitCoM.Z, 1.e-2f);
		EXPECT_NEAR(ImplicitInertia.M[0][0], ExectedInertia.M[0][0], 1.e-1f);
		EXPECT_NEAR(ImplicitInertia.M[1][1], ExectedInertia.M[1][1], 1.e-1f);
		EXPECT_NEAR(ImplicitInertia.M[2][2], ExectedInertia.M[2][2], 1.e-1f);
	}
#endif

}
