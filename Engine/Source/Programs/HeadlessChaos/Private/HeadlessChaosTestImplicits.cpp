// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestImplicits.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosCollisionConstraints.h"
#include "Modules/ModuleManager.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/BoundingVolumeHierarchy.h"
#include "Chaos/Box.h"
#include "Chaos/Plane.h"
#include "Chaos/Sphere.h"
#include "Chaos/Tetrahedron.h"
#include "Chaos/Cylinder.h"
#include "Chaos/TaperedCylinder.h"
#include "Chaos/TaperedCapsule.h"
#include "Chaos/Capsule.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectIntersection.h"
#include "Chaos/Levelset.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Utilities.h"
#include "Chaos/CastingUtilities.h"
#include "Chaos/Convex.h"
#include "Math/RandomStream.h"
#include "Chaos/ErrorReporter.h"

#define RUN_KNOWN_BROKEN_TESTS 0

namespace ChaosTest {

	using namespace Chaos;

	DEFINE_LOG_CATEGORY_STATIC(LogChaosTestImplicits, Verbose, All);
	

	/* HELPERS */


	/* Takes an ImplicitObject of unit size (circumscribed inside a 2x2 cube centered on the origin). 
	   Tests the .Normal() function and the .SignedDistance() function. */
	void UnitImplicitObjectNormalsInternal(FImplicitObject &Subject, FString Caller)
	{
		FString Error = FString("Called by ") + Caller + FString(".");

#if RUN_KNOWN_BROKEN_TESTS
		// Normal when equally close to many points (currently inconsistent between geometries)
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0)), FVec3(0, 0, 0), KINDA_SMALL_NUMBER, Error);
#endif

		// inside normal
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 0, 1 / 2.)), (FVec3(0, 0, 1)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 0, -1 / 2.)), (FVec3(0, 0, -1)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 1 / 2., 0)), (FVec3(0, 1, 0)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, -1 / 2., 0)), (FVec3(0, -1, 0)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(1 / 2., 0, 0)), (FVec3(1, 0, 0)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(-1 / 2., 0, 0)), (FVec3(-1, 0, 0)), KINDA_SMALL_NUMBER, Error);

		// inside phi
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0, 0, 1 / 2.)), -1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0, 0, -1 / 2.)), -1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0, 1 / 2., 0)), -1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0, -1 / 2., 0)), -1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(1 / 2., 0, 0)), -1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-1 / 2., 0, 0)), -1 / 2., KINDA_SMALL_NUMBER) << *Error;

	}

	void UnitImplicitObjectNormalsExternal(FImplicitObject &Subject, FString Caller)
	{
		FString Error = FString("Called by ") + Caller + FString(".");

		// outside normal 
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 0, 3 / 2.)), (FVec3(0, 0, 1)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 0, -3 / 2.)), (FVec3(0, 0, -1)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 3 / 2., 0)), (FVec3(0, 1, 0)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, -3 / 2., 0)), (FVec3(0, -1, 0)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(3 / 2., 0, 0)), (FVec3(1, 0, 0)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(-3 / 2., 0, 0)), (FVec3(-1, 0, 0)), KINDA_SMALL_NUMBER, Error);

		// outside phi
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0, 0, 3 / 2.)), 1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0, 0, -3 / 2.)), 1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0, 3 / 2., 0)), 1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0, -3 / 2., 0)), 1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(3 / 2., 0, 0)), 1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-3 / 2., 0, 0)), 1 / 2., KINDA_SMALL_NUMBER) << *Error;
	}


	/* Given an ImplicitObject and an InputPoint, verifies that when that point is reflected across the surface of the object, the point of 
	   intersection between those two points is ExpectedPoint. */
	void TestFindClosestIntersection(FImplicitObject& Subject, FVec3 InputPoint, FVec3 ExpectedPoint, FString Caller)
	{
		FString Error = FString("Called by ") + Caller + FString(".");
		
		FReal SamplePhi = Subject.SignedDistance(InputPoint);
		FVec3 SampleNormal = Subject.Normal(InputPoint);
		FVec3 EndPoint = InputPoint + SampleNormal * SamplePhi*-2.;
		Pair<FVec3, bool> Result = Subject.FindClosestIntersection(InputPoint, EndPoint, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR_ERR(Result.First, ExpectedPoint, 0.001, Error);
	}


	/* Takes an ImplicitObject of unit size (circumscribed inside a 2x2 cube centered on the origin).
	   Tests the FindClosestIntersection functionality on a point near the top of the unit object. */
	void UnitImplicitObjectIntersections(FImplicitObject &Subject, FString Caller)
	{
		// closest point near origin (+)
		TestFindClosestIntersection(Subject, FVec3(0, 0, 2), FVec3(0, 0, 1), Caller);

		// closest point near origin (-)
		TestFindClosestIntersection(Subject, FVec3(0, 0, 1 / 2.), FVec3(0, 0, 1), Caller);
	}


	/* Takes an ImplicitObject of unit size (circumscribed inside a 2x2 cube centered on the origin).
	   Tests the .Support() function. */
	template<typename GeometryType>
	void UnitImplicitObjectSupportPhis(GeometryType& Subject, FString Caller)
	{
		FString Error = FString("Called by ") + Caller + FString(".");
		int32 VertexIndex = INDEX_NONE;

		// support phi
		EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(0, 0, 1), FReal(0), VertexIndex), (FVec3(0, 0, 1)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(0, 0, -1), FReal(0), VertexIndex), (FVec3(0, 0, -1)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(0, 1, 0), FReal(0), VertexIndex), (FVec3(0, 1, 0)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(0, -1, 0), FReal(0), VertexIndex), (FVec3(0, -1, 0)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(1, 0, 0), FReal(0), VertexIndex), (FVec3(1, 0, 0)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(-1, 0, 0), FReal(0), VertexIndex), (FVec3(-1, 0, 0)), KINDA_SMALL_NUMBER, Error);

		EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(0, 0, 1), FReal(1), VertexIndex), (FVec3(0, 0, 2)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(0, 0, -1), FReal(1), VertexIndex), (FVec3(0, 0, -2)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(0, 1, 0), FReal(1), VertexIndex), (FVec3(0, 2, 0)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(0, -1, 0), FReal(1), VertexIndex), (FVec3(0, -2, 0)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(1, 0, 0), FReal(1), VertexIndex), (FVec3(2, 0, 0)), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(-1, 0, 0), FReal(1), VertexIndex), (FVec3(-2, 0, 0)), KINDA_SMALL_NUMBER, Error);
	}

	/* Takes 3 ImplictObject of unit size (circumscribed inside a 2x2 cube)
	*  - One aligned on axis centered on origin
	*  - One aligned on axis offset from origin
	*  - One not axis-aligned 
	*   Test the InertiaTensor and the RotationOfMass. 
	*/
	template<typename GeometryType>
	void UnitImplicitObjectInertiaTensorAndRotationOfMass(const GeometryType& AlignedSubject, const GeometryType& OffsetedAlignedSubject, const GeometryType& NonAlignedSubject,FString Caller)
	{
		const FReal Mass = (FReal)100.;
		FVec3 AlignedInertiaTensor = AlignedSubject.GetInertiaTensor(Mass).GetDiagonal();
		FVec3 OffsetedAlignedInertiaTensor = OffsetedAlignedSubject.GetInertiaTensor(Mass).GetDiagonal();
		FVec3 NonAlignedInteriaTensor = NonAlignedSubject.GetInertiaTensor(Mass).GetDiagonal();
		EXPECT_NEAR(AlignedInertiaTensor.X, OffsetedAlignedInertiaTensor.X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(AlignedInertiaTensor.Y, OffsetedAlignedInertiaTensor.Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(AlignedInertiaTensor.Z, OffsetedAlignedInertiaTensor.Z, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(AlignedInertiaTensor.X, NonAlignedInteriaTensor.X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(AlignedInertiaTensor.Y, NonAlignedInteriaTensor.Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(AlignedInertiaTensor.Z, NonAlignedInteriaTensor.Z, KINDA_SMALL_NUMBER);

		FRotation3 AlignedRotationOfMass = AlignedSubject.GetRotationOfMass();
		FRotation3 OffsetedAlignedRotationOfMass = OffsetedAlignedSubject.GetRotationOfMass();
		FRotation3 NonAlignedRotationOfMass = NonAlignedSubject.GetRotationOfMass();
		EXPECT_TRUE(FRotation3::IsNearlyEqual(AlignedRotationOfMass, OffsetedAlignedRotationOfMass, KINDA_SMALL_NUMBER));
		EXPECT_FALSE(FRotation3::IsNearlyEqual(AlignedRotationOfMass, NonAlignedRotationOfMass, KINDA_SMALL_NUMBER));
	}

	/* IMPLICIT OBJECT TESTS */


	void ImplicitPlane()
	{
		FString Caller("ImplicitPlane()");

		{// basic tests
			TPlane<FReal, 3> Subject(FVec3(0), FVec3(0, 0, 1));

			// check samples about the origin. 
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.Normal(FVec3(1, 1, 1)), (FVec3(0, 0, 1)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.Normal(FVec3(-1, -1, -1)), (FVec3(0, 0, 1)));

			EXPECT_EQ(Subject.SignedDistance(FVec3(1, 1, 1)) , 1.f);
			EXPECT_EQ(Subject.SignedDistance(FVec3(-1, -1, -1)) , -1.f);
	
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.FindClosestPoint(FVec3(0, 0, 1)), (FVec3(0, 0, 0)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.FindClosestPoint(FVec3(1, 1, 2)), (FVec3(1, 1, 0)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.FindClosestPoint(FVec3(0, 0, -1)), (FVec3(0, 0, 0)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.FindClosestPoint(FVec3(1, 1, -2)), (FVec3(1, 1, 0)));
		}
		
		{// closest point near origin
			TPlane<FReal, 3> Subject(FVec3(0), FVec3(0, 0, 1));
			FVec3 InputPoint = FVec3(1, 1, 1);
			TestFindClosestIntersection(Subject, InputPoint, FVec3(1, 1, 0), Caller);
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.FindClosestPoint(InputPoint), (FVec3(1, 1, 0)));
		}

		{// closest point single axis off origin (+)
			FVec3 InputPoint = FVec3(0, 0, 2);
			TPlane<FReal, 3> Subject = TPlane<FReal, 3>(FVec3(0, 0, 1), FVec3(0, 0, 1));
			TestFindClosestIntersection(Subject, InputPoint, FVec3(0, 0, 1), Caller);
			EXPECT_VECTOR_NEAR(Subject.FindClosestPoint(InputPoint), FVector(0, 0, 1), 0.001);
			EXPECT_VECTOR_NEAR(Subject.FindClosestPoint(FVec3(0, 1, 2)), FVector(0,1,1), 0.001);
		}
		
		{// closest point off origin (+)
			FVec3 InputPoint = FVec3(11,11,11);
			TPlane<FReal, 3> Subject = TPlane<FReal, 3>(FVec3(10, 10, 10), FVec3(1, 1, 1).GetSafeNormal());
			TestFindClosestIntersection(Subject, InputPoint, FVec3(10, 10, 10), Caller);
			FVec3 NearestPoint = Subject.FindClosestPoint(InputPoint); // wrong (9.26...)
			EXPECT_VECTOR_NEAR(Subject.FindClosestPoint(InputPoint), FVector(10, 10, 10), 0.001);
		}

		{// closest point off origin (-)
			FVec3 InputPoint = FVec3(9,9,9);
			TPlane<FReal, 3>Subject = TPlane<FReal, 3>(FVec3(10, 10, 10), FVec3(1, 1, 1).GetSafeNormal());
			TestFindClosestIntersection(Subject, InputPoint, FVec3(10, 10, 10), Caller);
			FVec3 NearestPoint = Subject.FindClosestPoint(InputPoint); // (10.73...)
			EXPECT_VECTOR_NEAR(Subject.FindClosestPoint(InputPoint), FVector(10, 10, 10), 0.001);
		}
	}

	void ImplicitTetrahedron()
	{
		FString Caller("ImplicitTetrahedron()");

		TTetrahedron<double> Tet(TVec3<double>(0, 0, 0), TVec3<double>(1, 0, 0), TVec3<double>(0, 1, 0), TVec3<double>(0, 0, 1));
		const TArray<TTriangle<double>> Tris = Tet.GetTriangles();

		double SVol = Tet.GetSignedVolume();
		EXPECT_NEAR(SVol, 1.0/6, 0.001);

		double Vol = Tet.GetVolume();
		EXPECT_NEAR(Vol, SVol, 0.001);

		double MinEL = Tet.GetMinEdgeLength();
		EXPECT_NEAR(MinEL, 1.0, 0.001);

		double MaxEL = Tet.GetMaxEdgeLength();
		EXPECT_NEAR(MaxEL, 1.4, 0.1);

		TVec3<double> Center = Tet.GetCenter();
		EXPECT_VECTOR_NEAR(Center, FVector(.25, .25, .25), 0.001);

		// Center
		TVec3<double> Pt = Center;
		bool Hit = !Tet.Outside(Pt); // generates tris
		EXPECT_TRUE(Hit);
		bool Inside = Tet.Inside(Pt);
		EXPECT_TRUE(Hit == Inside);
		bool RobustHit = Tet.RobustInside(Pt);
		EXPECT_TRUE(RobustHit);
		TVec3<double> Bary = Tet.GetFirstThreeBarycentricCoordinates(Pt);
		EXPECT_VECTOR_NEAR(Bary, FVector(.25, .25, .25), 0.001);
		TVec3<double> Surf = Tet.ProjectToSurface(Tris, Pt);
		EXPECT_VECTOR_NEAR(Surf, Tris[3].GetCentroid(), 0.001);
		TVec4<double> ClosestBary;
		TVec3<double> ClosestPoint = Tet.FindClosestPointAndBary(Pt, ClosestBary);
		EXPECT_VECTOR_NEAR(ClosestPoint, Pt, 0.001);
		EXPECT_VECTOR_NEAR(ClosestBary, TVec4<double>(0.25, 0.25, 0.25, 0.25), 0.001);

		// Point
		Pt = Tet[0];
		Hit = !Tet.Outside(Pt, 0.001);
		EXPECT_TRUE(Hit);
		Inside = Tet.Inside(Pt);
		EXPECT_TRUE(Hit == Inside);
		RobustHit = Tet.RobustInside(Pt, -0.001);
		EXPECT_TRUE(RobustHit);
		Bary = Tet.GetFirstThreeBarycentricCoordinates(Pt);
		EXPECT_VECTOR_NEAR(Bary, FVector(1, 0, 0), 0.001);
		Surf = Tet.ProjectToSurface(Tris, Pt);
		EXPECT_VECTOR_NEAR(Surf, Pt, 0.001);
		ClosestPoint = Tet.FindClosestPointAndBary(Pt, ClosestBary);
		EXPECT_VECTOR_NEAR(ClosestPoint, Pt, 0.001);
		EXPECT_VECTOR_NEAR(ClosestBary, TVec4<double>(1, 0, 0, 0), 0.001);


		Pt[0] -= 0.1;
		Hit = !Tet.Outside(Pt, 0.001);
		EXPECT_FALSE(Hit);
		Inside = Tet.Inside(Pt);
		EXPECT_TRUE(Hit == Inside);
		RobustHit = Tet.RobustInside(Pt, -0.001);
		EXPECT_FALSE(RobustHit);
		Surf = Tet.ProjectToSurface(Tris, Pt);
		ClosestPoint = Tet.FindClosestPointAndBary(Pt, ClosestBary);
		Pt[0] += 0.1;
		EXPECT_VECTOR_NEAR(Surf, Pt, 0.001);
		EXPECT_VECTOR_NEAR(ClosestPoint, Tet[0], 0.001);
		EXPECT_VECTOR_NEAR(ClosestBary, TVec4<double>(1, 0, 0, 0), 0.001);

		Pt -= FVec3(0.1);
		Surf = Tet.ProjectToSurface(Tris, Pt);
		ClosestPoint = Tet.FindClosestPointAndBary(Pt, ClosestBary);
		Pt += FVec3(0.1);
		EXPECT_VECTOR_NEAR(Surf, Pt, 0.001);
		EXPECT_VECTOR_NEAR(ClosestPoint, Tet[0], 0.001);
		EXPECT_VECTOR_NEAR(ClosestBary, TVec4<double>(1, 0, 0, 0), 0.001);

		// Edge
		Pt = TVec3<double>(0.5, 0, 0);
		Hit = !Tet.Outside(Pt, 0.001);
		EXPECT_TRUE(Hit);
		Inside = Tet.Inside(Pt);
		EXPECT_TRUE(Hit == Inside);
		RobustHit = Tet.RobustInside(Pt, -0.001);
		EXPECT_TRUE(RobustHit);
		Bary = Tet.GetFirstThreeBarycentricCoordinates(Pt);
		EXPECT_VECTOR_NEAR(Bary, FVector(.5, .5, 0), 0.001);
		Surf = Tet.ProjectToSurface(Tris, Pt);
		EXPECT_VECTOR_NEAR(Surf, Pt, 0.001);
		ClosestPoint = Tet.FindClosestPointAndBary(Pt, ClosestBary);
		EXPECT_VECTOR_NEAR(ClosestPoint, Pt, 0.001);
		EXPECT_VECTOR_NEAR(ClosestBary, TVec4<double>(.5, .5, 0, 0), 0.001);

		Pt[1] -= 0.1;
		Hit = !Tet.Outside(Pt, 0.001);
		EXPECT_FALSE(Hit);
		Inside = Tet.Inside(Pt);
		EXPECT_TRUE(Hit == Inside);
		RobustHit = Tet.RobustInside(Pt, -0.001);
		EXPECT_FALSE(RobustHit);
		Surf = Tet.ProjectToSurface(Tris, Pt);
		ClosestPoint = Tet.FindClosestPointAndBary(Pt, ClosestBary);
		Pt[1] += 0.1;
		EXPECT_VECTOR_NEAR(Surf, Pt, 0.001);
		EXPECT_VECTOR_NEAR(ClosestPoint, Pt, 0.001);
		EXPECT_VECTOR_NEAR(ClosestBary, TVec4<double>(.5, .5, 0, 0), 0.001);

		// This is a test case where ProjectToSurface will fail but FindClosestPointAndBary will succeed
		Pt = TVec3<double>(1, 0, 0);
		Pt += TVec3<double>(.1, -.05, 0);
		//Surf = Tet.ProjectToSurface(Tris, Pt);
		ClosestPoint = Tet.FindClosestPointAndBary(Pt, ClosestBary);
		Pt -= TVec3<double>(.1, -.05, 0);
		//EXPECT_VECTOR_NEAR(Surf, Pt, 0.001);
		EXPECT_VECTOR_NEAR(ClosestPoint, Pt, 0.001);
		EXPECT_VECTOR_NEAR(ClosestBary, TVec4<double>(0, 1, 0, 0), 0.001);

		// Face
		Pt = Tris[0].GetCentroid();
		Hit = !Tet.Outside(Pt, 0.001);
		EXPECT_TRUE(Hit);
		Inside = Tet.Inside(Pt);
		EXPECT_TRUE(Hit == Inside);
		RobustHit = Tet.RobustInside(Pt, -0.001);
		EXPECT_TRUE(RobustHit);
		Bary = Tet.GetFirstThreeBarycentricCoordinates(Pt);
		EXPECT_VECTOR_NEAR(Bary, FVector(.3333, .3333, .3333), 0.001);
		Surf = Tet.ProjectToSurface(Tris, Pt);
		EXPECT_VECTOR_NEAR(Surf, Pt, 0.001);
		ClosestPoint = Tet.FindClosestPointAndBary(Pt, ClosestBary);
		EXPECT_VECTOR_NEAR(ClosestPoint, Pt, 0.001);
		EXPECT_VECTOR_NEAR(ClosestBary, TVec4<double>(.3333, .3333, .3333, 0), 0.001);

		Pt += Tris[0].GetNormal() * 0.1;
		Hit = !Tet.Outside(Pt, 0.001);
		EXPECT_FALSE(Hit);
		Inside = Tet.Inside(Pt);
		EXPECT_TRUE(Hit == Inside);
		RobustHit = Tet.RobustInside(Pt, -0.001);
		EXPECT_FALSE(RobustHit);
		Surf = Tet.ProjectToSurface(Tris, Pt);
		ClosestPoint = Tet.FindClosestPointAndBary(Pt, ClosestBary);
		Pt -= Tris[0].GetNormal() * 0.1;
		EXPECT_VECTOR_NEAR(Surf, Pt, 0.001);
		EXPECT_VECTOR_NEAR(ClosestPoint, Pt, 0.001);
		EXPECT_VECTOR_NEAR(ClosestBary, TVec4<double>(.3333, .3333, .3333, 0), 0.001);

		// Bounding Volume Hierarchy
		// Put the center of the tet at the origin, and sweep point tests across it.
		Pt = Tet.GetCenter();
		for (int i = 0; i < 4; i++)
		{
			Tet[i] -= Pt;
		}
		TArray<TTetrahedron<double>*> Tetrahedra;
		Tetrahedra.Add(&Tet);
		TBoundingVolumeHierarchy<TArray<TTetrahedron<Chaos::FReal>*>, TArray<int32>, Chaos::FReal, 3> BVH(Tetrahedra);
		for (int32 i = -5; i < 5; i++)
		{
			TArray<int32> Intersections = BVH.FindAllIntersections(Chaos::TVec3<double>(i, i, i));
			if (i == 0)
			{
				EXPECT_TRUE(Intersections.Num() == 1);
			}
			else
			{
				EXPECT_TRUE(Intersections.Num() == 0);
			}
		}
	}

	void ImplicitCube()
	{
		FString Caller("ImplicitCube()");

		TBox<FReal, 3> Subject(FVec3(-1), FVec3(1));

		UnitImplicitObjectNormalsInternal(Subject, Caller);
		UnitImplicitObjectNormalsExternal(Subject, Caller);
		UnitImplicitObjectIntersections(Subject, Caller);
		
		{// support phi - expects the corners for boxes
			// Iterate through every face, edge, and corner direction, and ensure it snaps to the proper corner.
			int32 VertexIndex = INDEX_NONE;
			for (int i0 = -1; i0 < 2; ++i0)
			{
				for (int i1 = -1; i1 < 2; ++i1)
				{
					for (int i2 = -1; i2 < 2; ++i2)
					{
						// If the direction is 0 or 1, it should snap to the upper corner. 
						FVec3 Expected(1);
						// If the direction is -1, it should snap to the lower corner. 
						if (i0 == -1) Expected[0] = -1;
						if (i1 == -1) Expected[1] = -1;
						if (i2 == -1) Expected[2] = -1;

						FString Error("Direction: ");
						Error += FString::Printf(TEXT("(%d, %d, %d)"), i0, i1, i2);

						EXPECT_VECTOR_NEAR_ERR(Subject.Support(FVec3(i0, i1, i2), FReal(0), VertexIndex), Expected, KINDA_SMALL_NUMBER, Error);
					}
				}
			}

#if RUN_KNOWN_BROKEN_TESTS
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.Support(FVec3(0, 0, 1), FReal(1)), (FVec3(2, 2, 2)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.Support(FVec3(0, 0, -1), FReal(1)), (FVec3(2, 2, -2)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.Support(FVec3(0, 1, 0), FReal(1)), (FVec3(2, 2, 2)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.Support(FVec3(0, -1, 0), FReal(1)), (FVec3(2, -2, 2)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.Support(FVec3(1, 0, 0), FReal(1)), (FVec3(2, 2, 2)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.Support(FVec3(-1, 0, 0), FReal(1)), (FVec3(-2, 2, 2)));
#endif
		}

		{// support phi off origin
			TBox<FReal, 3> Subject2(FVec3(2), FVec3(4));
			int32 VertexIndex = INDEX_NONE;
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(0, 0, 1), FReal(0), VertexIndex), (FVec3(4, 4, 4)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(0, 0, -1), FReal(0), VertexIndex), (FVec3(4, 4, 2)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(0, 1, 0), FReal(0), VertexIndex), (FVec3(4, 4, 4)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(0, -1, 0), FReal(0), VertexIndex), (FVec3(4, 2, 4)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(0, 1, 0), FReal(0), VertexIndex), (FVec3(4, 4, 4)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(1, 0, 0), FReal(0), VertexIndex), (FVec3(4, 4, 4)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(-1, 0, 0), FReal(0), VertexIndex), (FVec3(2, 4, 4)));

#if RUN_KNOWN_BROKEN_TESTS
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(0, 0, 1), FReal(1), VertexIndex), (FVec3(5, 5, 5)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(0, 0, -1), FReal(1), VertexIndex), (FVec3(5, 5, 1)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(0, 1, 0), FReal(1), VertexIndex), (FVec3(5, 5, 5)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(0, -1, 0), FReal(1), VertexIndex), (FVec3(5, 1, 5)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(1, 0, 0), FReal(1), VertexIndex), (FVec3(5, 5, 5)));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject2.Support(FVec3(-1, 0, 0), FReal(1), VertexIndex), (FVec3(1, 5, 5)));
#endif
		}

		// intersection
		EXPECT_TRUE(Subject.BoundingBox().Intersects(FAABB3(FVec3(0.5), FVec3(1.5))));
		EXPECT_FALSE(Subject.BoundingBox().Intersects(FAABB3(FVec3(2), FVec3(3))));

		{// closest point near origin (+)
			FVec3 InputPoint(0, 0, 2);
			EXPECT_VECTOR_NEAR(Subject.FindClosestPoint(InputPoint), FVector(0,0,1), 0.001);
			EXPECT_VECTOR_NEAR(Subject.FindClosestPoint(FVec3(3 / 2., 0, 0)), FVector(1,0,0), 0.001);
		}

		{// closest point near origin (-)
			FVec3 InputPoint(0, 0, 1 / 2.);
			EXPECT_VECTOR_NEAR(Subject.FindClosestPoint(InputPoint), FVector(0, 0, 1), 0.001);
			EXPECT_VECTOR_NEAR(Subject.FindClosestPoint(FVec3(3 / 4., 0, 0)), FVector(1, 0, 0), 0.001);
			EXPECT_FALSE(Subject.FindClosestPoint(FVec3(0, 0, 0)).Equals(FVec3(0)));
			EXPECT_EQ(Subject.FindClosestPoint(FVec3(0, 0, 0)).Size(),1.0);
		}

		{// diagonal 3-corner case
			FAABB3 Subject2(FVec3(-1), FVec3(1));
			// outside
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(FVec3(2, 2, 2)), FVector(1,1,1), 0.001);
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(FVec3(-2, -2, -2)), FVector(-1, -1, -1), 0.001);
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(FVec3(3 / 2., 3 / 2., 3 / 2.)), FVector(1, 1, 1), 0.001);
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(FVec3(-3 / 2., 3 / 2., -3 / 2.)), FVector(-1, 1, -1), 0.001);
			// inside
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(FVec3(1 / 2., 1 / 2., 1 / 2.)), FVector(1, 1, 1), 0.001);
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(FVec3(1 / 2., -1 / 2., 1 / 2.)), FVector(1, -1, 1), 0.001);
		}

		{// diagonal 2-corner case
			FAABB3 Subject2(FVec3(-1), FVec3(1));
			FVec3 test1 = Subject.FindClosestPoint(FVec3(2, 2, 0));
			// outside
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(FVec3(2, 2, 0)), FVector(1, 1, 0), 0.001);
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(FVec3(0, 3 / 2., 3 / 2.)), FVector(0, 1, 1), 0.001);
			// inside
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(FVec3(1 / 2., 1 / 2., 0)), FVector(1, 1, 0), 0.001);
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(FVec3(-1 / 2., 1 / 2., 0)), FVector(-1, 1, 0), 0.001);
		}

		{// closest point off origin (+)
			TBox<FReal, 3> Subject2(FVec3(2), FVec3(4));
			FVec3 InputPoint(5, 5, 5);
			TestFindClosestIntersection(Subject2, InputPoint, FVec3(4, 4, 4), Caller);

			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(InputPoint), FVector(4,4,4), 0.001);
			FVec3 test2 = Subject2.FindClosestPoint(FVec3(3.5, 3.5, 3.5));
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(FVec3(3.5,3.5,3.5)), FVector(4,4,4), 0.001);
		}

#if RUN_KNOWN_BROKEN_TESTS
		{// different defining corners of the box
			// Ensure fails in PhiWithNormal
			TBox<FReal, 3> Test1(FVec3(-1, -1, 0), FVec3(1, 1, -1));
			EXPECT_VECTOR_NEAR(Test1.Normal(FVec3(0, 0, -2 / 3.)), (FVec3(0, 0, -1)), KINDA_SMALL_NUMBER);

			// Ensure fails in PhiWithNormal
			TBox<FReal, 3> Test2(FVec3(1, 1, -1), FVec3(-1, -1, 0));
			EXPECT_VECTOR_NEAR(Test2.Normal(FVec3(0, 0, -2 / 3.)), (FVec3(0, 0, -1)), KINDA_SMALL_NUMBER);

			// Ensure fails in PhiWithNormal
			TBox<FReal, 3> Test3(FVec3(1, 1, 0), FVec3(-1, -1, -1));
			EXPECT_VECTOR_NEAR(Test3.Normal(FVec3(0, 0, -2 / 3.)), (FVec3(0, 0, -1)), KINDA_SMALL_NUMBER);

			// Works fine!
			TBox<FReal, 3> Test4(FVec3(-1, -1, -1), FVec3(1, 1, 0));
			EXPECT_VECTOR_NEAR(Test4.Normal(FVec3(0, 0, -2 / 3.)), (FVec3(0, 0, -1)), KINDA_SMALL_NUMBER);
		}
#endif
	}
	

	void ImplicitSphere()
	{
		
		FString Caller("ImplicitSphere()");

		Chaos::FSphere Subject(FVec3(0), 1);
		UnitImplicitObjectNormalsInternal(Subject, Caller);
		UnitImplicitObjectNormalsExternal(Subject, Caller);
		UnitImplicitObjectIntersections(Subject, Caller);
		UnitImplicitObjectSupportPhis<TSphere<FReal,3>>(Subject, Caller);

		// intersection
		EXPECT_TRUE(Subject.Intersects(Chaos::FSphere(FVec3(0.f), 2.f)));
		EXPECT_TRUE(Subject.Intersects(Chaos::FSphere(FVec3(.5f), 1.f)));
		EXPECT_FALSE(Subject.Intersects(Chaos::FSphere(FVec3(2.f), 1.f)));

		{// closest point near origin (+)
			FVec3 InputPoint(0, 0, 2.);
			EXPECT_VECTOR_NEAR(Subject.FindClosestPoint(InputPoint), FVector(0, 0, 1), 0.001);
			EXPECT_VECTOR_NEAR(Subject.FindClosestPoint(FVec3(3 / 2., 0, 0)), FVector(1, 0, 0), 0.001);
		}

		{// closest point near origin (-)
			FVec3 InputPoint(0, 0, 1 / 2.);
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.FindClosestPoint(FVec3(0, 0, 0)), FVec3(0));
			EXPECT_VECTOR_NEAR(Subject.FindClosestPoint(InputPoint), FVector(0, 0, 1), 0.001);
			EXPECT_VECTOR_NEAR(Subject.FindClosestPoint(FVec3(3 / 4., 0, 0)), FVector(1, 0, 0), 0.001);
		}

		{// closest point off origin (+)
			Chaos::FSphere Subject2(FVec3(2), 2);
			FVec3 InputPoint(2, 2, 5);
			TestFindClosestIntersection(Subject2, InputPoint, FVec3(2, 2, 4), Caller);
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(InputPoint), FVector(2, 2, 4), 0.001);
			EXPECT_VECTOR_NEAR(Subject2.FindClosestPoint(FVec3(2, 2, 3.5)), FVector(2, 2, 4), 0.001);
		}

		// Intersects
		{
			// Overlap
			{
				Chaos::FSphere Sphere0(FVec3(0), 2);
				Chaos::FSphere Sphere1(FVec3(4, 0, 0), 3);
				EXPECT_TRUE(Sphere0.Intersects(Sphere1));
				EXPECT_TRUE(Sphere1.Intersects(Sphere0));
			}
			// Tangent
			{
				Chaos::FSphere Sphere0(FVec3(0), 2);
				Chaos::FSphere Sphere1(FVec3(5, 0, 0), 3);
				EXPECT_TRUE(Sphere0.Intersects(Sphere1));
				EXPECT_TRUE(Sphere1.Intersects(Sphere0));
			}
			// Separating
			{
				Chaos::FSphere Sphere0(FVec3(0), 2);
				Chaos::FSphere Sphere1(FVec3(6, 0, 0), 3);
				EXPECT_FALSE(Sphere0.Intersects(Sphere1));
				EXPECT_FALSE(Sphere1.Intersects(Sphere0));
			}
		}
	}
	
	/* Cylinder Helpers */

	// Expects a unit cylinder. 
	void CheckCylinderEdgeBehavior(FImplicitObject &Subject, FString Caller)
	{
		FString Error = FString("Called by ") + Caller + FString(".");

		// inside normal
		// defaults to side of cylinder when equally close to side and endcap
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 1 / 2., 1 / 2.)), FVec3(0, 1, 0), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 1 / 3., 1 / 2.)), FVec3(0, 0, 1), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 1 / 2., -1 / 2.)), FVec3(0, 1, 0), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 1 / 3., -1 / 2.)), FVec3(0, 0, -1), KINDA_SMALL_NUMBER, Error);

		// outside normal 		
		// defaults to endcap of cylinder above intersection of side and endcap
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 1., 3 / 2.)), FVec3(0, 0, 1), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 1., -3 / 2.)), FVec3(0, 0, -1), KINDA_SMALL_NUMBER, Error);
		// defaults to side of cylinder next to intersection of side and endcap
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 3 / 2., 1.)), FVec3(0, 1, 0), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0, 3 / 2., -1.)), FVec3(0, 1, 0), KINDA_SMALL_NUMBER, Error);

		//inside phi
		EXPECT_EQ(Subject.SignedDistance(FVec3(0, 1, 3 / 2.)), 1 / 2.) << *Error;
		EXPECT_EQ(Subject.SignedDistance(FVec3(0, 1, -3 / 2.)), 1 / 2.) << *Error;
		EXPECT_EQ(Subject.SignedDistance(FVec3(0, -1, 3 / 2.)), 1 / 2.) << *Error;
		EXPECT_EQ(Subject.SignedDistance(FVec3(0, -1, -3 / 2.)), 1 / 2.) << *Error;
	}


	// Expects a cylinder with endcap points (1,1,1) and (-1,-1,-1), radius 1.
	void TiltedUnitImplicitCylinder(FImplicitObject &Subject, FString Caller)
	{
		FString Error = FString("Called by ") + Caller + FString(".");

		// inside normals
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(1 / 2., 1 / 2., 1 / 2.)), FVec3(1, 1, 1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(-1 / 2., -1 / 2., -1 / 2.)), FVec3(-1, -1, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);

		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0., 1 / 2., -1 / 2.)), FVec3(0, 1, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0., -1 / 2., 1 / 2.)), FVec3(0, -1, 1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(1 / 2., 0., -1 / 2.)), FVec3(1, 0, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(-1 / 2., 0., 1 / 2.)), FVec3(-1, 0, 1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);

		//outside normals
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(3 / 2., 3 / 2., 3 / 2.)), FVec3(1, 1, 1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(-3 / 2., -3 / 2., -3 / 2.)), FVec3(-1, -1, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);

		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0., 3 / 2., -3 / 2.)), FVec3(0, 1, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0., -3 / 2., 3 / 2.)), FVec3(0, -1, 1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(3 / 2., 0., -3 / 2.)), FVec3(1, 0, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(-3 / 2., 0., 3 / 2.)), FVec3(-1, 0, 1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);

		// inside phi
		EXPECT_NEAR(Subject.SignedDistance(FVec3(1 / 2., 1 / 2., 1 / 2.)), -FVec3(1 / 2.).Size(), KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-1 / 2., -1 / 2., -1 / 2.)), -FVec3(1 / 2.).Size(), KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0., sqrt(2) / 4., -sqrt(2) / 4.)), -1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0., -sqrt(2) / 4., sqrt(2) / 4.)), -1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(sqrt(2) / 4., 0., -sqrt(2) / 4.)), -1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-sqrt(2) / 4., 0., sqrt(2) / 4.)), -1 / 2., KINDA_SMALL_NUMBER) << *Error;

		// outside phi
		EXPECT_NEAR(Subject.SignedDistance(FVec3(3 / 2., 3 / 2., 3 / 2.)), FVec3(1 / 2.).Size(), KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-3 / 2., -3 / 2., -3 / 2.)), FVec3(1 / 2.).Size(), KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0., 3 * sqrt(2) / 4., -3 * sqrt(2) / 4.)), 1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0., -3 * sqrt(2) / 4., 3 * sqrt(2) / 4.)), 1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(3 * sqrt(2) / 4., 0., -3 * sqrt(2) / 4.)), 1 / 2., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-3 * sqrt(2) / 4., 0., 3 * sqrt(2) / 4.)), 1 / 2., KINDA_SMALL_NUMBER) << *Error;
	}

	/* End Cylinder Helpers */

	void ImplicitCylinder()
	{
		FString Caller("ImplicitCylinder()");

		// unit cylinder tests
		FCylinder Subject(FVec3(0, 0, 1), FVec3(0, 0, -1), 1);
		UnitImplicitObjectNormalsInternal(Subject, Caller);
		UnitImplicitObjectNormalsExternal(Subject, Caller);
		UnitImplicitObjectIntersections(Subject, Caller);
		CheckCylinderEdgeBehavior(Subject, Caller);

		// tilted tests
		FCylinder SubjectTilted(FVec3(1), FVec3(-1), 1);
		TiltedUnitImplicitCylinder(SubjectTilted, Caller);

#if RUN_KNOWN_BROKEN_TESTS
		{// nearly flat cylinder tests (BROKEN)
			FCylinder SubjectFlat(FVec3(0, 0, KINDA_SMALL_NUMBER), FVec3(0, 0, -KINDA_SMALL_NUMBER), 1);
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.Normal(FVec3(0, 0, 1 / 2.)), FVec3(0, 0, 1));
			EXPECT_VECTOR_NEAR_DEFAULT(Subject.Normal(FVec3(0, 0, -1 / 2.)), FVec3(0, 0, -1));
			EXPECT_EQ(Subject.SignedDistance(FVec3(0, 0, 1 / 2.)), 1 / 2.);
			EXPECT_EQ(Subject.SignedDistance(FVec3(0, 0, -1 / 2.)), 1 / 2.);
			Pair<FVec3, bool> Result = SubjectFlat.FindClosestIntersection(FVec3(0, 1, 1), FVec3(0, -1, -1), KINDA_SMALL_NUMBER);
			EXPECT_FALSE(Result.Second);
		}
#endif

		{// closest point off origin (+)
			FCylinder Subject2(FVec3(2,2,4), FVec3(2,2,0), 2);
			FVec3 InputPoint(2, 2, 5);
			TestFindClosestIntersection(Subject2, InputPoint, FVec3(2, 2, 4), Caller);
		}

		{// closest point off origin (-)
			FCylinder Subject2(FVec3(2, 2, 4), FVec3(2, 2, 0), 2);
			FVec3 InputPoint(2, 3, 2);
			TestFindClosestIntersection(Subject2, InputPoint, FVec3(2, 4, 2), Caller);
		}

		{// near edge intersection
			FCylinder Cylinder(FVec3(1, 1, -14), FVec3(1, 1, 16), 15);
			Pair<FVec3, bool> Result = Cylinder.FindClosestIntersection(FVec3(16, 16, 1), FVec3(16, -16, 1), 0);
			EXPECT_TRUE(Result.Second);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(16, 1, 1), KINDA_SMALL_NUMBER);
		}

		{	// Inertia tensor and rotation of mass
			FCylinder AlignedSubject(FVec3(0, 0, 1), FVec3(0, 0, -1), 1);
			FCylinder OffsetedAlignedSubject(FVec3(5, 10, 1), FVec3(5, 10, -1), 1);
			FCylinder NonAlignedSubject(FVec3(-1, -1, -1).GetSafeNormal(), FVec3(1, 1, 1).GetSafeNormal(), 1);

			UnitImplicitObjectInertiaTensorAndRotationOfMass(AlignedSubject, OffsetedAlignedSubject, NonAlignedSubject, Caller);
		}
	}

	void ImplicitTaperedCylinder()
	{
		FString Caller("ImplicitTaperedCylinder()");

		// unit tapered cylinder tests
		FTaperedCylinder Subject(FVec3(0, 0, 1), FVec3(0, 0, -1), 1, 1);
		UnitImplicitObjectNormalsInternal(Subject, Caller);
		UnitImplicitObjectNormalsExternal(Subject, Caller);
		UnitImplicitObjectIntersections(Subject, Caller);
		CheckCylinderEdgeBehavior(Subject, Caller);

		// tilted tapered cylinder tests
		FTaperedCylinder SubjectTilted(FVec3(1), FVec3(-1), 1, 1);
		TiltedUnitImplicitCylinder(SubjectTilted, Caller);

		FTaperedCylinder SubjectCone(FVec3(0, 0, 1), FVec3(0, 0, 0), 0, 1);

		// inside normals 
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectCone.Normal(FVec3(0, 0, 0)), FVec3(0, 0, -1));
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectCone.Normal(FVec3(0, 0, 1)), FVec3(0, 0, 1));
		
		// Note: tapered cylinders always return normals parallel to the endcap planes when calculating for points near/on the body,
		// very much like a normal cylinder. The slant is ignored. 
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectCone.Normal(FVec3(0, 1 / 2., 1 / 2.)),  FVec3(0, 1, 0));
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectCone.Normal(FVec3(1 / 2., 0, 1 / 2.)),  FVec3(1, 0, 0)); 
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectCone.Normal(FVec3(0, -1 / 2., 1 / 2.)), FVec3(0, -1, 0)); 
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectCone.Normal(FVec3(-1 / 2., 0, 1 / 2.)), FVec3(-1, 0, 0)); 
		EXPECT_VECTOR_NEAR(SubjectCone.Normal(FVec3(1 / 2., 1 / 2., 1 / 2.)), FVec3(0.707, 0.707, 0), 0.001); 

		// outside normals
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectCone.Normal(FVec3(0, 0, -1 / 2.)), FVec3(0, 0, -1));
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectCone.Normal(FVec3(0, 0,  3 / 2.)), FVec3(0, 0, 1));
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectCone.Normal(FVec3( 0,  1, 1 / 2.)), FVec3(0, 1, 0));
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectCone.Normal(FVec3( 1,  0, 1 / 2.)), FVec3(1, 0, 0));
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectCone.Normal(FVec3( 0, -1, 1 / 2.)), FVec3(0, -1, 0));
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectCone.Normal(FVec3(-1,  0, 1 / 2.)), FVec3(-1, 0, 0));

		{// closest point off origin (+)
			FTaperedCylinder Subject2(FVec3(2, 2, 4), FVec3(2, 2, 0), 2, 2);
			FVec3 InputPoint(2, 2, 5);
			TestFindClosestIntersection(Subject2, InputPoint, FVec3(2, 2, 4), Caller);
		}

		{// closest point off origin (-)
			FTaperedCylinder Subject2(FVec3(2, 2, 4), FVec3(2, 2, 0), 2, 2);
			FVec3 InputPoint(2, 3, 2);
			TestFindClosestIntersection(Subject2, InputPoint, FVec3(2, 4, 2), Caller);
		}

		{	// Inertia tensor and rotation of mass
			FTaperedCylinder AlignedSubject(FVec3(0, 0, 1), FVec3(0, 0, -1), 1, 2);
			FTaperedCylinder OffsetedAlignedSubject(FVec3(5, 10, 1), FVec3(5, 10, -1), 1, 2);
			FTaperedCylinder NonAlignedSubject(FVec3(-1, -1, -1).GetSafeNormal(), FVec3(1, 1, 1).GetSafeNormal(), 1, 2);

			UnitImplicitObjectInertiaTensorAndRotationOfMass(AlignedSubject, OffsetedAlignedSubject, NonAlignedSubject, Caller);
		}
	}
	
	// Expects a cylinder with endcap points (1,1,1) and (-1,-1,-1), radius 1.
	void TiltedUnitImplicitCapsule(FImplicitObject& Subject, FString Caller)
	{
		FString Error = FString("Called by ") + Caller + FString(".");

		// inside normals - within the cylinder
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0.,   0.,  0.5)), FVec3(-0.5, -0.5,  1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0.,   0., -0.5)), FVec3( 0.5,  0.5, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0.,  0.5, -0.5)), FVec3( 0,  1, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0., -0.5,  0.5)), FVec3( 0, -1,  1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3( 0.5, 0., -0.5)), FVec3( 1,  0, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(-0.5, 0.,  0.5)), FVec3(-1,  0,  1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);

		// inside normals - within the spherical ends
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3( 1.1,  1.1,  1.1)), FVec3( 1,  1,  1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(-1.1, -1.1, -1.1)), FVec3(-1, -1, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3( 1.,  1.,  1.1)), FVec3(0, 0,  1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(-1., -1., -1.1)), FVec3(0, 0, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);

		// outside normals - close to the cylinder
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0., 0., 2.)), FVec3(-0.5, -0.5, 1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0., 0., -2.)), FVec3(0.5, 0.5, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0., 2., -2.)), FVec3(0, 1, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(0., -2., 2.)), FVec3(0, -1, 1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(2., 0., -2.)), FVec3(1, 0, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(-2., 0., 2.)), FVec3(-1, 0, 1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);

		//outside normals - close to spherical ends
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3( 2.,  2.,  2.)), FVec3( 1,  1,  1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(-2., -2., -2.)), FVec3(-1, -1, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3( 1.,  1.,  3.)), FVec3( 0,  0,  1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);
		EXPECT_VECTOR_NEAR_ERR(Subject.Normal(FVec3(-1., -1., -3.)), FVec3( 0,  0, -1).GetSafeNormal(), KINDA_SMALL_NUMBER, Error);

		// inside phi - within the cylinder
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-0.5, -0.5,  1).GetSafeNormal() * 0.5), -0.5, KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3( 0.5,  0.5, -1).GetSafeNormal() * 0.5), -0.5, KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0, 1, -1).GetSafeNormal() * 0.5), -0.5, KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0, -1, 1).GetSafeNormal() * 0.5), -0.5, KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(1, 0, -1).GetSafeNormal() * 0.5), -0.5, KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-1, 0, 1).GetSafeNormal() * 0.5), -0.5, KINDA_SMALL_NUMBER) << *Error;

		//// inside phi - within the spherical ends
		EXPECT_NEAR(Subject.SignedDistance(FVec3( 1.1,  1.1,  1.1)), -(1. - FVec3(0.1).Size()), KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-1.1, -1.1, -1.1)), -(1. - FVec3(0.1).Size()), KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3( 1.,  1.,  1.1)), -0.9, KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-1., -1., -1.1)), -0.9, KINDA_SMALL_NUMBER) << *Error;

		//// outside phi - close to the cylinder
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-0.5, -0.5, 1).GetSafeNormal() * 2.0), 1.0, KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0.5, 0.5, -1).GetSafeNormal() * 2.0), 1.0, KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0, 1, -1).GetSafeNormal() * 2.0), 1.0, KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(0, -1, 1).GetSafeNormal() * 2.0), 1.0, KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(1, 0, -1).GetSafeNormal() * 2.0), 1.0, KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-1, 0, 1).GetSafeNormal() * 2.0), 1.0, KINDA_SMALL_NUMBER) << *Error;

		//outside phi - close to spherical ends
		EXPECT_NEAR(Subject.SignedDistance(FVec3( 2.,  2.,  2.)), (FVec3(1).Size() - 1), KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-2., -2., -2.)), (FVec3(1).Size() - 1), KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3( 1.,  1.,  3.)), 1., KINDA_SMALL_NUMBER) << *Error;
		EXPECT_NEAR(Subject.SignedDistance(FVec3(-1., -1., -3.)), 1., KINDA_SMALL_NUMBER) << *Error;
	}

	FReal LerpRadius(FReal Height0, FReal Height1, FReal Radius0, FReal Radius1, FReal ZPos)
	{
		FReal Alpha = (ZPos - Height0) / (Height1 - Height0);
		return Radius0 * (1. - Alpha) + Radius1 * Alpha;
	}

	void ImplicitTaperedCapsule()
	{
		FString Caller("ImplicitTaperedCapsule()");

		// unit tapered cylinder tests
		FTaperedCapsule SubjectUnit(FVec3(0, 0, 0), FVec3(0, 0, 0), 1, 1);
		UnitImplicitObjectNormalsInternal(SubjectUnit, Caller);
		UnitImplicitObjectNormalsExternal(SubjectUnit, Caller);
		UnitImplicitObjectIntersections(SubjectUnit, Caller);

		// tilted tapered cylinder tests
		FTaperedCapsule SubjectTilted(FVec3(1), FVec3(-1), 1, 1);
		TiltedUnitImplicitCapsule(SubjectTilted, Caller);

		const FReal Height0 = 0.5;
		const FReal Height1 = 2.0;
		const FReal Radius0 = 0.5;
		const FReal Radius1 = 1.0;
		FTaperedCapsule SubjectTapered(FVec3(0, 0, Height0), FVec3(0, 0, Height1), Radius0, Radius1);

		// inside normals 
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectTapered.Normal(FVec3(0, 0, 0.25)), FVec3(0, 0, -1));
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectTapered.Normal(FVec3(0, 0, 2.5)), FVec3(0, 0, 1));

		// tapered section part inside normals - normals are currently perpendicular axis regardless of the slant
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectTapered.Normal(FVec3( 0.25,  0.25, 0.5)), FVec3( 1,  1, 0).GetSafeNormal());
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectTapered.Normal(FVec3( 0.25, -0.25, 1.0)), FVec3( 1, -1, 0).GetSafeNormal());
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectTapered.Normal(FVec3(-0.25, -0.25, 1.5)), FVec3(-1, -1, 0).GetSafeNormal());
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectTapered.Normal(FVec3(-0.25,  0.25, 2.0)), FVec3(-1,  1, 0).GetSafeNormal());

		// tapered section part ouside normals - normals are currently perpendicular axis regardless of the slant
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectTapered.Normal(FVec3( 1,  1, 0.5)), FVec3( 1,  1, 0).GetSafeNormal());
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectTapered.Normal(FVec3( 1, -1, 1.0)), FVec3( 1, -1, 0).GetSafeNormal());
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectTapered.Normal(FVec3(-1, -1, 1.5)), FVec3(-1, -1, 0).GetSafeNormal());
		EXPECT_VECTOR_NEAR_DEFAULT(SubjectTapered.Normal(FVec3(-1,  1, 2.0)), FVec3(-1,  1, 0).GetSafeNormal());

		// tapered section part inside phi - slant is accounted for 
		EXPECT_NEAR(SubjectTapered.SignedDistance(FVec3( 0.25,  0.25, 0.5)), FVec3( 0.25,  0.25, 0).Size() - LerpRadius(Height0, Height1, Radius0, Radius1, 0.5), KINDA_SMALL_NUMBER) << *Caller;
		EXPECT_NEAR(SubjectTapered.SignedDistance(FVec3( 0.25, -0.25, 1.0)), FVec3( 0.25, -0.25, 0).Size() - LerpRadius(Height0, Height1, Radius0, Radius1, 1.0), KINDA_SMALL_NUMBER) << *Caller;
		EXPECT_NEAR(SubjectTapered.SignedDistance(FVec3(-0.25, -0.25, 1.5)), FVec3(-0.25, -0.25, 0).Size() - LerpRadius(Height0, Height1, Radius0, Radius1, 1.5), KINDA_SMALL_NUMBER) << *Caller;
		EXPECT_NEAR(SubjectTapered.SignedDistance(FVec3(-0.25,  0.25, 2.0)), FVec3(-0.25,  0.25, 0).Size() - LerpRadius(Height0, Height1, Radius0, Radius1, 2.0), KINDA_SMALL_NUMBER) << *Caller;

		// tapered section part outside phi - slant is accounted for 
		EXPECT_NEAR(SubjectTapered.SignedDistance(FVec3( 1,  1, 0.5)), FVec3( 1,  1, 0).Size() - LerpRadius(Height0, Height1, Radius0, Radius1, 0.5), KINDA_SMALL_NUMBER) << *Caller;
		EXPECT_NEAR(SubjectTapered.SignedDistance(FVec3( 1, -1, 1.0)), FVec3( 1, -1, 0).Size() - LerpRadius(Height0, Height1, Radius0, Radius1, 1.0), KINDA_SMALL_NUMBER) << *Caller;
		EXPECT_NEAR(SubjectTapered.SignedDistance(FVec3(-1, -1, 1.5)), FVec3(-1, -1, 0).Size() - LerpRadius(Height0, Height1, Radius0, Radius1, 1.5), KINDA_SMALL_NUMBER) << *Caller;
		EXPECT_NEAR(SubjectTapered.SignedDistance(FVec3(-1,  1, 2.0)), FVec3(-1,  1, 0).Size() - LerpRadius(Height0, Height1, Radius0, Radius1, 2.0), KINDA_SMALL_NUMBER) << *Caller;


		{// closest point off origin (+)
			FTaperedCapsule Subject2(FVec3(2, 2, 4), FVec3(2, 2, 0), 2, 2);
			FVec3 InputPoint(2, 2, 5);
			TestFindClosestIntersection(Subject2, InputPoint, FVec3(2, 2, 6), Caller);
		}

		{// closest point off origin (-)
			FTaperedCapsule Subject2(FVec3(2, 2, 4), FVec3(2, 2, 0), 2, 2);
			FVec3 InputPoint(2, 3, 2);
			TestFindClosestIntersection(Subject2, InputPoint, FVec3(2, 4, 2), Caller);
		}

		{	// Inertia tensor and rotation of mass
			FTaperedCapsule AlignedSubject(FVec3(0, 0, 1), FVec3(0, 0, -1), 1, 2);
			FTaperedCapsule OffsetedAlignedSubject(FVec3(5, 10, 1), FVec3(5, 10, -1), 1, 2);
			FTaperedCapsule NonAlignedSubject(FVec3(-1, -1, -1).GetSafeNormal(), FVec3(1, 1, 1).GetSafeNormal(), 1, 2);

			UnitImplicitObjectInertiaTensorAndRotationOfMass(AlignedSubject, OffsetedAlignedSubject, NonAlignedSubject, Caller);
		}
	}


	void ImplicitCapsule()
	{
		
		FString Caller("ImplicitCapsule()");

		// Effectively a sphere - flat cylinder with two radius 1 spheres overlapping at origin.
		FCapsule SubjectUnit(FVec3(0, 0, 0), FVec3(0, 0, 0), 1);

		UnitImplicitObjectNormalsInternal(SubjectUnit, Caller);
		UnitImplicitObjectNormalsExternal(SubjectUnit, Caller);
		UnitImplicitObjectSupportPhis<FCapsule>(SubjectUnit, Caller);

		FCapsule SubjectTilted(FVec3(1), FVec3(-1), 1);
		TiltedUnitImplicitCapsule(SubjectTilted, Caller);

#if RUN_KNOWN_BROKEN_TESTS
		// FindClosestIntersection broken with cylinder size 0
		UnitImplicitObjectIntersections(SubjectUnit, Caller); 
#endif

		FCapsule Subject(FVec3(0, 0, 1), FVec3(0, 0, -1), 1);

		{// closest point near origin (+)
			FVec3 InputPoint(0, 0, 3);
			TestFindClosestIntersection(Subject, InputPoint, FVec3(0, 0, 2), Caller);
		}
		
		{// closest point near origin (-)
			FVec3 InputPoint(0, 0, 3 / 2.);
			// Equally close to inner cylinder and top sphere - defaults to sphere. 
			TestFindClosestIntersection(Subject, InputPoint, FVec3(0, 0, 2), Caller);
		}

		{// closest point off origin (+)
			FCapsule Subject2(FVec3(5, 4, 4), FVec3(3, 4, 4), 1);
			FVec3 InputPoint(4, 4, 6);
			TestFindClosestIntersection(Subject2, InputPoint, FVec3(4, 4, 5), Caller);
		}

		{// closest point off origin (-)
			FCapsule Subject2(FVec3(5, 4, 4), FVec3(3, 4, 4), 1);
			FVec3 InputPoint(4, 4, 4 + 1 / 2.);
			TestFindClosestIntersection(Subject2, InputPoint, FVec3(4, 4, 5), Caller);
		}

		{	// Inertia tensor and rotation of mass
			FCapsule AlignedSubject(FVec3(0, 0, 1), FVec3(0, 0, -1), 1);
			FCapsule OffsetedAlignedSubject(FVec3(5, 10, 1), FVec3(5, 10, -1), 1);
			FCapsule NonAlignedSubject(FVec3(-1, -1, -1).GetSafeNormal(), FVec3(1, 1, 1).GetSafeNormal(), 1.0);

			UnitImplicitObjectInertiaTensorAndRotationOfMass(AlignedSubject, OffsetedAlignedSubject, NonAlignedSubject, Caller);
		}
	}

	
	void ImplicitScaled()
	{
		FBoxPtr UnitCube( new TBox<FReal, 3>(FVec3(-1), FVec3(1)));
		TImplicitObjectScaled<TBox<FReal,3>> UnitUnscaled(UnitCube, FVec3(1));
		UnitImplicitObjectNormalsInternal(UnitUnscaled, FString("ImplicitTransformed()"));
		UnitImplicitObjectNormalsExternal(UnitUnscaled, FString("ImplicitTransformed()"));
		UnitImplicitObjectIntersections(UnitUnscaled, FString("ImplicitTransformed()"));

		FSpherePtr Sphere( new Chaos::FSphere(FVec3(3, 0, 0), 5));
		TImplicitObjectScaled<Chaos::FSphere> Unscaled(Sphere, FVec3(1));
		TImplicitObjectScaled<Chaos::FSphere> UniformScale(Sphere, FVec3(2));
		TImplicitObjectScaled<Chaos::FSphere> NonUniformScale(Sphere, FVec3(2, 1, 1));

		{//phi
			const FVec3 NearEdge(7.5, 0, 0);
			FVec3 UnscaledNormal;
			const FReal UnscaledPhi = Unscaled.PhiWithNormal(NearEdge, UnscaledNormal);
			EXPECT_FLOAT_EQ(UnscaledPhi, -0.5);
			EXPECT_VECTOR_NEAR(UnscaledNormal, FVec3(1, 0, 0), 0);

			FVec3 ScaledNormal;
			FReal ScaledPhi = UniformScale.PhiWithNormal(NearEdge, ScaledNormal);
			EXPECT_FLOAT_EQ(ScaledPhi, -(16 - 7.5));
			EXPECT_VECTOR_NEAR(ScaledNormal, FVec3(1, 0, 0), 0);

			const FVec3 NearTop(6, 0, 4.5);
			ScaledPhi = UniformScale.PhiWithNormal(NearTop, ScaledNormal);
			EXPECT_FLOAT_EQ(ScaledPhi, -(10 - 4.5));
			EXPECT_VECTOR_NEAR(ScaledNormal, FVec3(0, 0, 1), 0);

			ScaledPhi = NonUniformScale.PhiWithNormal(NearTop, ScaledNormal);
			EXPECT_FLOAT_EQ(ScaledPhi, -0.5);
			EXPECT_VECTOR_NEAR(ScaledNormal, FVec3(0, 0, 1), 0);
		}
		
		{//support
			int32 VertexIndex = INDEX_NONE;
			
			const FVec3 DirX(1, 0, 0);
			FVec3 SupportPt = Unscaled.Support(DirX, 1, VertexIndex);
			EXPECT_VECTOR_NEAR(SupportPt, FVec3(9, 0, 0), 0);

			SupportPt = UniformScale.Support(DirX, 1, VertexIndex);
			EXPECT_VECTOR_NEAR(SupportPt, FVec3(17, 0, 0), 0);

			const FVec3 DirZ(0, 0, -1);
			SupportPt = UniformScale.Support(DirZ, 1, VertexIndex);
			EXPECT_VECTOR_NEAR(SupportPt, FVec3(6, 0, -11), 0);

			SupportPt = NonUniformScale.Support(DirX, 1, VertexIndex);
			EXPECT_VECTOR_NEAR(SupportPt, FVec3(17, 0, 0), 0);

			SupportPt = NonUniformScale.Support(DirZ, 1, VertexIndex);
			EXPECT_VECTOR_NEAR(SupportPt, FVec3(6, 0, -6), 0);
		}

		{// closest intersection
			Pair<FVec3, bool> Result;
			Result = Unscaled.FindClosestIntersection(FVec3(7.5, 0, 0), FVec3(8.5, 0, 0), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(8, 0, 0), 0.001);

			Result = UniformScale.FindClosestIntersection(FVec3(15.5, 0, 0), FVec3(16.5, 0, 0), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(16, 0, 0), 0.001);

			Result = NonUniformScale.FindClosestIntersection(FVec3(6, 0, 4.5), FVec3(6, 0, 5.5), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(6, 0, 5), 0.001);
		}
	}

	TEST(ImplicitTests, TestImplicitConvex_PhiWithNormal_Penetrating)
	{
		const FVec3 Size = FVec3(500, 500, 100);
		FImplicitConvex3 Convex = CreateConvexBox(Size, 10);
		{
			// Near point just inside the top face, near the forward edge
			const FVec3 Point = FVec3(0.5f * Size.X, 0.0f, 0.5f * Size.Z) - FVec3(10, 0, 1);
			FVec3 Normal;
			FReal Phi = Convex.PhiWithNormal(Point, Normal);
			EXPECT_NEAR(Normal.Z, 1.0f, 1.e-4f);
			EXPECT_NEAR(Phi, -1.0f, 1.e-4f);
		}
		{
			// Near point just inside the top face, near the forward edge
			const FVec3 Point = FVec3(0.5f * Size.X, 0.0f, 0.5f * Size.Z) - FVec3(3, 0, 1);
			FVec3 Normal;
			FReal Phi = Convex.PhiWithNormal(Point, Normal);
			EXPECT_NEAR(Normal.Z, 1.0f, 1.e-4f);
			EXPECT_NEAR(Phi, -1.0f, 1.e-4f);
		}
		{
			// Near point just inside the top face, near the forward edge
			const FVec3 Point = FVec3(0.5f * Size.X, 0.0f, 0.5f * Size.Z) - FVec3(1, 0, 0.1);
			FVec3 Normal;
			FReal Phi = Convex.PhiWithNormal(Point, Normal);
			EXPECT_NEAR(Normal.Z, 1.0f, 1.e-4f);
			EXPECT_NEAR(Phi, -0.1f, 1.e-4f);
		}
	}

	TEST(ImplicitTests, TestImplicitConvex_PhiWithNormal_Separated)
	{
		const FVec3 Size = FVec3(500, 500, 100);
		FImplicitConvex3 Convex = CreateConvexBox(Size, 10);
		{
			const FVec3 Offset = FVec3(10, 0, 1);
			const FVec3 Point = FVec3(0.5f * Size.X, 0.5f * Size.Y, 0.5f * Size.Z) + Offset;
			FVec3 Normal;
			FReal Phi = Convex.PhiWithNormal(Point, Normal);
			const FVec3 ExpectedNormal = Offset.GetUnsafeNormal();
			const FReal ExpectedPhi = Offset.Size();
			EXPECT_NEAR(Normal.X, ExpectedNormal.X, 1.e-4f);
			EXPECT_NEAR(Normal.Y, ExpectedNormal.Y, 1.e-4f);
			EXPECT_NEAR(Normal.Z, ExpectedNormal.Z, 1.e-4f);
			EXPECT_NEAR(Phi, ExpectedPhi, 1.e-4f);
		}
		{
			const FVec3 Offset = FVec3(3, 2, 1);
			const FVec3 Point = FVec3(0.5f * Size.X, 0.5f * Size.Y, 0.5f * Size.Z) + Offset;
			FVec3 Normal;
			FReal Phi = Convex.PhiWithNormal(Point, Normal);
			const FVec3 ExpectedNormal = Offset.GetUnsafeNormal();
			const FReal ExpectedPhi = Offset.Size();
			EXPECT_NEAR(Normal.X, ExpectedNormal.X, 1.e-4f);
			EXPECT_NEAR(Normal.Y, ExpectedNormal.Y, 1.e-4f);
			EXPECT_NEAR(Normal.Z, ExpectedNormal.Z, 1.e-4f);
			EXPECT_NEAR(Phi, ExpectedPhi, 1.e-4f);
		}
		{
			const FVec3 Offset = FVec3(0, 1, 1);
			const FVec3 Point = FVec3(0.5f * Size.X, 0.5f * Size.Y, 0.5f * Size.Z) + Offset;
			FVec3 Normal;
			FReal Phi = Convex.PhiWithNormal(Point, Normal);
			const FVec3 ExpectedNormal = Offset.GetUnsafeNormal();
			const FReal ExpectedPhi = Offset.Size();
			EXPECT_NEAR(Normal.X, ExpectedNormal.X, 1.e-4f);
			EXPECT_NEAR(Normal.Y, ExpectedNormal.Y, 1.e-4f);
			EXPECT_NEAR(Normal.Z, ExpectedNormal.Z, 1.e-4f);
			EXPECT_NEAR(Phi, ExpectedPhi, 1.e-4f);
		}
	}


	// Check that PhiWithNormal works properly on Scaled Convex.
	// There was a bug where scaled convexed would bias face selection based on the 
	// scale, so a unit box scaled by 5 in the X would report the +X face as the 
	// contact face for the position (0.4, 0.0, 4.8) even though the +Z face is closer.
	TEST(ImplicitTests, TestImplicitScaledConvex_PhiWithNormal_Penetrating)
	{
		const FVec3 Size = FVec3(500, 500, 100);
		const FVec3 Scale = FVec3(5, 5, 1);
		const FVec3 ScaledSize = Scale * Size;
		TImplicitObjectScaled<FImplicitConvex3> ScaledConvex = CreateScaledConvexBox(Size, Scale, 10);

		{
			// Near point just inside the top face, near the forward edge
			const FVec3 Point = FVec3(0.5f * ScaledSize.X, 0.0f, 0.5f * ScaledSize.Z) - FVec3(10, 0, 1);
			FVec3 Normal;
			FReal Phi = ScaledConvex.PhiWithNormal(Point, Normal);
			EXPECT_NEAR(Normal.Z, 1.0f, 1.e-4f);
			EXPECT_NEAR(Phi, -1.0f, 1.e-4f);
		}
		{
			// Near point just inside the top face, near the forward edge
			const FVec3 Point = FVec3(0.5f * ScaledSize.X, 0.0f, 0.5f * ScaledSize.Z) - FVec3(3, 0, 1);
			FVec3 Normal;
			FReal Phi = ScaledConvex.PhiWithNormal(Point, Normal);
			EXPECT_NEAR(Normal.Z, 1.0f, 1.e-4f);
			EXPECT_NEAR(Phi, -1.0f, 1.e-4f);
		}
		{
			// Near point just inside the top face, near the forward edge
			const FVec3 Point = FVec3(0.5f * ScaledSize.X, 0.0f, 0.5f * ScaledSize.Z) - FVec3(1, 0, 0.1);
			FVec3 Normal;
			FReal Phi = ScaledConvex.PhiWithNormal(Point, Normal);
			EXPECT_NEAR(Normal.Z, 1.0f, 1.e-4f);
			EXPECT_NEAR(Phi, -0.1f, 1.e-4f);
		}
	}

	TEST(ImplicitTests, TestImplicitScaledConvex_PhiWithNormal_Separated)
	{
		const FVec3 Size = FVec3(500, 500, 100);
		const FVec3 Scale = FVec3(5, 5, 1);
		const FVec3 ScaledSize = Scale * Size;
		TImplicitObjectScaled<FImplicitConvex3> ScaledConvex = CreateScaledConvexBox(Size, Scale, 10);

		{
			// Near point just inside the top face, near the forward edge
			const FVec3 Point = FVec3(0.5f * ScaledSize.X, 0.0f, 0.5f * ScaledSize.Z) + FVec3(-10, 0, 1);
			FVec3 Normal;
			FReal Phi = ScaledConvex.PhiWithNormal(Point, Normal);
			EXPECT_NEAR(Normal.Z, 1.0f, 1.e-4f);
			EXPECT_NEAR(Phi, 1.0f, 1.e-4f);
		}
		{
			// Near point just inside the top face, near the forward edge
			const FVec3 Point = FVec3(0.5f * ScaledSize.X, 0.0f, 0.5f * ScaledSize.Z) + FVec3(-3, 0, 1);
			FVec3 Normal;
			FReal Phi = ScaledConvex.PhiWithNormal(Point, Normal);
			EXPECT_NEAR(Normal.Z, 1.0f, 1.e-4f);
			EXPECT_NEAR(Phi, 1.0f, 1.e-4f);
		}
		{
			// Near point just inside the top face, near the forward edge
			const FVec3 Point = FVec3(0.5f * ScaledSize.X, 0.0f, 0.5f * ScaledSize.Z) + FVec3(-1, 0, 0.1);
			FVec3 Normal;
			FReal Phi = ScaledConvex.PhiWithNormal(Point, Normal);
			EXPECT_NEAR(Normal.Z, 1.0f, 1.e-4f);
			EXPECT_NEAR(Phi, 0.1f, 1.e-4f);
		}
		{
			// Point outside the face edge 
			const FVec3 Point = FVec3(0.5f * ScaledSize.X, 0.0f, 0.5f * ScaledSize.Z) + FVec3(1, 0, 1);
			FVec3 Normal;
			FReal Phi = ScaledConvex.PhiWithNormal(Point, Normal);
			const FVec3 ExpectedNormal = FVec3(1, 0, 1).GetUnsafeNormal();
			const FReal ExpectedPhi = FVec3(1, 0, 1).Size();
			EXPECT_NEAR(Normal.X, ExpectedNormal.X, 1.e-4f);
			EXPECT_NEAR(Normal.Y, ExpectedNormal.Y, 1.e-4f);
			EXPECT_NEAR(Normal.Z, ExpectedNormal.Z, 1.e-4f);
			EXPECT_NEAR(Phi, ExpectedPhi, 1.e-4f);
		}
		{
			// Point outside the face corner 
			const FVec3 Point = FVec3(0.5f * ScaledSize.X, 0.5f * ScaledSize.Y, 0.5f * ScaledSize.Z) + FVec3(3, 2, 1);
			FVec3 Normal;
			FReal Phi = ScaledConvex.PhiWithNormal(Point, Normal);
			const FVec3 ExpectedNormal = FVec3(3, 2, 1).GetUnsafeNormal();
			const FReal ExpectedPhi = FVec3(3, 2, 1).Size();
			EXPECT_NEAR(Normal.X, ExpectedNormal.X, 1.e-4f);
			EXPECT_NEAR(Normal.Y, ExpectedNormal.Y, 1.e-4f);
			EXPECT_NEAR(Normal.Z, ExpectedNormal.Z, 1.e-4f);
			EXPECT_NEAR(Phi, ExpectedPhi, 1.e-4f);
		}
	}


	void ImplicitTransformed()
	{
		FRigidTransform3 Identity(FVec3(0), FQuat::Identity);
		
		FImplicitObjectPtr UnitCube = MakeImplicitObjectPtr<TBox<FReal, 3>>(FVec3(-1), FVec3(1));
		TImplicitObjectTransformed<FReal, 3> UnitUnrotated(UnitCube, FRigidTransform3(FVec3(0), FQuat::Identity));
		UnitImplicitObjectNormalsInternal(UnitUnrotated, FString("ImplicitTransformed()"));
		UnitImplicitObjectNormalsExternal(UnitUnrotated, FString("ImplicitTransformed()"));
		UnitImplicitObjectIntersections(UnitUnrotated, FString("ImplicitTransformed()"));
		
		// Rotate 45 degrees around z axis @ origin.
		TImplicitObjectTransformed<FReal, 3> UnitRotated(UnitCube, FRigidTransform3(FVec3(0), FQuat(0, 0, sin(.3927), cos(.3927))));
		
		{// unit rotated normals
			FVec3 Normal;
			FReal TestPhi = UnitRotated.PhiWithNormal(FVec3(1 / 2., 1 / 2., 0), Normal);
			EXPECT_VECTOR_NEAR_DEFAULT(Normal, FVec3(sqrt(2) / 2., sqrt(2) / 2., 0));
			TestPhi = UnitRotated.PhiWithNormal(FVec3(-1 / 2., 1 / 2., 0), Normal);
			EXPECT_VECTOR_NEAR_DEFAULT(Normal, FVec3(-sqrt(2) / 2., sqrt(2) / 2., 0));
			TestPhi = UnitRotated.PhiWithNormal(FVec3(1 / 2., -1 / 2., 0), Normal);
			EXPECT_VECTOR_NEAR_DEFAULT(Normal, FVec3(sqrt(2) / 2., -sqrt(2) / 2., 0));
			TestPhi = UnitRotated.PhiWithNormal(FVec3(-1 / 2., -1 / 2., 0), Normal);
			EXPECT_VECTOR_NEAR_DEFAULT(Normal, FVec3(-sqrt(2) / 2., -sqrt(2) / 2., 0));
		}

		FImplicitObjectPtr Cube = MakeImplicitObjectPtr<TBox<FReal, 3>>(FVec3(-2, -5, -5), FVec3(8, 5, 5));
		TImplicitObjectTransformed<FReal, 3> Untransformed(Cube, FRigidTransform3(FVec3(0), FQuat::Identity));
		TImplicitObjectTransformed<FReal, 3> Translated(Cube, FRigidTransform3(FVec3(4, 0, 0), FQuat::Identity));
		
		// Rotate 90 degrees around z axis @ origin. 
		FReal rad_45 = FMath::DegreesToRadians(45);
		TImplicitObjectTransformed<FReal, 3> Rotated(Cube, FRigidTransform3(FVec3(0), FQuat(0, 0, sin(rad_45), cos(rad_45))));
		TImplicitObjectTransformed<FReal, 3> Transformed(Cube, FRigidTransform3(FVec3(4, 0, 0), FQuat(0, 0, sin(rad_45), cos(rad_45))));

		{// phi
			const FVec3 NearEdge(7.5, 0, 0);
			FVec3 UntransformedNormal;
			const FReal UntransformedPhi = Untransformed.PhiWithNormal(NearEdge, UntransformedNormal);
			EXPECT_FLOAT_EQ(UntransformedPhi, -0.5);
			EXPECT_VECTOR_NEAR_DEFAULT(UntransformedNormal, FVec3(1, 0, 0));

			FVec3 TransformedNormal;
			FReal TranslatedPhi = Translated.PhiWithNormal(NearEdge, TransformedNormal);
			EXPECT_FLOAT_EQ(TranslatedPhi, -(0.5 + 4));
			EXPECT_VECTOR_NEAR_DEFAULT(TransformedNormal, FVec3(1, 0, 0));

			const FVec3 NearEdgeRotated(0, 7.5, 0);
			FReal RotatedPhi = Rotated.PhiWithNormal(NearEdgeRotated, TransformedNormal);
			EXPECT_FLOAT_EQ(RotatedPhi, -0.5);
			EXPECT_VECTOR_NEAR_DEFAULT(TransformedNormal, FVec3(0, 1, 0));

			FReal TransformedPhi = Transformed.PhiWithNormal(NearEdge, TransformedNormal);
			EXPECT_FLOAT_EQ(TransformedPhi, -(0.5 + 1));
			EXPECT_VECTOR_NEAR_DEFAULT(TransformedNormal, FVec3(1, 0, 0));

			const FVec3 NearTop(7, 0, 4.5);
			TransformedPhi = Transformed.PhiWithNormal(NearTop, TransformedNormal);
			EXPECT_FLOAT_EQ(TransformedPhi, -(0.5));
			EXPECT_VECTOR_NEAR_DEFAULT(TransformedNormal, FVec3(0, 0, 1));
		}
		
		{//support

			const FVec3 DirX(1, 0, 0);
			int32 VertexIndex = INDEX_NONE;
			FVec3 SupportPt = Utilities::CastHelper(Untransformed, Identity, [&](const auto& Concrete, const auto& FullTM)
			{
				FVec3 SupportLocal = Concrete.Support(FullTM.InverseTransformVectorNoScale(DirX), 1, VertexIndex);
				return FullTM.TransformPosition(SupportLocal);
			});
			EXPECT_VECTOR_NEAR_DEFAULT(SupportPt, FVec3(9, 5, 5));

			SupportPt = Utilities::CastHelper(Translated, Identity, [&](const auto& Concrete, const auto& FullTM)
			{
				FVec3 SupportLocal = Concrete.Support(FullTM.InverseTransformVectorNoScale(DirX), 1, VertexIndex);
				return FullTM.TransformPosition(SupportLocal);
			});
			EXPECT_VECTOR_NEAR_DEFAULT(SupportPt, FVec3(13, 5, 5));

			const FVec3 DirZ(0, 0, -1);
			SupportPt = Utilities::CastHelper(Translated, Identity, [&](const auto& Concrete, const auto& FullTM)
			{
				FVec3 SupportLocal = Concrete.Support(FullTM.InverseTransformVectorNoScale(DirZ), 1, VertexIndex);
				return FullTM.TransformPosition(SupportLocal);
			});
			EXPECT_VECTOR_NEAR_DEFAULT(SupportPt, FVec3(12, 5, -6));

			SupportPt = Utilities::CastHelper(Rotated, Identity, [&](const auto& Concrete, const auto& FullTM)
			{
				FVec3 SupportLocal = Concrete.Support(FullTM.InverseTransformVectorNoScale(DirZ), 1, VertexIndex);
				return FullTM.TransformPosition(SupportLocal);
			});
			EXPECT_VECTOR_NEAR_DEFAULT(SupportPt, FVec3(-5, 8, -6)); // @todo why -5?

			SupportPt = Utilities::CastHelper(Transformed, Identity, [&](const auto& Concrete, const auto& FullTM)
			{
				FVec3 SupportLocal = Concrete.Support(FullTM.InverseTransformVectorNoScale(DirZ), 1, VertexIndex);
				FVec3 TransformedPt = FullTM.TransformPosition(SupportLocal);
				return TransformedPt;
			});
			EXPECT_VECTOR_NEAR_DEFAULT(SupportPt, FVec3(-1, 8, -6));
		}

		{// closest intersection
			Pair<FVec3, bool> Result;
			Result = Untransformed.FindClosestIntersection(FVec3(7.5, 0, 0), FVec3(8.5, 0, 0), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(8, 0, 0), 0.001);

			Result = Translated.FindClosestIntersection(FVec3(11.5, 0, 0), FVec3(12.5, 0, 0), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(12, 0, 0), 0.001);

			Result = Rotated.FindClosestIntersection(FVec3(0, 7.5, 0), FVec3(0, 8.5, 0), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(0, 8, 0), 0.001);

			Result = Translated.FindClosestIntersection(FVec3(7, 0, 4.5), FVec3(7, 0, 5.5), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(7, 0, 5), 0.001);
		}
	}
	

	void ImplicitIntersection()
	{
		FString Caller("ImplicitIntersection()");

		// Two cylinders intersected to make a unit cylinder.
		TArray<Chaos::FImplicitObjectPtr> Objects;
		Objects.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(0, 0, 2), FVec3(0, 0, -1), 1));
		Objects.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(0, 0, 1), FVec3(0, 0, -2), 1));

		TImplicitObjectIntersection<FReal, 3> MIntersectedObjects(std::move(Objects));

		UnitImplicitObjectNormalsInternal(MIntersectedObjects, Caller);
		UnitImplicitObjectNormalsExternal(MIntersectedObjects, Caller);
		UnitImplicitObjectIntersections(MIntersectedObjects, Caller);

		Pair<FVec3, bool> Result;
		{// closest intersection near origin
			Result = MIntersectedObjects.FindClosestIntersection(FVec3(0, 0, 1 / 2.), FVec3(0, 0, 3 / 2.), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(0, 0, 1), 0.001);

			Result = MIntersectedObjects.FindClosestIntersection(FVec3(0, 0, -3 / 2.), FVec3(0, 0, -1 / 2.), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(0, 0, -1), 0.001);

			Result = MIntersectedObjects.FindClosestIntersection(FVec3(0, 1 / 2., 0), FVec3(0, 3 / 2., 0), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(0, 1, 0), 0.001);

			Result = MIntersectedObjects.FindClosestIntersection(FVec3(0, 3 / 2., 0), FVec3(0, 1 / 2., 0), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(0, 1, 0), 0.001);

			// Verify that there's no intersection with non-overlapping parts of the two cylinders. 
			Result = MIntersectedObjects.FindClosestIntersection(FVec3(0, 0, 5 / 2.), FVec3(0, 0, 7 / 2.), KINDA_SMALL_NUMBER);
			EXPECT_FALSE(Result.Second);

			Result = MIntersectedObjects.FindClosestIntersection(FVec3(0, 0, -7 / 2.), FVec3(0, 0, -5 / 2.), KINDA_SMALL_NUMBER);
			EXPECT_FALSE(Result.Second);
		}

		TArray<Chaos::FImplicitObjectPtr> Objects2;
		Objects2.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(4, 4, 6), FVec3(4, 4, 3), 1));
		Objects2.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(4, 4, 5), FVec3(4, 4, 2), 1));

		TImplicitObjectIntersection<FReal, 3> MIntersectedObjects2(std::move(Objects2));
		
		{// closest intersection off origin
			Result = MIntersectedObjects2.FindClosestIntersection(FVec3(4, 4, 4 + 1 / 2.), FVec3(4, 4, 4 + 3 / 2.), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(4, 4, 5), 0.001);

			Result = MIntersectedObjects2.FindClosestIntersection(FVec3(4, 4, 4 + -3 / 2.), FVec3(4, 4, 4 + -1 / 2.), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(4, 4, 3), 0.001);

			Result = MIntersectedObjects2.FindClosestIntersection(FVec3(4, 4 + 1 / 2., 4), FVec3(4, 4 + 3 / 2., 4), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(4, 5, 4), 0.001);

			Result = MIntersectedObjects2.FindClosestIntersection(FVec3(4, 4 + 3 / 2., 4), FVec3(4, 4 + 1 / 2., 4), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.First, FVec3(4, 5, 4), 0.001);

			// Verify that there's no intersection with non-overlapping parts of the two cylinders. 
			Result = MIntersectedObjects2.FindClosestIntersection(FVec3(4, 4, 4 + 5 / 2.), FVec3(4, 4, 4 + 7 / 2.), KINDA_SMALL_NUMBER);
			EXPECT_FALSE(Result.Second);

			Result = MIntersectedObjects2.FindClosestIntersection(FVec3(4, 4, 4 + -7 / 2.), FVec3(4, 4, 4 + -5 / 2.), KINDA_SMALL_NUMBER);
			EXPECT_FALSE(Result.Second);
		}
	}


	void ImplicitUnion()
	{
		FString Caller("ImplicitUnion()");
		FImplicitObjectUnionPtr MUnionedObjects;

		{// unit cylinder - sanity check
			TArray<Chaos::FImplicitObjectPtr> Objects;
			Objects.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(0, 0, 1), FVec3(0), 1));
			Objects.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(0, 0, -1), FVec3(0), 1));
			MUnionedObjects = FImplicitObjectUnionPtr(new Chaos::FImplicitObjectUnion(std::move(Objects)));

			// Can't use the default internal unit tests because they expect different behavior internally where the two cylinders are joined together. 
			EXPECT_VECTOR_NEAR(MUnionedObjects->Normal(FVec3(0, 0, 2 / 3.)), (FVec3(0, 0, 1)), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(MUnionedObjects->Normal(FVec3(0, 0, -2 / 3.)), (FVec3(0, 0, -1)), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(MUnionedObjects->Normal(FVec3(0, 2 / 3., 0)), (FVec3(0, 0, 0)), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(MUnionedObjects->Normal(FVec3(0, -2 / 3., 0)), (FVec3(0, 0, 0)), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(MUnionedObjects->Normal(FVec3(2 / 3., 0, 0)), (FVec3(0, 0, 0)), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(MUnionedObjects->Normal(FVec3(-2 / 3., 0, 0)), (FVec3(0, 0, 0)), KINDA_SMALL_NUMBER);

			UnitImplicitObjectNormalsExternal(*MUnionedObjects, Caller);

			EXPECT_NEAR(MUnionedObjects->SignedDistance(FVec3(0, 0, 5 / 4.)), 1 / 4., KINDA_SMALL_NUMBER);
			EXPECT_NEAR(MUnionedObjects->SignedDistance(FVec3(0, 0, 3 / 4.)), -1 / 4., KINDA_SMALL_NUMBER);
			EXPECT_NEAR(MUnionedObjects->SignedDistance(FVec3(0, 5 / 4., 0)), 1 / 4., KINDA_SMALL_NUMBER);
			// Internal distance 0 because it's where the spheres overlap.
			EXPECT_NEAR(MUnionedObjects->SignedDistance(FVec3(0, 3 / 4., 0)), 0., KINDA_SMALL_NUMBER);

			TestFindClosestIntersection(*MUnionedObjects, FVec3(0, 0, 5 / 4.), FVec3(0, 0, 1), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(0, 0, -5 / 4.), FVec3(0, 0, -1), Caller);
		}

		TArray<Chaos::FImplicitObjectPtr> Objects;
		Objects.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(0, 0, -2), FVec3(0, 0, 2), 1));
		Objects.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(0, -2, 0), FVec3(0, 2, 0), 1));
		MUnionedObjects = FImplicitObjectUnionPtr(new Chaos::FImplicitObjectUnion(std::move(Objects)));

		{// closest point near origin (+)
			EXPECT_NEAR(MUnionedObjects->SignedDistance(FVec3(0, 0, 9 / 4.)), 1 / 4., KINDA_SMALL_NUMBER);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(0, 0, 9 / 4.), FVec3(0, 0, 2), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(0, 0, -9 / 4.), FVec3(0, 0, -2), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(0, 9 / 4., 0), FVec3(0, 2, 0), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(0, -9 / 4., 0), FVec3(0, -2, 0), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(3 / 2., 0, 0), FVec3(1, 0, 0), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(-3 / 2., 0, 0), FVec3(-1, 0, 0), Caller);
		}

		{// closest point near origin (-)
			EXPECT_NEAR(MUnionedObjects->SignedDistance(FVec3(0, 0, 7 / 4.)), -1 / 4., KINDA_SMALL_NUMBER);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(0, 0, 7 / 4.), FVec3(0, 0, 2), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(0, 0, -7 / 4.), FVec3(0, 0, -2), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(0, 7 / 4., 0), FVec3(0, 2, 0), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(0, -7 / 4., 0), FVec3(0, -2, 0), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(1 / 2., 0, 0), FVec3(1, 0, 0), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(-1 / 2., 0, 0), FVec3(-1, 0, 0), Caller);
		}
		
		TArray<Chaos::FImplicitObjectPtr> Objects2;
		Objects2.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(4, 4, 2), FVec3(4, 4, 6), 1));
		Objects2.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(4, 2, 4), FVec3(4, 6, 4), 1));
		MUnionedObjects = FImplicitObjectUnionPtr(new Chaos::FImplicitObjectUnion(std::move(Objects2)));

		{// closest point off origin (+)
			EXPECT_NEAR(MUnionedObjects->SignedDistance(FVec3(4, 4, 4 + 9 / 4.)), 1 / 4., KINDA_SMALL_NUMBER);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(4, 4, 4 + 9 / 4.), FVec3(4, 4, 6), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(4, 4, 4 + -9 / 4.), FVec3(4, 4, 2), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(4, 4 + 9 / 4., 4), FVec3(4, 6, 4), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(4, 4 + -9 / 4., 4), FVec3(4, 2, 4), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(4 + 3 / 2., 4, 4), FVec3(5, 4, 4), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(4 + -3 / 2., 4, 4), FVec3(3, 4, 4), Caller);
		}

		{// closest point off origin (-)
			EXPECT_NEAR(MUnionedObjects->SignedDistance(FVec3(4, 4, 4 + 7 / 4.)), -1 / 4., KINDA_SMALL_NUMBER);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(4, 4, 4 + 7 / 4.), FVec3(4, 4, 6), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(4, 4, 4 + -7 / 4.), FVec3(4, 4, 2), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(4, 4 + 7 / 4., 4), FVec3(4, 6, 4), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(4, 4 + -7 / 4., 4), FVec3(4, 2, 4), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(4 + 1 / 2., 4, 4), FVec3(5, 4, 4), Caller);
			TestFindClosestIntersection(*MUnionedObjects, FVec3(4 + -1 / 2., 4, 4), FVec3(3, 4, 4), Caller);
		}

		/* Nested Unions */
		
		{// Union of unions (capsule)
			TArray<Chaos::FImplicitObjectPtr> Unions;
			Unions.Add(MakeImplicitObjectPtr<FCapsule>(FVec3(0, 0, 0), FVec3(0, 0, -2), 1));
			Unions.Add(MakeImplicitObjectPtr<FCapsule>(FVec3(0, 0, 0), FVec3(0, 0, 2), 1));
			MUnionedObjects = FImplicitObjectUnionPtr(new Chaos::FImplicitObjectUnion(std::move(Unions)));

			EXPECT_VECTOR_NEAR(MUnionedObjects->Normal(FVec3(0, 0, 7 / 3.)), (FVec3(0, 0, 1)), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(MUnionedObjects->Normal(FVec3(0, 0, -7 / 3.)), (FVec3(0, 0, -1)), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(MUnionedObjects->Normal(FVec3(0, 1 / 2., 0)), (FVec3(0, 1, 0)), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(MUnionedObjects->Normal(FVec3(0, -1 / 2., 0)), (FVec3(0, -1, 0)), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(MUnionedObjects->Normal(FVec3(1 / 2., 0, 0)), (FVec3(1, 0, 0)), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(MUnionedObjects->Normal(FVec3(-1 / 2., 0, 0)), (FVec3(-1, 0, 0)), KINDA_SMALL_NUMBER);
			EXPECT_NEAR(MUnionedObjects->SignedDistance(FVec3(0, 0, 13 / 4.)), 1 / 4., KINDA_SMALL_NUMBER);
			EXPECT_NEAR(MUnionedObjects->SignedDistance(FVec3(0, 0, 11 / 4.)), -1 / 4., KINDA_SMALL_NUMBER);
			EXPECT_NEAR(MUnionedObjects->SignedDistance(FVec3(0, 1 / 2., 0)), -1 / 2., KINDA_SMALL_NUMBER);
			EXPECT_NEAR(MUnionedObjects->SignedDistance(FVec3(0, 3 / 2., 0)), 1 / 2., KINDA_SMALL_NUMBER);
		}

		{// Union of a union containing all the unit geometries overlapping - should still pass all the normal unit tests. 
			TArray<Chaos::FImplicitObjectPtr> Objects1;
			Objects1.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(0, 0, 1), FVec3(0, 0, -1), 1));
			Objects1.Add(MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0, 0, 0), 1));
			Objects1.Add(MakeImplicitObjectPtr<TBox<FReal, 3>>(FVec3(-1, -1, -1), FVec3(1, 1, 1)));
			Objects1.Add(MakeImplicitObjectPtr<FTaperedCylinder>(FVec3(0, 0, 1), FVec3(0, 0, -1), 1, 1));

			TArray<Chaos::FImplicitObjectPtr> Unions;
			Unions.Emplace(new FImplicitObjectUnion(MoveTemp(Objects1)));
			FImplicitObjectUnionPtr UnionedUnions(new Chaos::FImplicitObjectUnion(std::move(Unions)));

			UnitImplicitObjectNormalsExternal(*UnionedUnions, FString("ImplicitUnion() - nested union unit cylinder 1"));
			UnitImplicitObjectNormalsInternal(*UnionedUnions, FString("ImplicitUnion() - nested union unit cylinder 1"));
			UnitImplicitObjectIntersections(*UnionedUnions, FString("ImplicitUnion() - nested union unit cylinder 1"));
		}

		{// Union of two unions, each with two unit objects
			TArray<Chaos::FImplicitObjectPtr> ObjectsA;
			TArray<Chaos::FImplicitObjectPtr> ObjectsB;
			ObjectsA.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(0, 0, 1), FVec3(0, 0, -1), 1));
			ObjectsA.Add(MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0, 0, 0), 1));
			ObjectsB.Add(MakeImplicitObjectPtr<TBox<FReal, 3>>(FVec3(-1, -1, -1), FVec3(1, 1, 1)));
			ObjectsB.Add(MakeImplicitObjectPtr<FTaperedCylinder>(FVec3(0, 0, 1), FVec3(0, 0, -1), 1, 1));

			TArray<Chaos::FImplicitObjectPtr> Unions;
			Unions.Emplace(new FImplicitObjectUnion(MoveTemp(ObjectsA)));
			Unions.Emplace(new FImplicitObjectUnion(MoveTemp(ObjectsB)));
			FImplicitObjectUnionPtr UnionedUnions(new Chaos::FImplicitObjectUnion(std::move(Unions)));

			UnitImplicitObjectNormalsExternal(*UnionedUnions, FString("ImplicitUnion() - nested union unit sphere 1"));
			UnitImplicitObjectNormalsInternal(*UnionedUnions, FString("ImplicitUnion() - nested union unit sphere 1"));
			UnitImplicitObjectIntersections(*UnionedUnions, FString("ImplicitUnion() - nested union unit sphere 1"));
		}

		{// Mimic a unit cylinder, but made up of multiple unions. 
			TArray<Chaos::FImplicitObjectPtr> ObjectsA;
			TArray<Chaos::FImplicitObjectPtr> ObjectsB;
			ObjectsA.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(0, 0, 0), FVec3(0, 0, -1), 1));
			ObjectsB.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(0, 0, 0), FVec3(0, 0, 1), 1));
			TArray<Chaos::FImplicitObjectPtr> Unions;
			Unions.Emplace(new FImplicitObjectUnion(MoveTemp(ObjectsA)));
			Unions.Emplace(new FImplicitObjectUnion(MoveTemp(ObjectsB)));
			FImplicitObjectUnionPtr UnionedUnions(new Chaos::FImplicitObjectUnion(std::move(Unions)));

			UnitImplicitObjectNormalsExternal(*UnionedUnions, FString("ImplicitUnion() - nested union unit cylinder 2"));

			EXPECT_VECTOR_NEAR(UnionedUnions->Normal(FVec3(0, 0, 2 / 3.)), (FVec3(0, 0, 1)), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(UnionedUnions->Normal(FVec3(0, 0, -2 / 3.)), (FVec3(0, 0, -1)), KINDA_SMALL_NUMBER);
			// Normal is averaged to 0 at the joined faces. 
			EXPECT_VECTOR_NEAR(UnionedUnions->Normal(FVec3(0, 0, 0)), (FVec3(0, 0, 0)), KINDA_SMALL_NUMBER);

			EXPECT_NEAR(UnionedUnions->SignedDistance(FVec3(0, 5 / 4., 0)), 1 / 4., KINDA_SMALL_NUMBER);
			EXPECT_NEAR(UnionedUnions->SignedDistance(FVec3(0, -5 / 4., 0)), 1 / 4., KINDA_SMALL_NUMBER);
			EXPECT_NEAR(UnionedUnions->SignedDistance(FVec3(5 / 4., 0, 0)), 1 / 4., KINDA_SMALL_NUMBER);
			EXPECT_NEAR(UnionedUnions->SignedDistance(FVec3(-5 / 4., 0, 0)), 1 / 4., KINDA_SMALL_NUMBER);

			// Distance is 0 at the joined faces.
			EXPECT_NEAR(UnionedUnions->SignedDistance(FVec3(0, 0, 0)), 0., KINDA_SMALL_NUMBER);
		}

	}


	void ImplicitLevelset()
	{
		Chaos::FPBDRigidParticles Particles;
		TArray<TVec3<int32>> CollisionMeshElements;
		int32 BoxId = AppendParticleBox(Particles, FVec3(1), &CollisionMeshElements);
		FLevelSet Levelset = ConstructLevelset(*Particles.CollisionParticles(BoxId), CollisionMeshElements);

		FVec3 Normal;
		FReal Phi = Levelset.PhiWithNormal(FVec3(0, 0, 2), Normal);
		EXPECT_GT(Phi, 0);
		EXPECT_NEAR((Phi - 1.5), 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Normal, FVec3(0, 0, 1), 0.001);

		Phi = Levelset.PhiWithNormal(FVec3(0, 2, 0), Normal);
		EXPECT_GT(Phi, 0);
		EXPECT_NEAR((Phi - 1.5), 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Normal, FVec3(0, 1, 0), 0.001);
		
		Phi = Levelset.PhiWithNormal(FVec3(2, 0, 0), Normal);
		EXPECT_GT(Phi, 0);
		EXPECT_NEAR((Phi - 1.5), 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Normal, FVec3(1, 0, 0), 0.001);

		Phi = Levelset.PhiWithNormal(FVec3(0, 0, -2), Normal);
		EXPECT_GT(Phi, 0);
		EXPECT_NEAR((Phi - 1.5), 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Normal, FVec3(0, 0, -1), 0.001);

		Phi = Levelset.PhiWithNormal(FVec3(0, -2, 0), Normal);
		EXPECT_GT(Phi, 0);
		EXPECT_NEAR((Phi - 1.5), 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Normal, FVec3(0, -1, 0), 0.001);
		
		Phi = Levelset.PhiWithNormal(FVec3(-2, 0, 0), Normal);
		EXPECT_GT(Phi, 0);
		EXPECT_NEAR((Phi - 1.5), 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), 0.001); /**/

		Phi = Levelset.PhiWithNormal(FVec3(0, 0, 0.25f), Normal);
		EXPECT_LT(Phi, 0);
		EXPECT_NEAR((Phi + 0.25), 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Normal, FVec3(0, 0, 1), 0.001);

		Phi = Levelset.PhiWithNormal(FVec3(0, 0.25f, 0), Normal);
		EXPECT_LT(Phi, 0);
		EXPECT_NEAR((Phi + 0.25), 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Normal, FVec3(0, 1, 0), 0.001);

		Phi = Levelset.PhiWithNormal(FVec3(0.25f, 0, 0), Normal);
		EXPECT_LT(Phi, 0);
		EXPECT_NEAR((Phi + 0.25), 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Normal, FVec3(1, 0, 0), 0.001);

		Phi = Levelset.PhiWithNormal(FVec3(0, 0, -0.25f), Normal);
		EXPECT_LT(Phi, 0);
		EXPECT_NEAR((Phi + 0.25), 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Normal, FVec3(0, 0, -1), 0.001);

		Phi = Levelset.PhiWithNormal(FVec3(0, -0.25f, 0), Normal);
		EXPECT_LT(Phi, 0);
		EXPECT_NEAR((Phi + 0.25), 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Normal, FVec3(0, -1, 0), 0.001);

		Phi = Levelset.PhiWithNormal(FVec3(-0.25f, 0, 0), Normal);
		EXPECT_LT(Phi, 0);
		EXPECT_NEAR((Phi + 0.25), 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Normal, FVec3(-1, 0, 0), 0.001);
	}

		void RasterizationImplicit()
	{
		FImplicitObjectPtr Box(new TBox<FReal,3>(FVec3(-0.5, -0.5, -0.5), FVec3(0.5, 0.5, 0.5)));
		TArray<Chaos::FImplicitObjectPtr> Objects;
		Objects.Add(MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(Box, FRigidTransform3(FVec3(0.5, 0, 0), FRotation3::FromVector(FVec3(0)))));
		Objects.Add(MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(Box, FRigidTransform3(FVec3(-0.5, 0, 0), FRotation3::FromVector(FVec3(0)))));
		FImplicitObjectUnion Union(MoveTemp(Objects));
		FErrorReporter ErrorReporter;
		// This one should be exactly right as we don't actually do an fast marching interior to the region
		{
			TUniformGrid<FReal, 3> Grid(FVec3(-2.0, -1.5, -1.5), FVec3(2.0, 1.5, 1.5), TVec3<int32>(4, 3, 3));
			FLevelSet LevelSet(ErrorReporter, Grid, Union);
			EXPECT_TRUE(LevelSet.IsConvex());
			EXPECT_LT(LevelSet.SignedDistance(FVec3(0)) + FReal(0.5), KINDA_SMALL_NUMBER);
		}
		// We should get closer answers every time we refine the resolution
		{
			ErrorReporter.HandleLatestError();
			TUniformGrid<FReal, 3> Grid(FVec3(-1.5, -1.0, -1.0), FVec3(1.5, 1.0, 1.0), TVec3<int32>(6, 4, 4));
			FLevelSet LevelSet(ErrorReporter, Grid, Union);
			EXPECT_TRUE(LevelSet.IsConvex());
			EXPECT_LT(LevelSet.SignedDistance(FVec3(0)) + FReal(0.25), KINDA_SMALL_NUMBER);
		}
		{
			ErrorReporter.HandleLatestError();
			TUniformGrid<FReal, 3> Grid(FVec3(-1.25, -0.75, -0.75), FVec3(1.25, 0.75, 0.75), TVec3<int32>(10, 6, 6));
			FLevelSet LevelSet(ErrorReporter, Grid, Union);
			EXPECT_TRUE(LevelSet.IsConvex());
			EXPECT_LT(LevelSet.SignedDistance(FVec3(0)) + FReal(0.3), KINDA_SMALL_NUMBER);
		}
		{
			ErrorReporter.HandleLatestError();
			TUniformGrid<FReal, 3> Grid(FVec3(-1.1, -0.6, -0.6), FVec3(1.1, 0.6, 0.6), TVec3<int32>(22, 12, 12));
			FLevelSet LevelSet(ErrorReporter, Grid, Union);
			EXPECT_TRUE(LevelSet.IsConvex());
			EXPECT_LT(LevelSet.SignedDistance(FVec3(0)) + FReal(0.4), KINDA_SMALL_NUMBER);
		}
		{
			ErrorReporter.HandleLatestError();
			TUniformGrid<FReal, 3> Grid(FVec3(-1.05, -0.55, -0.55), FVec3(1.05, 0.55, 0.55), TVec3<int32>(42, 22, 22));
			FLevelSet LevelSet(ErrorReporter, Grid, Union);
			EXPECT_TRUE(LevelSet.IsConvex());
			EXPECT_LT(LevelSet.SignedDistance(FVec3(0)) + FReal(0.45), KINDA_SMALL_NUMBER);
		}
		{
			ErrorReporter.HandleLatestError();
			TUniformGrid<FReal, 3> Grid(FVec3(-1.5, -1.0, -1.0), FVec3(1.5, 1.0, 1.0), TVec3<int32>(20, 20, 20));
			FLevelSet LevelSet(ErrorReporter, Grid, Union);

			FReal Volume;
			FVec3 COM;
			FMatrix33 Inertia;
			FRotation3 RotationOfMass;

			LevelSet.ComputeMassProperties(Volume, COM, Inertia, RotationOfMass);
			EXPECT_GT(Volume, 1);
			EXPECT_LT(Volume, 3);
			EXPECT_LT(Inertia.M[0][0] * 1.5, Inertia.M[1][1]);
			EXPECT_GT(Inertia.M[0][0] * 3, Inertia.M[1][1]);
			EXPECT_NEAR(Inertia.M[2][2], Inertia.M[1][1], SMALL_NUMBER);
		}
	}

	void RasterizationImplicitWithHole()
	{
		FImplicitObjectPtr Box(new TBox<FReal, 3>(FVec3(-0.5, -0.5, -0.5), FVec3(0.5, 0.5, 0.5)));
		TArray<Chaos::FImplicitObjectPtr> Objects;
		Objects.Add(MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(Box, FRigidTransform3(FVec3(1, 1, 0), FRotation3::FromVector(FVec3(0)))));
		Objects.Add(MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(Box, FRigidTransform3(FVec3(0, 1, 0), FRotation3::FromVector(FVec3(0)))));
		Objects.Add(MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(Box, FRigidTransform3(FVec3(-1, 1, 0), FRotation3::FromVector(FVec3(0)))));
		Objects.Add(MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(Box, FRigidTransform3(FVec3(1, 0, 0), FRotation3::FromVector(FVec3(0)))));
		Objects.Add(MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(Box, FRigidTransform3(FVec3(-1, 0, 0), FRotation3::FromVector(FVec3(0)))));
		Objects.Add(MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(Box, FRigidTransform3(FVec3(1, -1, 0), FRotation3::FromVector(FVec3(0)))));
		Objects.Add(MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(Box, FRigidTransform3(FVec3(0, -1, 0), FRotation3::FromVector(FVec3(0)))));
		Objects.Add(MakeImplicitObjectPtr<TImplicitObjectTransformed<FReal, 3>>(Box, FRigidTransform3(FVec3(-1, -1, 0), FRotation3::FromVector(FVec3(0)))));
		FImplicitObjectUnion Union(MoveTemp(Objects));
		{
			TUniformGrid<FReal, 3> Grid(FVec3(-1.6, -1.6, -0.6), FVec3(1.6, 1.6, 0.6), TVec3<int32>(32, 32, 12));
			FErrorReporter ErrorReporter;
			FLevelSet LevelSet(ErrorReporter, Grid, Union);
			EXPECT_FALSE(LevelSet.IsConvex());
			EXPECT_GT(LevelSet.SignedDistance(FVec3(0)), -KINDA_SMALL_NUMBER);
			EXPECT_LT(LevelSet.SignedDistance(FVec3(1, 1, 0)), KINDA_SMALL_NUMBER);
			EXPECT_LT(LevelSet.SignedDistance(FVec3(0, 1, 0)), KINDA_SMALL_NUMBER);
			EXPECT_LT(LevelSet.SignedDistance(FVec3(-1, 1, 0)), KINDA_SMALL_NUMBER);
			EXPECT_LT(LevelSet.SignedDistance(FVec3(-1, 0, 0)), KINDA_SMALL_NUMBER);
			EXPECT_LT(LevelSet.SignedDistance(FVec3(1, 0, 0)), KINDA_SMALL_NUMBER);
			EXPECT_LT(LevelSet.SignedDistance(FVec3(1, -1, 0)), KINDA_SMALL_NUMBER);
			EXPECT_LT(LevelSet.SignedDistance(FVec3(0, -1, 0)), KINDA_SMALL_NUMBER);
			EXPECT_LT(LevelSet.SignedDistance(FVec3(-1, -1, 0)), KINDA_SMALL_NUMBER);
		}
	}

	void ConvexHull()
	{
		{
			FParticles Particles;
			Particles.AddParticles(9);
			Particles.SetX(0, FVec3(-1, -1, -1));
			Particles.SetX(1, FVec3(-1, -1, 1));
			Particles.SetX(2, FVec3(-1, 1, -1));
			Particles.SetX(3, FVec3(-1, 1, 1));
			Particles.SetX(4, FVec3(1, -1, -1));
			Particles.SetX(5, FVec3(1, -1, 1));
			Particles.SetX(6, FVec3(1, 1, -1));
			Particles.SetX(7, FVec3(1, 1, 1));
			Particles.SetX(8, FVec3(0, 0, 0));
			const FTriangleMesh TriMesh = FTriangleMesh::GetConvexHullFromParticles(Particles);
			EXPECT_EQ(TriMesh.GetSurfaceElements().Num(), 12);
			for (const auto& Tri : TriMesh.GetSurfaceElements())
			{
				EXPECT_NE(Tri.X, 8);
				EXPECT_NE(Tri.Y, 8);
				EXPECT_NE(Tri.Z, 8);
			}

			TArray<FConvex::FVec3Type> Vertices;
			Vertices.SetNum((int32)Particles.Size());
			for (int32 VertexIndex = 0; VertexIndex < (int32)Particles.Size(); ++VertexIndex)
			{
				Vertices[VertexIndex] = Particles.GetX(VertexIndex);
			}
			FConvex Convex(Vertices, 0.0f);
			const TArray<FConvex::FVec3Type>& CulledParticles = Convex.GetVertices();
			EXPECT_EQ(CulledParticles.Num(), 8);

			for (int32 Idx = 0; Idx < CulledParticles.Num(); ++Idx)
			{
				EXPECT_NE(Particles.GetX(8), (Chaos::TVector<FRealDouble, 3>)CulledParticles[Idx]);	//interior particle gone
				bool bFound = false;
				for (uint32 InnerIdx = 0; InnerIdx < Particles.Size(); ++InnerIdx)	//remaining particles are from the original set
				{
					if (Particles.GetX(InnerIdx) == (Chaos::TVector<FRealDouble,3>)CulledParticles[Idx])
					{
						bFound = true;
						break;
					}
				}
				EXPECT_TRUE(bFound);
			}

		}

		{
			FParticles Particles;
			Particles.AddParticles(6);
			Particles.SetX(0, FVec3(-1, -1, -1));
			Particles.SetX(1, FVec3(1, -1, -1));
			Particles.SetX(2, FVec3(1, 1, -1));
			Particles.SetX(3, FVec3(0, 0, 0.5));
			Particles.SetX(4, (Particles.GetX(3) - Particles.GetX(1)) * 0.5 + Particles.GetX(1) + FVec3(0, 0, 0.1));
			Particles.SetX(5, Particles.GetX(4) + FVec3(-0.1, 0, 0));
			const FTriangleMesh TriMesh = FTriangleMesh::GetConvexHullFromParticles(Particles);
			//EXPECT_EQ(TriMesh.GetSurfaceElements().Num(), 6);
		}
	}


	void ConvexHull2()
	{
		{
			//degenerates
			TArray<FConvex::FVec3Type> Particles;
			Particles.SetNum(3);
			Particles[0] = { -1, -1, -1};
			Particles[1] = { 1, -1, -1 };
			Particles[2] = { 1, 1, -1 };
			TArray<TVector<int32, 3>>Indices;
			Chaos::FConvexBuilder::BuildConvexHull(Particles, Indices);
			EXPECT_EQ(Indices.Num(), 0);
			Particles.Add(Chaos::FVec3( 2, 3, -1 ));
			Chaos::FConvexBuilder::BuildConvexHull(Particles, Indices);
			EXPECT_EQ(Indices.Num(), 0);
		}
		{
			TArray <FConvex::FVec3Type> Particles;
			Particles.SetNum(9);
			Particles[0] = { -1, -1, -1 };
			Particles[1] = { -1, -1, 1 };
			Particles[2] = { -1, 1, -1 };
			Particles[3] = { -1, 1, 1 };
			Particles[4] = { 1, -1, -1 };
			Particles[5] = { 1, -1, 1 };
			Particles[6] = { 1, 1, -1 };
			Particles[7] = { 1, 1, 1 };
			Particles[8] = { 0, 0, 0 };
			TArray<TVector<int32, 3>>Indices;
			Chaos::FConvexBuilder::BuildConvexHull(Particles, Indices);
			EXPECT_EQ(Indices.Num(), 12);
			for (const auto& Tri : Indices)
			{
				EXPECT_NE(Tri.X, 8);
				EXPECT_NE(Tri.Y, 8);
				EXPECT_NE(Tri.Z, 8);
			}
		}
		{
			TArray<FConvex::FVec3Type> Particles;
			Particles.SetNum(5);
			Particles[0] = { -1, -1, -1 };
			Particles[1] = { 1, -1, -1 };
			Particles[2] = { 1, 1, -1 };
			Particles[3] = { 0, 0, 0.5 };
			Particles[4] = (Particles[3] - Particles[1]) * 0.5 + Particles[1] + FConvex::FVec3Type{ 0, 0, 0.1f };
			TArray<TVector<int32, 3>> Indices;
			Chaos::FConvexBuilder::BuildConvexHull(Particles, Indices);
			EXPECT_EQ(Indices.Num(), 6);
		}
		{
			TArray<FConvex::FVec3Type> Particles;
			Particles.SetNum(6);
			Particles[0] = { -1, -1, -1 };
			Particles[1] = { 1, -1, -1 };
			Particles[2] = { 1, 1, -1 };
			Particles[3] = { 0, 0, 0.5 };
			Particles[4] = (Particles[3] - Particles[1]) * 0.5 + Particles[1] + FConvex::FVec3Type{ 0, 0, 0.1f };
			Particles[5] = Particles[4] + FConvex::FVec3Type{ -0.1f, 0, 0 };
			TArray<TVector<int32, 3>> Indices;
			Chaos::FConvexBuilder::BuildConvexHull(Particles, Indices);
			EXPECT_EQ(Indices.Num(), 8);
		}
		{
			// This is a specific case where without coplaner face merging and
			// a large enough epsilon for building horizons in hull generation
			// (tested to fail with 1e-1) we will generate a non-convex hull
			// Using a scaled epsilon resolves this case
			TArray<FConvex::FVec3Type> Particles;
			Particles.SetNum(9);
			Particles[0] = { -1, -1, -1 };
			Particles[1] = { -1, -1, 1 };
			Particles[2] = { -1, 1, -1 };
			Particles[3] = { -1, 1, 1 };
			Particles[4] = { 1, -1, -1 };
			Particles[5] = { 1, -1, 1 };
			Particles[6] = { 1, 1, -1 };
			Particles[7] = { 1, 1, 1 };
			Particles[8] = { 0.966962576f, -0.0577232838f, 0.959515572f };
			
			TArray<TVec3<int32>> Indices;
			Chaos::FConvexBuilder::Params BuildParams;
			BuildParams.HorizonEpsilon = Chaos::FConvexBuilder::SuggestEpsilon(Particles);

			Chaos::FConvexBuilder::BuildConvexHull(Particles, Indices, BuildParams);

			EXPECT_EQ(Indices.Num(), 12);

			for (const TVec3<int32>& Tri : Indices)
			{
				for (int32 i = 0; i < 3; ++i)
				{
					FVec3 V = Particles[Tri[i]];
					FVec3 VAbs = V.GetAbs();
					FReal Max = VAbs.GetMax();
					EXPECT_GE(Max, 1 - 1e-2);
				}
			}
		}
		{
			// Build a box and fill it with many other points. Correct hull generation should produce
			// only the original box - ignoring all interior and coplanar points.
			// Note: If hull generation is changed to support non-triangular faces the conditions here
			// will need to change as a correct hull in that method will produce only 6 faces not 12
			TArray<FConvex::FVec3Type> Particles;
			int32 NumParticles = 3600;
			Particles.SetNum(NumParticles);
			Particles[0] = { -1, -1, -1 };
			Particles[1] = { -1, -1, 1 };
			Particles[2] = { -1, 1, -1 };
			Particles[3] = { -1, 1, 1 };
			Particles[4] = { 1, -1, -1 };
			Particles[5] = { 1, -1, 1 };
			Particles[6] = { 1, 1, -1 };
			Particles[7] = { 1, 1, 1 };
			FRandomStream Stream(42);
			for(int i = 8; i < NumParticles; ++i)
			{
				Particles[i]= Chaos::FVec3(Stream.FRandRange(-1.f, 1.f), Stream.FRandRange(-1.f, 1.f), Stream.FRandRange(-1.f, 1.f));
			}
			TArray<TVec3<int32>> Indices;

			Chaos::FConvexBuilder::Params BuildParams;
			BuildParams.HorizonEpsilon = Chaos::FConvexBuilder::SuggestEpsilon(Particles);

			Chaos::FConvexBuilder::BuildConvexHull(Particles, Indices, BuildParams);

			EXPECT_EQ(Indices.Num(), 12);
			for(auto Tri : Indices)
			{
				for(int i = 0; i < 3; ++i)
				{
					FVec3 V = Particles[Tri[i]];
					FVec3 VAbs = V.GetAbs();
					FReal Max = VAbs.GetMax();
					EXPECT_GE(Max, 1 - 1e-2);
				}
			}
		}
	}

	void Simplify()
	{
		TArray<FConvex::FVec3Type> Particles;
		Particles.SetNum(18);
		Particles[0] = { 0, 0, 12.0f };
		Particles[1] = { -0.707f, -0.707f, 10.0f };
		Particles[2] = { 0, -1, 10.0f };
		Particles[3] = { 0.707f, -0.707f, 10.0f };
		Particles[4] = { 1, 0, 10.0f };
		Particles[5] = { 0.707f, 0.707f, 10.0f };
		Particles[6] = { 0.0f, 1.0f, 10.0f };
		Particles[7] = { -0.707f, 0.707f, 10.0f };
		Particles[8] = { -1.0f, 0.0f, 10.0f };
		Particles[9] = { -0.707f, -0.707f, 0.0f };
		Particles[10] = { 0, -1, 0.0f };
		Particles[11] = { 0.707f, -0.707f, 0.0f };
		Particles[12] = { 1, 0, 0.0f };
		Particles[13] = { 0.707f, 0.707f, 0.0f };
		Particles[14] = { 0.0f, 1.0f, 0.0f };
		Particles[15] = { -0.707f, 0.707f, 0.0f };
		Particles[16] = { -1.0f, 0.0f, 0.0f };
		Particles[17] = { 0, 0, -2.0f };
					
		FConvex Convex(Particles, 0.0f);

		// capture original details
		int32 OriginalNumberParticles = Convex.NumVertices();
		int32 OriginalNumberFaces = Convex.GetFaces().Num();
		FAABB3 OriginalBoundingBox = Convex.BoundingBox();

		const TArray<FConvex::FVec3Type>& CulledParticles = Convex.GetVertices();
		const TArray<FConvex::FPlaneType> Planes = Convex.GetFaces();

		// set target number of particles in simplified convex
		FConvexBuilder::PerformGeometryReduction = 1;
		FConvexBuilder::VerticesThreshold = 10;

		// simplify
		Convex.PerformanceWarningAndSimplifaction();

		// capture new details
		int32 NewNumberParticles = Convex.NumVertices();
		int32 NewNumberFaces = Convex.GetFaces().Num();
		FAABB3 NewBoundingBox = Convex.BoundingBox();

		EXPECT_EQ(OriginalNumberParticles, 18);
		EXPECT_EQ(NewNumberParticles, 10);
		EXPECT_LT(NewNumberFaces, OriginalNumberFaces);

		FVec3 DiffMin = OriginalBoundingBox.Min() - NewBoundingBox.Min();
		FVec3 DiffMax = OriginalBoundingBox.Max() - NewBoundingBox.Max();

		// bounding box won't be identical, so long as it's not too far out
		for (int Idx=0; Idx<3; Idx++)
		{
			EXPECT_LT(FMath::Abs(DiffMin[Idx]), 0.15f);
			EXPECT_LT(FMath::Abs(DiffMax[Idx]), 0.15f);
		}

		FConvexBuilder::PerformGeometryReduction = 0;
	}
	
		void ImplicitScaled2()
	{
		// Note: Margins are internal and should not impact Phi or Support calculations.
		// Specifically for spheres, which are represented as a core point with margin equal to the
		// radius, the margin cannot be increased and any margin "added" by a wrapper shape like
		// ImplicitObjectScaled is ignored.
		FReal Thickness = 0.1;
		FSpherePtr Sphere( new TSphere<FReal,3>(FVec3(3, 0, 0), 5));
		TImplicitObjectScaled<Chaos::FSphere> Unscaled(Sphere, FVec3(1));
		TImplicitObjectScaled<Chaos::FSphere> UnscaledThickened(Sphere, FVec3(1), Thickness);
		TImplicitObjectScaled<Chaos::FSphere> UniformScale(Sphere, FVec3(2));
		TImplicitObjectScaled<Chaos::FSphere> UniformScaleThickened(Sphere, FVec3(2), Thickness);
		TImplicitObjectScaled<Chaos::FSphere> NonUniformScale(Sphere, FVec3(2, 1, 1));
		TImplicitObjectScaled<Chaos::FSphere> NonUniformScaleThickened(Sphere, FVec3(2, 1, 1), Thickness);

		//phi
		{
			const FVec3 NearEdge(7.5, 0, 0);
			FVec3 UnscaledNormal;
			const FReal UnscaledPhi = Unscaled.PhiWithNormal(NearEdge, UnscaledNormal);
			EXPECT_FLOAT_EQ(UnscaledPhi, -0.5);
			EXPECT_FLOAT_EQ(UnscaledNormal[0], 1);
			EXPECT_FLOAT_EQ(UnscaledNormal[1], 0);
			EXPECT_FLOAT_EQ(UnscaledNormal[2], 0);

			FVec3 UnscaledNormalThickened;
			const FReal UnscaledThickenedPhi = UnscaledThickened.PhiWithNormal(NearEdge, UnscaledNormalThickened);
			EXPECT_FLOAT_EQ(UnscaledThickenedPhi, -0.5);
			EXPECT_FLOAT_EQ(UnscaledNormalThickened[0], 1);
			EXPECT_FLOAT_EQ(UnscaledNormalThickened[1], 0);
			EXPECT_FLOAT_EQ(UnscaledNormalThickened[2], 0);

			FVec3 ScaledNormal;
			FReal ScaledPhi = UniformScale.PhiWithNormal(NearEdge, ScaledNormal);
			EXPECT_FLOAT_EQ(ScaledPhi, -(16 - 7.5));
			EXPECT_FLOAT_EQ(ScaledNormal[0], 1);
			EXPECT_FLOAT_EQ(ScaledNormal[1], 0);
			EXPECT_FLOAT_EQ(ScaledNormal[2], 0);

			FVec3 ScaledNormalThickened;
			FReal ScaledPhiThickened = UniformScaleThickened.PhiWithNormal(NearEdge, ScaledNormalThickened);
			EXPECT_FLOAT_EQ(ScaledPhiThickened, -(16 - 7.5));
			EXPECT_FLOAT_EQ(ScaledNormalThickened[0], 1);
			EXPECT_FLOAT_EQ(ScaledNormalThickened[1], 0);
			EXPECT_FLOAT_EQ(ScaledNormalThickened[2], 0);

			const FVec3 NearTop(6, 0, 4.5);
			ScaledPhi = UniformScale.PhiWithNormal(NearTop, ScaledNormal);
			EXPECT_FLOAT_EQ(ScaledPhi, -(10-4.5));
			EXPECT_FLOAT_EQ(ScaledNormal[0], 0);
			EXPECT_FLOAT_EQ(ScaledNormal[1], 0);
			EXPECT_FLOAT_EQ(ScaledNormal[2], 1);

			ScaledPhiThickened = UniformScaleThickened.PhiWithNormal(NearTop, ScaledNormalThickened);
			EXPECT_FLOAT_EQ(ScaledPhiThickened, -(10 - 4.5));
			EXPECT_FLOAT_EQ(ScaledNormalThickened[0], 0);
			EXPECT_FLOAT_EQ(ScaledNormalThickened[1], 0);
			EXPECT_FLOAT_EQ(ScaledNormalThickened[2], 1);

			ScaledPhi = NonUniformScale.PhiWithNormal(NearTop, ScaledNormal);
			EXPECT_FLOAT_EQ(ScaledPhi, -0.5);
			EXPECT_FLOAT_EQ(ScaledNormal[0], 0);
			EXPECT_FLOAT_EQ(ScaledNormal[1], 0);
			EXPECT_FLOAT_EQ(ScaledNormal[2], 1);

			ScaledPhiThickened = NonUniformScaleThickened.PhiWithNormal(NearTop, ScaledNormalThickened);
			EXPECT_FLOAT_EQ(ScaledPhiThickened, -0.5);
			EXPECT_FLOAT_EQ(ScaledNormalThickened[0], 0);
			EXPECT_FLOAT_EQ(ScaledNormalThickened[1], 0);
			EXPECT_FLOAT_EQ(ScaledNormalThickened[2], 1);

			ScaledPhiThickened = NonUniformScaleThickened.PhiWithNormal(NearEdge, ScaledNormalThickened);
			EXPECT_FLOAT_EQ(ScaledPhiThickened, -(16 - 7.5));
			EXPECT_FLOAT_EQ(ScaledNormalThickened[0], 1);
			EXPECT_FLOAT_EQ(ScaledNormalThickened[1], 0);
			EXPECT_FLOAT_EQ(ScaledNormalThickened[2], 0);

		}

		//support
		{
			int32 VertexIndex = INDEX_NONE;
			const FVec3 DirX(1, 0, 0);
			FVec3 SupportPt = Unscaled.Support(DirX, 1, VertexIndex);
			EXPECT_FLOAT_EQ(SupportPt[0], 9);
			EXPECT_FLOAT_EQ(SupportPt[1], 0);
			EXPECT_FLOAT_EQ(SupportPt[2], 0);

			SupportPt = UnscaledThickened.Support(DirX, 1, VertexIndex);
			EXPECT_FLOAT_EQ(SupportPt[0], 9);
			EXPECT_FLOAT_EQ(SupportPt[1], 0);
			EXPECT_FLOAT_EQ(SupportPt[2], 0);

			SupportPt = UniformScale.Support(DirX, 1, VertexIndex);
			EXPECT_FLOAT_EQ(SupportPt[0], 17);
			EXPECT_FLOAT_EQ(SupportPt[1], 0);
			EXPECT_FLOAT_EQ(SupportPt[2], 0);

			SupportPt = UniformScaleThickened.Support(DirX, 1, VertexIndex);
			EXPECT_FLOAT_EQ(SupportPt[0], 17);
			EXPECT_FLOAT_EQ(SupportPt[1], 0);
			EXPECT_FLOAT_EQ(SupportPt[2], 0);

			const FVec3 DirZ(0, 0, -1);
			SupportPt = UniformScale.Support(DirZ, 1, VertexIndex);
			EXPECT_FLOAT_EQ(SupportPt[0], 6);
			EXPECT_FLOAT_EQ(SupportPt[1], 0);
			EXPECT_FLOAT_EQ(SupportPt[2], -11);

			SupportPt = UniformScaleThickened.Support(DirZ, 1, VertexIndex);
			EXPECT_FLOAT_EQ(SupportPt[0], 6);
			EXPECT_FLOAT_EQ(SupportPt[1], 0);
			EXPECT_FLOAT_EQ(SupportPt[2], -11);

			SupportPt = NonUniformScale.Support(DirX, 1, VertexIndex);
			EXPECT_FLOAT_EQ(SupportPt[0], 17);
			EXPECT_FLOAT_EQ(SupportPt[1], 0);
			EXPECT_FLOAT_EQ(SupportPt[2], 0);

			SupportPt = NonUniformScaleThickened.Support(DirX, 1, VertexIndex);
			EXPECT_FLOAT_EQ(SupportPt[0], 17);
			EXPECT_FLOAT_EQ(SupportPt[1], 0);
			EXPECT_FLOAT_EQ(SupportPt[2], 0);

			SupportPt = NonUniformScale.Support(DirZ, 1, VertexIndex);
			EXPECT_FLOAT_EQ(SupportPt[0], 6);
			EXPECT_FLOAT_EQ(SupportPt[1], 0);
			EXPECT_FLOAT_EQ(SupportPt[2], -6);

			SupportPt = NonUniformScaleThickened.Support(DirZ, 1, VertexIndex);
			EXPECT_FLOAT_EQ(SupportPt[0], 6);
			EXPECT_FLOAT_EQ(SupportPt[1], 0);
			EXPECT_FLOAT_EQ(SupportPt[2], -6);
		}
	}

	void UpdateImplicitUnion()
	{
		TArray<Chaos::FImplicitObjectPtr> Objects;
		Objects.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(0, 0, 1), FVec3(0), 1));
		Objects.Add(MakeImplicitObjectPtr<FCylinder>(FVec3(0, 0, -1), FVec3(0), 1));
		
		FImplicitObjectUnionPtr MUnionedObjects(new Chaos::FImplicitObjectUnion(std::move(Objects)));

		TArray<Chaos::FImplicitObjectPtr> Objects2;
		Objects2.Add(MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(4, 0, 0), 1));
		Objects2.Add(MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(5, 0, 0), 2));
		Objects2.Add(MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(10, 0, 0), 3));

		const FAABB3 OriginalBounds = MUnionedObjects->BoundingBox();

		EXPECT_EQ(MUnionedObjects->GetObjects().Num(), 2);
		EXPECT_FLOAT_EQ(OriginalBounds.Extents().X, 2.f);
		EXPECT_FLOAT_EQ(OriginalBounds.Extents().Y, 2.f);
		EXPECT_FLOAT_EQ(OriginalBounds.Extents().Z, 4.f);

		MUnionedObjects->Combine(Objects2);

		EXPECT_EQ(MUnionedObjects->GetObjects().Num(), 5);
		const FAABB3 CombinedBounds = MUnionedObjects->BoundingBox();
		EXPECT_FLOAT_EQ(CombinedBounds.Extents().X, 14.f);
		EXPECT_FLOAT_EQ(CombinedBounds.Extents().Y, 6.f);
		EXPECT_FLOAT_EQ(CombinedBounds.Extents().Z, 6.f);

		MUnionedObjects->RemoveAt(1);
		MUnionedObjects->RemoveAt(0);

		EXPECT_EQ(MUnionedObjects->GetObjects().Num(), 3);
		const FAABB3 RemovedBounds = MUnionedObjects->BoundingBox();
		EXPECT_FLOAT_EQ(RemovedBounds.Extents().X, 10.f);
		EXPECT_FLOAT_EQ(RemovedBounds.Extents().Y, 6.f);
		EXPECT_FLOAT_EQ(RemovedBounds.Extents().Z, 6.f);

	}

	GTEST_TEST(ImplicitTests, TestImplicitCasts)
	{
		TUniquePtr<FImplicitObject> Sphere = MakeUnique<FImplicitSphere3>(FVec3(0), 100.0);
		const FImplicitObject* ConstSphere = Sphere.Get();

		TArray<FImplicitObjectPtr> UnionObjects;
		UnionObjects.Emplace(MakeImplicitObjectPtr<FImplicitSphere3>(FVec3(0), 100.0));

		TArray<FImplicitObjectPtr> UnionClusteredObjects;
		UnionClusteredObjects.Emplace(MakeImplicitObjectPtr<FImplicitSphere3>(FVec3(0), 100.0));

		FImplicitObjectPtr Union = MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(UnionObjects));
		FImplicitObjectPtr UnionClustered = MakeImplicitObjectPtr<FImplicitObjectUnionClustered>(MoveTemp(UnionClusteredObjects));

		// We can cast an implicit object to its exact type (const and non-const)
		FImplicitSphere3* SphereCast = Sphere->AsA<FImplicitSphere3>();
		EXPECT_NE(SphereCast, nullptr);

		FImplicitSphere3* SphereCastChecked = Sphere->AsAChecked<FImplicitSphere3>();
		EXPECT_NE(SphereCastChecked, nullptr);

		const FImplicitSphere3* ConstSphereCast = ConstSphere->AsA<FImplicitSphere3>();
		EXPECT_NE(ConstSphereCast, nullptr);

		const FImplicitSphere3* ConstSphereCastChecked = ConstSphere->AsAChecked<FImplicitSphere3>();
		EXPECT_NE(ConstSphereCastChecked, nullptr);

		FImplicitObjectUnion* UnionCast = Union->AsA<FImplicitObjectUnion>();
		EXPECT_NE(UnionCast, nullptr);

		FImplicitObjectUnionClustered* UnionClusteredCast = UnionClustered->AsA<FImplicitObjectUnionClustered>();
		EXPECT_NE(UnionClusteredCast, nullptr);

		// We can cast a Clustered Union to a Union
		FImplicitObjectUnion* UnionClusteredUpCast = UnionClustered->AsA<FImplicitObjectUnion>();
		EXPECT_NE(UnionClusteredUpCast, nullptr);

	}
}
