// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HeadlessChaosTestConstraints.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/Box.h"
#include "Chaos/Utilities.h"

namespace ChaosTest
{
	using namespace Chaos;

	/**
	 * Base class for simple joint chain tests.
	 */
	template <typename TEvolution>
	class FJointChainTest : public FConstraintsTest<TEvolution>
	{
	public:
		using Base = FConstraintsTest<TEvolution>;
		using Base::Evolution;
		using Base::AddParticleBox;
		using Base::GetParticle;

		FJointChainTest(const int32 NumIterations, const FReal Gravity)
			: Base(NumIterations, Gravity)
		{
		}

		FPBDJointConstraintHandle* AddJoint(const TVec2<TGeometryParticleHandle<FReal, 3>*>& InConstrainedParticleIndices, const int32 JointIndex)
		{
			
			FPBDJointConstraintHandle* Joint = nullptr;
				
			//Unless someone sets the joint to use nonlinear solver beforehand, otherwise it will default to linear solver. 
			if (JointIndex < JointSettings.Num())
			{
				if (!JointSettings[JointIndex].bUseLinearSolver)
				{
					Joint = Evolution.GetJointCombinedConstraints().NonLinearConstraints.AddConstraint(InConstrainedParticleIndices, FRigidTransform3(JointPositions[JointIndex], FRotation3::FromIdentity()));
				}
			}
			if (!Joint)
			{
				Joint = Evolution.GetJointCombinedConstraints().LinearConstraints.AddConstraint(InConstrainedParticleIndices, FRigidTransform3(JointPositions[JointIndex], FRotation3::FromIdentity()));
			}

			// @todo(chaos): this indicates we need to change the AddConstraint API (since ConnectorTransforms were added to Settings). 
			// Calling AddConstraint followed by SetSettings will overwrite the ConnectorTransforms
			JointSettings[JointIndex].ConnectorTransforms = Joint->GetSettings().ConnectorTransforms;

			if (JointIndex < JointSettings.Num())
			{
				Joint->SetSettings(JointSettings[JointIndex]);
			}

			return Joint;
		}

		FPBDJointConstraintHandle* GetJoint(const int32 JointIndex)
		{
			if (JointIndex < JointSettings.Num())
			{
				if (!JointSettings[JointIndex].bUseLinearSolver)
				{
					return Evolution.GetJointCombinedConstraints().NonLinearConstraints.GetConstConstraintHandles()[JointIndex];
				}
			}

			return Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstConstraintHandles()[JointIndex];
		}

		void Create()
		{
			for (int32 ParticleIndex = 0; ParticleIndex < ParticlePositions.Num(); ++ParticleIndex)
			{
				AddParticleBox(ParticlePositions[ParticleIndex], FRotation3::MakeFromEuler(FVec3(0.f, 0.f, 0.f)).GetNormalized(), ParticleSizes[ParticleIndex], ParticleMasses[ParticleIndex]);
			}

			for (int32 JointIndex = 0; JointIndex < JointPositions.Num(); ++JointIndex)
			{
				const TVec2<TGeometryParticleHandle<FReal, 3>*> ConstraintedParticleIds(GetParticle(JointParticleIndices[JointIndex][0]), GetParticle(JointParticleIndices[JointIndex][1]));
				AddJoint(ConstraintedParticleIds, JointIndex);
			}
		}

		// Create a pendulum chain along the specified direction with the first particle kinematic
		void InitChain(int32 NumParticles, const FVec3& Dir, FReal Size = 10.f, FReal Separation = 30.f)
		{
			for (int32 ParticleIndex = 0; ParticleIndex < NumParticles; ++ParticleIndex)
			{
				FReal D = ParticleIndex * Separation;
				FReal M = (ParticleIndex == 0) ? 0.0f : 100.0f;
				ParticlePositions.Add(D * Dir);
				ParticleSizes.Add(FVec3(Size));
				ParticleMasses.Add(M);
			}

			for (int32 JointIndex = 0; JointIndex < NumParticles - 1; ++JointIndex)
			{
				int32 ParticleIndex0 = JointIndex;
				int32 ParticleIndex1 = JointIndex + 1;
				FReal D = JointIndex * Separation;
				JointPositions.Add(D * Dir);
				JointParticleIndices.Add(TVec2<int32>(ParticleIndex0, ParticleIndex1));
			}

			JointSettings.SetNum(NumParticles - 1);
		}

		void Advance(const FReal Dt)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		// Initial particles setup
		TArray<FVec3> ParticlePositions;
		TArray<FVec3> ParticleSizes;
		TArray<FReal> ParticleMasses;

		// Initial joints setup
		TArray<FVec3> JointPositions;
		TArray<TVec2<int32>> JointParticleIndices;
		TArray<FPBDJointSettings> JointSettings;
	};

}