// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestJoint.h"
#include "Modules/ModuleManager.h"

namespace ChaosTest {

	using namespace Chaos;

	// Set up a test with a non-breakable joint, then manually break it.
	// Verify that the break callback is called and the joint is disabled.
	template <typename TEvolution, bool bUseSimd = false>
	void JointBreak_ManualBreak()
	{
		const int32 NumIterations = 1;
		const FReal Gravity = 980;
		const FReal Dt = 0.01f;
		int32 NumIts = 10;

		FJointChainTest<TEvolution> Test(NumIterations, Gravity);
		Test.InitChain(2, FVec3(0,0,-1));
		Test.Create();
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

		bool bBrokenCallbackCalled = false;
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetBreakCallback([&bBrokenCallbackCalled](FPBDJointConstraintHandle* Constraint)
			{
				bBrokenCallbackCalled = true;
			});

		// Run the sim - nothing should move
		for (int32 i = 0; i < NumIts; ++i)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);
		}
		EXPECT_NEAR(Test.GetParticle(1)->GetX().Z, Test.ParticlePositions[1].Z, 1.0f);

		// Nothing should have broken
		EXPECT_FALSE(bBrokenCallbackCalled);
		EXPECT_TRUE(Test.Evolution.GetJointCombinedConstraints().LinearConstraints.IsConstraintEnabled(0));

		// Manually break the constraints
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.BreakConstraint(0);

		// Check that it worked
		EXPECT_TRUE(bBrokenCallbackCalled);
		EXPECT_FALSE(Test.Evolution.GetJointCombinedConstraints().LinearConstraints.IsConstraintEnabled(0));

		// Run the sim - body should fall
		for (int32 i = 0; i < NumIts; ++i)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);
		}
		FReal ExpectedZ = Test.ParticlePositions[1].Z - 0.5f * Gravity * (NumIts * Dt) * (NumIts * Dt);
		EXPECT_NEAR(Test.GetParticle(1)->GetX().Z, ExpectedZ, 1.0f);
	}

	GTEST_TEST(AllEvolutions, JointBreakTests_TestManualBreak)
	{
		JointBreak_ManualBreak<FPBDRigidsEvolutionGBF, false>();
		JointBreak_ManualBreak<FPBDRigidsEvolutionGBF, true>();
	}

	// 1 Kinematic Body with 1 Dynamic body hanging from it by a breakable constraint.
	// Constraint break force is larger than M x G, so joint should not break.
	template <typename TEvolution, bool bUseSimd = false>
	void JointBreak_UnderLinearThreshold()
	{
		const int32 NumIterations = 1;
		const FReal Gravity = 980;
		const FReal Dt = 0.01f;
		int32 NumIts = 10;

		FJointChainTest<TEvolution> Test(NumIterations, Gravity);
		Test.InitChain(2, FVec3(0, 0, -1));
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);
		// Joint should break only if Threshold < MG
		// So not in this test
		Test.JointSettings[0].LinearBreakForce = 1.1f * Test.ParticleMasses[1] * Gravity;

		Test.Create();

		bool bBrokenCallbackCalled = false;
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetBreakCallback([&bBrokenCallbackCalled](FPBDJointConstraintHandle* Constraint)
			{
				bBrokenCallbackCalled = true;
			});

		// Run the sim
		for (int32 i = 0; i < NumIts; ++i)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);
		}

		// Nothing should have broken
		EXPECT_FALSE(bBrokenCallbackCalled);
		EXPECT_TRUE(Test.Evolution.GetJointCombinedConstraints().LinearConstraints.IsConstraintEnabled(0));
	}

	GTEST_TEST(AllEvolutions, JointBreakTests_TestUnderLinearThreshold)
	{
		JointBreak_UnderLinearThreshold<FPBDRigidsEvolutionGBF, false>();
		JointBreak_UnderLinearThreshold<FPBDRigidsEvolutionGBF, true>();
	}

	// 1 Kinematic Body with 2 Dynamic bodies hanging from it by a breakable constraint.
	// Constraint break forces are larger than M x G, so joint should not break.
	template <typename TEvolution, bool bUseSimd>
	void JointBreak_UnderLinearThreshold2()
	{
		const int32 NumIterations = 1;
		const FReal Gravity = 980;
		const FReal Dt = 0.01f;
		int32 NumIts = 10;

		FJointChainTest<TEvolution> Test(NumIterations, Gravity);
		Test.InitChain(3, FVec3(0, 0, -1));
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

		// Joint should break only if Threshold < MG
		// So not in this test
		// NOTE: internal forces reach almost 50% over MG
		Test.JointSettings[0].LinearBreakForce = 1.5f * (Test.ParticleMasses[1] + Test.ParticleMasses[2]) * Gravity;
		Test.JointSettings[1].LinearBreakForce = 1.5f * Test.ParticleMasses[2] * Gravity;

		Test.Create();

		bool bBrokenCallbackCalled = false;
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetBreakCallback([&bBrokenCallbackCalled](FPBDJointConstraintHandle* Constraint)
			{
				bBrokenCallbackCalled = true;
			});

		// Run the sim
		for (int32 i = 0; i < NumIts; ++i)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);
		}

		// Nothing should have broken
		EXPECT_FALSE(bBrokenCallbackCalled);
		EXPECT_TRUE(Test.Evolution.GetJointCombinedConstraints().LinearConstraints.IsConstraintEnabled(0));
		EXPECT_TRUE(Test.Evolution.GetJointCombinedConstraints().LinearConstraints.IsConstraintEnabled(1));
	}

	GTEST_TEST(AllEvolutions, JointBreakTests_TestUnderLinearThreshold2)
	{
		JointBreak_UnderLinearThreshold2<FPBDRigidsEvolutionGBF, false>();
		JointBreak_UnderLinearThreshold2<FPBDRigidsEvolutionGBF, true>();
	}

	// 1 Kinematic Body with 1 Dynamic body hanging from it by a breakable constraint.
	// Constraint break force is less than M x G, so joint should break.
	template <typename TEvolution, bool bUseSimd>
	void JointBreak_OverLinearThreshold()
	{
		const int32 NumIterations = 1;
		const FReal Gravity = 980;
		const FReal Dt = 0.01f;
		int32 NumIts = 10;

		FJointChainTest<TEvolution> Test(NumIterations, Gravity);
		Test.InitChain(2, FVec3(0,0,-1));
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);
		// Joint should break only if Threshold < MG
		// So yes in this test
		Test.JointSettings[0].LinearBreakForce = 0.9f * Test.ParticleMasses[1] * Gravity;

		Test.Create();

		bool bBrokenCallbackCalled = false;
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetBreakCallback([&bBrokenCallbackCalled](FPBDJointConstraintHandle* Constraint)
			{
				bBrokenCallbackCalled = true;
			});

		// Run the sim
		for (int32 i = 0; i < NumIts; ++i)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);
		}

		// Constraint should have broken
		EXPECT_TRUE(bBrokenCallbackCalled);
		EXPECT_FALSE(Test.Evolution.GetJointCombinedConstraints().LinearConstraints.IsConstraintEnabled(0));
	}

	GTEST_TEST(AllEvolutions, JointBreakTests_TestOverLinearThreshold)
	{
		JointBreak_OverLinearThreshold<FPBDRigidsEvolutionGBF, false>();
		JointBreak_OverLinearThreshold<FPBDRigidsEvolutionGBF, true>();
	}


	// 1 Kinematic Body with 2 Dynamic bodies hanging from it by a breakable constraint.
	// Constraint break force is less than M x G, so joint should not break.
	template <typename TEvolution, bool bUseSimd = false>
	void JointBreak_UnderLinearThreshold3()
	{
		const int32 NumIterations = 1;
		const FReal Gravity = 980;
		const FReal Dt = 0.01f;
		int32 NumIts = 10;

		FJointChainTest<TEvolution> Test(NumIterations, Gravity);
		Test.InitChain(3, FVec3(0, 0, -1));
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

		// Joint should break only if Threshold < MG
		// So no in this test
		Test.JointSettings[0].LinearBreakForce = 1.2f * (Test.ParticleMasses[1] + Test.ParticleMasses[2]) * Gravity;
		Test.JointSettings[1].LinearBreakForce = 1.2f * Test.ParticleMasses[2] * Gravity;

		Test.Create();

		bool bBrokenCallbackCalled = false;
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetBreakCallback([&bBrokenCallbackCalled](FPBDJointConstraintHandle* Constraint)
			{
				bBrokenCallbackCalled = true;
			});

		// Run the sim
		for (int32 i = 0; i < NumIts; ++i)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);
		}

		// Constraint should not have broken
		EXPECT_FALSE(bBrokenCallbackCalled);
		EXPECT_TRUE(Test.Evolution.GetJointCombinedConstraints().LinearConstraints.IsConstraintEnabled(0));
		EXPECT_TRUE(Test.Evolution.GetJointCombinedConstraints().LinearConstraints.IsConstraintEnabled(1));
	}

	GTEST_TEST(AllEvolutions, JointBreakTests_TestUnderLinearThreshold3)
	{
		JointBreak_UnderLinearThreshold3<FPBDRigidsEvolutionGBF>();
		constexpr bool bUseSimd = true;
		JointBreak_UnderLinearThreshold3<FPBDRigidsEvolutionGBF, bUseSimd>();
	}

	// 1 Kinematic Body with 1 Dynamic body held vertically by a breakable angular constraint.
	// Constraint break torque is larger than input torque so constraint will not break.
	template <typename TEvolution, bool bUseSimd>
	void JointBreak_UnderAngularThreshold()
	{
		const int32 NumIterations = 1;
		const FReal Gravity = 980;
		const FReal Dt = 0.01f;
		int32 NumIts = 10;
		const FVec3 Torque = FVec3(10000, 0.0f, 0.0f);

		FJointChainTest<TEvolution> Test(NumIterations, Gravity);
		Test.InitChain(2, FVec3(0, 0, -1));
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);
		// Joint should break only if Threshold < MGL
		// So not in this test
		Test.JointSettings[0].AngularBreakTorque = 1.1f * Torque.X;
		Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Locked, EJointMotionType::Locked , EJointMotionType::Locked };

		Test.Create();

		bool bBrokenCallbackCalled = false;
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetBreakCallback([&bBrokenCallbackCalled](FPBDJointConstraintHandle* Constraint)
			{
				bBrokenCallbackCalled = true;
			});

		// Run the sim
		for (int32 i = 0; i < NumIts; ++i)
		{
			Test.GetParticle(1)->CastToRigidParticle()->SetTorque(Torque);

			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);
		}

		// Nothing should have broken
		EXPECT_FALSE(bBrokenCallbackCalled);
		EXPECT_TRUE(Test.Evolution.GetJointCombinedConstraints().LinearConstraints.IsConstraintEnabled(0));
	}

	GTEST_TEST(AllEvolutions, JointBreakTests_TestUnderAngularThreshold)
	{
		JointBreak_UnderAngularThreshold<FPBDRigidsEvolutionGBF, false>();
		JointBreak_UnderAngularThreshold<FPBDRigidsEvolutionGBF, true>();
	}

	// 1 Kinematic Body with 1 Dynamic body held vertically by a breakable angular constraint.
	// Constraint break torque is less than input torque so constraint will break.
	template <typename TEvolution, bool bUseSimd>
	void JointBreak_OverAngularThreshold()
	{
		const int32 NumSolverIterations = 10;
		const FReal Gravity = 980;
		const FReal Dt = 0.01f;
		int32 NumSteps = 10;
		const FVec3 Torque = FVec3(10000, 0.0f, 0.0f);

		FJointChainTest<TEvolution> Test(NumSolverIterations, Gravity);
		Test.InitChain(2, FVec3(0, 0, -1));
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);
		// Joint should break only if Threshold < MGL
		// So not in this test
		Test.JointSettings[0].AngularBreakTorque = 0.9f * Torque.X;
		Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Locked, EJointMotionType::Locked , EJointMotionType::Locked };

		Test.Create();

		bool bBrokenCallbackCalled = false;
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetBreakCallback([&bBrokenCallbackCalled](FPBDJointConstraintHandle* Constraint)
			{
				bBrokenCallbackCalled = true;
			});

		// Run the sim
		for (int32 i = 0; i < NumSteps; ++i)
		{
			Test.GetParticle(1)->CastToRigidParticle()->SetTorque(Torque);

			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);
		}

		// Nothing should have broken
		EXPECT_TRUE(bBrokenCallbackCalled);
		EXPECT_FALSE(Test.Evolution.GetJointCombinedConstraints().LinearConstraints.IsConstraintEnabled(0));
	}

	GTEST_TEST(AllEvolutions, JointBreakTests_TestOverAngularThreshold)
	{
		JointBreak_OverAngularThreshold<FPBDRigidsEvolutionGBF, false>();
		JointBreak_OverAngularThreshold<FPBDRigidsEvolutionGBF, true>();
	}

}