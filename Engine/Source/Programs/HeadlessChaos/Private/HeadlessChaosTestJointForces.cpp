// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestJoint.h"
#include "Modules/ModuleManager.h"
#include "Chaos/ParticleHandle.h"

namespace ChaosTest {

	using namespace Chaos;

	// 1 Kinematic Body with 4 Dynamic bodies hanging from it by a breakable constraint.
	// Verify that the force in each joint settles to about (Sum of Child Mass)xG
	template <typename TEvolution>
	void JointForces_Linear()
	{
		for (int32 IndexSimd = 0; IndexSimd < 2; IndexSimd++)
		{
			const bool bUseSimd = IndexSimd == 0;
			const int32 NumSolverIterations = 10;
			const FReal Gravity = 980;
			const FReal Dt = 0.01f;
			const int32 NumSteps = 100;
			const int32 NumBodies = 5;

			FJointChainTest<TEvolution> Test(NumSolverIterations, Gravity);
			Test.InitChain(NumBodies, FVec3(0, 0, -1));
			Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);
			Test.Create();

			// Run the sim
			for (int32 i = 0; i < NumSteps; ++i)
			{
				Test.Evolution.AdvanceOneTimeStep(Dt);
				Test.Evolution.EndFrame(Dt);
			}

			for (int32 JointIndex = 0; JointIndex < Test.Evolution.GetJointCombinedConstraints().LinearConstraints.NumConstraints(); ++JointIndex)
			{
				FReal ChildMass = 0.0f;
				for (int32 ChildBodyIndex = JointIndex + 1; ChildBodyIndex < Test.ParticleMasses.Num(); ++ChildBodyIndex)
				{
					ChildMass += Test.ParticleMasses[ChildBodyIndex];
				}

				FVec3 ExpectedLinearImpulse = FVec3(0.0f, 0.0f, -ChildMass * Gravity * Dt);
				FVec3 LinearImpulse = Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstraintLinearImpulse(JointIndex);
				EXPECT_NEAR(LinearImpulse.X, ExpectedLinearImpulse.X, ExpectedLinearImpulse.Size() / 100.0f);
				EXPECT_NEAR(LinearImpulse.Y, ExpectedLinearImpulse.Y, ExpectedLinearImpulse.Size() / 100.0f);
				EXPECT_NEAR(LinearImpulse.Z, ExpectedLinearImpulse.Z, ExpectedLinearImpulse.Size() / 100.0f);
			}
		}
	}

	GTEST_TEST(AllEvolutions, JointForcesTests_TestLinear)
	{
		JointForces_Linear<FPBDRigidsEvolutionGBF>();
	}

	// 1 Kinematic Body with 1 Dynamic bodies arranged horizontally.
	// Verify that the force in the joint settles to about MxG
	template <typename TEvolution>
	void JointForces_Linear2()
	{
		for (int32 IndexSimd = 0; IndexSimd < 2; IndexSimd++)
		{
			const bool bUseSimd = IndexSimd == 0;
			const int32 NumSolverIterations = 40;
			const FReal Gravity = 980;
			const FReal Dt = 0.01f;
			const int32 NumSteps = 100;
			const int32 NumBodies = 3;

			FJointChainTest<TEvolution> Test(NumSolverIterations, Gravity);
			Test.InitChain(NumBodies, FVec3(1, 0, 0));
			Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

			// Set all joints to fixed angular
			for (int32 JointIndex = 0; JointIndex < Test.JointSettings.Num(); ++JointIndex)
			{
				Test.JointSettings[JointIndex].AngularMotionTypes = { EJointMotionType::Locked, EJointMotionType::Locked, EJointMotionType::Locked };
				Test.JointSettings[JointIndex].bProjectionEnabled = false;
			}

			Test.Create();

			// Run the sim
			for (int32 i = 0; i < NumSteps; ++i)
			{
				Test.Evolution.AdvanceOneTimeStep(Dt);
				Test.Evolution.EndFrame(Dt);
			}

			FReal L = Test.ParticlePositions[1].X - Test.ParticlePositions[0].X;

			for (int32 JointIndex = 0; JointIndex < Test.Evolution.GetJointCombinedConstraints().NumConstraints(); ++JointIndex)
			{
				FReal ChildMass = 0.0f;
				FReal ChildMoment = 0.0f;
				for (int32 ChildBodyIndex = JointIndex + 1; ChildBodyIndex < Test.ParticleMasses.Num(); ++ChildBodyIndex)
				{
					ChildMass += Test.ParticleMasses[ChildBodyIndex];
					FReal ChildL = (ChildBodyIndex - JointIndex) * L;
					ChildMoment += Test.ParticleMasses[ChildBodyIndex] * ChildL;
				}

				FVec3 ExpectedLinearImpulse = FVec3(0.0f, 0.0f, -ChildMass * Gravity * Dt);
				FVec3 LinearImpulse = Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstraintLinearImpulse(JointIndex);
				EXPECT_NEAR(LinearImpulse.X, ExpectedLinearImpulse.X, ExpectedLinearImpulse.Size() / 100.0f);
				EXPECT_NEAR(LinearImpulse.Y, ExpectedLinearImpulse.Y, ExpectedLinearImpulse.Size() / 100.0f);
				EXPECT_NEAR(LinearImpulse.Z, ExpectedLinearImpulse.Z, ExpectedLinearImpulse.Size() / 100.0f);

				FVec3 ExpectedAngularImpulse = FVec3(0.0f, ChildMoment * Gravity * Dt, 0.0f);
				FVec3 AngularImpulse = Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstraintAngularImpulse(JointIndex);
				EXPECT_NEAR(AngularImpulse.X, ExpectedAngularImpulse.X, ExpectedAngularImpulse.Size() / 100.0f);
				EXPECT_NEAR(AngularImpulse.Y, ExpectedAngularImpulse.Y, ExpectedAngularImpulse.Size() / 100.0f);
				EXPECT_NEAR(AngularImpulse.Z, ExpectedAngularImpulse.Z, ExpectedAngularImpulse.Size() / 100.0f);
			}
		}
	}

	GTEST_TEST(AllEvolutions, JointForcesTests_TestLinear2)
	{
		JointForces_Linear2<FPBDRigidsEvolutionGBF>();
	}


	// 1 Kinematic Body with 1 Dynamic bodies arranged vertically.
	// Apply a torque to the body and verify that the joint torque is the same.
	template <typename TEvolution>
	void JointForces_Angular()
	{
		for (int32 IndexSimd = 0; IndexSimd < 2; IndexSimd++)
		{
			bool bUseSimd = IndexSimd == 0;
			const int32 NumSolverIterations = 20;
			const FReal Gravity = 0;
			const FReal Dt = 0.01f;
			const int32 NumSteps = 100;
			const int32 NumBodies = 2;
			const FVec3 Torque = FVec3(10000, 0, 0);

			FJointChainTest<TEvolution> Test(NumSolverIterations, Gravity);
			Test.InitChain(NumBodies, FVec3(1, 0, 0));
			Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

			// Set all joints to fixed angular and disable projection
			for (int32 JointIndex = 0; JointIndex < Test.JointSettings.Num(); ++JointIndex)
			{
				Test.JointSettings[JointIndex].AngularMotionTypes = { EJointMotionType::Locked, EJointMotionType::Locked, EJointMotionType::Locked };
				Test.JointSettings[JointIndex].bProjectionEnabled = false;
			}

			Test.Create();

			// Run the sim
			for (int32 i = 0; i < NumSteps; ++i)
			{
				Test.GetParticle(1)->CastToRigidParticle()->SetTorque(Torque);

				Test.Evolution.AdvanceOneTimeStep(Dt);
				Test.Evolution.EndFrame(Dt);
			}

			for (int32 JointIndex = 0; JointIndex < Test.Evolution.GetJointCombinedConstraints().NumConstraints(); ++JointIndex)
			{
				FReal ChildMass = 0.0f;
				for (int32 ChildBodyIndex = JointIndex + 1; ChildBodyIndex < Test.ParticleMasses.Num(); ++ChildBodyIndex)
				{
					ChildMass += Test.ParticleMasses[ChildBodyIndex];
				}

				FVec3 ExpectedAngularImpulse = Torque * Dt;
				FVec3 AngularImpulse = Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstraintAngularImpulse(JointIndex);
				EXPECT_NEAR(AngularImpulse.X, ExpectedAngularImpulse.X, ExpectedAngularImpulse.Size() / 100.0f);
				EXPECT_NEAR(AngularImpulse.Y, ExpectedAngularImpulse.Y, ExpectedAngularImpulse.Size() / 100.0f);
				EXPECT_NEAR(AngularImpulse.Z, ExpectedAngularImpulse.Z, ExpectedAngularImpulse.Size() / 100.0f);
			}
		}
	}

	GTEST_TEST(AllEvolutions, JointForcesTests_Angular)
	{
		JointForces_Angular<FPBDRigidsEvolutionGBF>();
	}

	// Check that a joint drive linear stiffness calculates the correct force F=-K.X assuming implicit integration
	// NOTE: Uses Joint Force mode which isn't currently selectable from the editor
	//
	GTEST_TEST(JointForceTests, TestLinearDriveForceMode_Force)
	{
		for (int32 IndexSimd = 0; IndexSimd < 2; IndexSimd++)
		{
			bool bUseSimd = IndexSimd == 0;
			const int32 NumSolverIterations = 20;
			const FReal Gravity = 0;
			const FReal Dt = 0.01;
			const int32 NumBodies = 2;

			const FReal Extension = 10;
			const FReal Stiffness = 10000;
			const FReal Damping = 0;

			FJointChainTest<FPBDRigidsEvolutionGBF> Test(NumSolverIterations, Gravity);
			Test.InitChain(NumBodies, FVec3(0, 0, -1));
			Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

			// Disable all limits
			Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };
			Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };

			// Set up the drive in force mode
			Test.JointSettings[0].bLinearPositionDriveEnabled = { false, false, true };
			Test.JointSettings[0].bLinearVelocityDriveEnabled = { false, false, false };
			Test.JointSettings[0].LinearDriveForceMode = EJointForceMode::Force;
			Test.JointSettings[0].LinearDriveStiffness = FVec3(0, 0, Stiffness);
			Test.JointSettings[0].LinearDriveDamping = FVec3(0, 0, Damping);

			Test.Create();

			FGenericParticleHandle P1 = Test.GetParticle(1);

			// Reposition the particle to have some extension in the spring
			P1->InitTransform(P1->GetX() + FVec3(0, 0, -Extension), P1->GetR());

			// Run the sim
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			// Calculate expected force from F = -K.X with implicit integration
			const FReal M = Test.ParticleMasses[1];
			FReal ExpectedForceZ = 0;
			FReal DP = 0;
			for (int32 It = 0; It < 10; ++It)
			{
				const FReal X = -Extension + DP;
				const FReal F = -Stiffness * X;
				const FReal DV = ((F - ExpectedForceZ) / M) * Dt;
				DP += DV * Dt;
				ExpectedForceZ = F;
			}

			// Check the joint forces agree
			const FReal ForceZ = -Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstraintLinearImpulse(0).Z / Dt;
			EXPECT_NEAR(ForceZ, ExpectedForceZ, 0.01);
		}
	}

	// Check that a joint drive linear damping calculates the correct force F=-D.V assuming implicit integration
	// NOTE: Uses Joint Force mode which isn't currently selectable from the editor
	//
	GTEST_TEST(JointForceTests, TestLinearDriveForceMode_Damping)
	{
		for (int32 IndexSimd = 0; IndexSimd < 2; IndexSimd++)
		{
			bool bUseSimd = IndexSimd == 0;
			const int32 NumSolverIterations = 20;
			const FReal Gravity = 0;
			const FReal Dt = 0.01;
			const int32 NumBodies = 2;

			const FReal Velocity = 100;
			const FReal Stiffness = 0;
			const FReal Damping = 2000;

			FJointChainTest<FPBDRigidsEvolutionGBF> Test(NumSolverIterations, Gravity);
			Test.InitChain(NumBodies, FVec3(0, 0, -1));
			Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

			if (!Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetSettings().bUsePositionBasedDrives)
			{
				Test.Evolution.SetNumVelocityIterations(NumSolverIterations);
			}

			// Disable all limits
			Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };
			Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };

			// Set up the drive in force mode
			Test.JointSettings[0].bLinearPositionDriveEnabled = { false, false, false };
			Test.JointSettings[0].bLinearVelocityDriveEnabled = { false, false, true };
			Test.JointSettings[0].LinearDriveForceMode = EJointForceMode::Force;
			Test.JointSettings[0].LinearDriveStiffness = FVec3(0, 0, Stiffness);
			Test.JointSettings[0].LinearDriveDamping = FVec3(0, 0, Damping);

			Test.Create();

			FGenericParticleHandle P1 = Test.GetParticle(1);

			// Give the particle velocity so that joint damping has some work to do
			P1->SetV(FVec3(0, 0, -Velocity));

			// Run the sim
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			// Calculate expected force from F = -D.V with implicit integration
			const FReal M = Test.ParticleMasses[1];
			FReal ExpectedForceZ = 0;
			FReal DP = 0;
			for (int32 It = 0; It < NumSolverIterations; ++It)
			{
				const FReal V = -Velocity + DP / Dt;
				const FReal F = -Damping * V;
				const FReal DV = ((F - ExpectedForceZ) / M) * Dt;
				DP += DV * Dt;
				ExpectedForceZ = F;
			}

			// Check the joint forces agree
			const FReal ForceZ = -Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstraintLinearImpulse(0).Z / Dt;
			EXPECT_NEAR(ForceZ, ExpectedForceZ, 1);
		}
	}

	GTEST_TEST(JointForceTests, TestAngularDriveForceMode_Damping)
	{
		for (int32 IndexSimd = 0; IndexSimd < 2; IndexSimd++)
		{
			bool bUseSimd = IndexSimd == 0;
			const int32 NumSolverIterations = 20;
			const FReal Gravity = 0;
			const FReal Dt = 0.01;
			const int32 NumBodies = 2;

			const FReal AngularVelocity = 3;
			const FReal Stiffness = 0;
			const FReal Damping = 200;

			FJointChainTest<FPBDRigidsEvolutionGBF> Test(NumSolverIterations, Gravity);
			Test.InitChain(NumBodies, FVec3(0, 0, -1));
			Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

			if (!Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetSettings().bUsePositionBasedDrives)
			{
				Test.Evolution.SetNumVelocityIterations(NumSolverIterations);
			}

			// Disable all limits
			Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };
			Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };

			// Set up the drive in force mode
			Test.JointSettings[0].bAngularSLerpVelocityDriveEnabled = true;
			Test.JointSettings[0].AngularDriveForceMode = EJointForceMode::Force;
			Test.JointSettings[0].AngularDriveStiffness = FVec3(Stiffness, Stiffness, Stiffness);
			Test.JointSettings[0].AngularDriveDamping = FVec3(Damping, Damping, Damping);

			Test.Create();

			FGenericParticleHandle P1 = Test.GetParticle(1);

			// Give the particle angular velocity so that joint damping has some work to do
			P1->SetW(FVec3(0, 0, -AngularVelocity));

			// Run the sim
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			// Calculate expected Trque from T = -D.W with implicit integration
			const FReal I = FConstGenericParticleHandle(Test.GetParticle(1))->I().Z;
			FReal ExpectedTorqueZ = 0;
			FReal DQ = 0;
			for (int32 It = 0; It < NumSolverIterations; ++It)
			{
				const FReal W = -AngularVelocity + DQ / Dt;
				const FReal T = -Damping * W;
				const FReal DW = ((T - ExpectedTorqueZ) / I) * Dt;
				DQ += DW * Dt;
				ExpectedTorqueZ = T;
			}

			// Check the joint forces agree
			const FReal TorqueZ = -Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstraintAngularImpulse(0).Z / Dt;
			EXPECT_NEAR(TorqueZ, ExpectedTorqueZ, 0.1);
		}
	}

	// Check that a hanging mass on a joint drive reaches the correct extension with the correct spring force when using Force mode.
	// This is just a pre-test for TestLinearDriveForceMode_MaxForce to verify that everything works in the absense of a max force
	//
	GTEST_TEST(JointForceTests, TestLinearDriveForceMode_MaxForcePreTest)
	{
		for (int32 IndexSimd = 0; IndexSimd < 2; IndexSimd++)
		{
			bool bUseSimd = IndexSimd == 0;
			const int32 NumSolverIterations = 20;
			const FReal Gravity = 1000;
			const FReal Dt = 0.01;
			const int32 NumSteps = 100;
			const int32 NumBodies = 2;
			const FReal Extension = 10;

			FJointChainTest<FPBDRigidsEvolutionGBF> Test(NumSolverIterations, Gravity);
			Test.InitChain(NumBodies, FVec3(0, 0, -1));
			Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

			if (!Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetSettings().bUsePositionBasedDrives)
			{
				Test.Evolution.SetNumVelocityIterations(NumSolverIterations);
			}

			// Disable all limits
			Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };
			Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };

			// Set up the drive without a force limit
			const FReal Stiffness = Test.ParticleMasses[1] * Gravity / Extension;
			const FReal Damping = 2 * FMath::Sqrt(Stiffness * Test.ParticleMasses[1]);
			Test.JointSettings[0].bLinearPositionDriveEnabled = { false, false, true };
			Test.JointSettings[0].bLinearVelocityDriveEnabled = { false, false, true };
			Test.JointSettings[0].LinearDriveForceMode = EJointForceMode::Force;
			Test.JointSettings[0].LinearDriveStiffness = FVec3(0, 0, Stiffness);
			Test.JointSettings[0].LinearDriveDamping = FVec3(0, 0, Damping);

			Test.Create();

			FConstGenericParticleHandle P1 = Test.GetParticle(1);

			// Run the sim - the dangling box should reach a steady state
			for (int32 i = 0; i < NumSteps; ++i)
			{
				Test.Evolution.AdvanceOneTimeStep(Dt);
				Test.Evolution.EndFrame(Dt);
			}

			// We should be stationary at the desired extension
			const FReal ExpectedVZ = 0;
			const FReal ExpectedZ = Test.ParticlePositions[1].Z - Extension;
			EXPECT_NEAR(P1->V().Z, ExpectedVZ, 1);
			EXPECT_NEAR(P1->GetX().Z, ExpectedZ, 1);
		}
	}

	// Check that the maximum drive force setting honored for linear drives.
	// We repeat TestLinearDriveForceMode_MaxForcePreTest but with a max force which is less than K.X at the rest extension
	//
	GTEST_TEST(JointForceTests, TestLinearDriveForceMode_MaxForce)
	{
		for (int32 IndexUseLinear = 0; IndexUseLinear < 2; ++IndexUseLinear)
		{
			const bool bUseLinear = IndexUseLinear == 0;

			for (int32 IndexSimd = 0; IndexSimd < 2; IndexSimd++)
			{
				bool bUseSimd = IndexSimd == 0;
				const int32 NumSolverIterations = 20;
				const FReal Gravity = 1000.0;
				const FReal Dt = 0.01;
				const int32 NumSteps = 100;
				const int32 NumBodies = 2;
				const FReal Extension = 10;

				FJointChainTest<FPBDRigidsEvolutionGBF> Test(NumSolverIterations, Gravity);
				Test.InitChain(NumBodies, FVec3(0, 0, -1));

				FPBDJointConstraints& Constraints = bUseLinear ? Test.Evolution.GetJointCombinedConstraints().LinearConstraints : Test.Evolution.GetJointCombinedConstraints().NonLinearConstraints;

				Constraints.SetUseSimd(bUseSimd);

				if (!Constraints.GetSettings().bUsePositionBasedDrives)
				{
					Test.Evolution.SetNumVelocityIterations(NumSolverIterations);
				}

				// Disable all limits
				Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };
				Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };

				// Set up the drive with a force limit
				const FReal Stiffness = Test.ParticleMasses[1] * Gravity / Extension;
				const FReal Damping = 2 * FMath::Sqrt(Stiffness * Test.ParticleMasses[1]);
				Test.JointSettings[0].bLinearPositionDriveEnabled = { false, false, true };
				Test.JointSettings[0].bLinearVelocityDriveEnabled = { false, false, true };
				Test.JointSettings[0].LinearDriveForceMode = EJointForceMode::Force;
				Test.JointSettings[0].LinearDriveStiffness = FVec3(0, 0, Stiffness);
				Test.JointSettings[0].LinearDriveDamping = FVec3(0, 0, Damping);
				Test.JointSettings[0].LinearDriveMaxForce = FVec3(0, 0, 0.5 * Stiffness * Extension);
				Test.JointSettings[0].bUseLinearSolver = bUseLinear;

				Test.Create();

				FConstGenericParticleHandle P1 = Test.GetParticle(1);

				// Run the sim - the dangling box should reach a steady state
				for (int32 i = 0; i < NumSteps; ++i)
				{
					Test.Evolution.AdvanceOneTimeStep(Dt);
					Test.Evolution.EndFrame(Dt);

					const FReal ForceZ = -Constraints.GetConstraintLinearImpulse(0).Z / Dt;
					EXPECT_LT(ForceZ, Test.JointSettings[0].LinearDriveMaxForce.Z + 0.1);
				}
			}
		}
	}

	// Check that a handing mass on a joint drive reaches the correct extension with the correct spring force when using Acceleration mode.
	// This is just a pre-test for TestLinearDriveForceMode_MaxForce to verify that everything works in the absense of a max force
	// NOTE: Using Accleration mode on the drive but with adjusted stiffness and damping. 
	// Should yield same results as TestLinearDriveForceMode_MaxForcePreTest.
	//
	GTEST_TEST(JointForceTests, TestLinearDriveAccMode_MaxForcePreTest)
	{
		for (int32 IndexSimd = 0; IndexSimd < 2; IndexSimd++)
		{
			bool bUseSimd = IndexSimd == 0;
			const int32 NumSolverIterations = 20;
			const FReal Gravity = 1000;
			const FReal Dt = 0.01;
			const int32 NumSteps = 100;
			const int32 NumBodies = 2;
			const FReal Extension = 10;

			FJointChainTest<FPBDRigidsEvolutionGBF> Test(NumSolverIterations, Gravity);
			Test.InitChain(NumBodies, FVec3(0, 0, -1));
			Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

			if (!Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetSettings().bUsePositionBasedDrives)
			{
				Test.Evolution.SetNumVelocityIterations(NumSolverIterations);
			}

			// Disable all limits
			Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };
			Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };

			// Set up the drive without a force limit
			// NOTE: Acceleration mode - no masses in expressions
			const FReal Stiffness = Gravity / Extension;
			const FReal Damping = 2 * FMath::Sqrt(Stiffness);
			Test.JointSettings[0].bLinearPositionDriveEnabled = { false, false, true };
			Test.JointSettings[0].bLinearVelocityDriveEnabled = { false, false, true };
			Test.JointSettings[0].LinearDriveForceMode = EJointForceMode::Acceleration;
			Test.JointSettings[0].LinearDriveStiffness = FVec3(0, 0, Stiffness);
			Test.JointSettings[0].LinearDriveDamping = FVec3(0, 0, Damping);

			Test.Create();

			FConstGenericParticleHandle P1 = Test.GetParticle(1);

			// Run the sim - the dangling box should reach a steady state
			for (int32 i = 0; i < NumSteps; ++i)
			{
				Test.Evolution.AdvanceOneTimeStep(Dt);
				Test.Evolution.EndFrame(Dt);
			}

			// We should be stationary at the desired extension
			const FReal ExpectedVZ = 0;
			const FReal ExpectedZ = Test.ParticlePositions[1].Z - Extension;
			EXPECT_NEAR(P1->V().Z, ExpectedVZ, 1);
			EXPECT_NEAR(P1->GetX().Z, ExpectedZ, 1);
		}
	}


	// Check that the maximum drive force setting honored for linear drives.
	// We repeat TestLinearDriveForceMode_MaxForcePreTest but with a max force which is less than K.X at the rest extension
	// NOTE: Using Acceleration mode on the drive but with adjusted stiffness and damping. 
	//
	GTEST_TEST(JointForceTests, TestLinearDriveAccMode_MaxForce)
	{
		for (int32 IndexSimd = 0; IndexSimd < 2; IndexSimd++)
		{
			bool bUseSimd = IndexSimd == 0;
			const int32 NumSolverIterations = 20;
			const FReal Gravity = 1000;
			const FReal Dt = 0.01;
			const int32 NumSteps = 100;
			const int32 NumBodies = 2;
			const FReal Extension = 10;

			FJointChainTest<FPBDRigidsEvolutionGBF> Test(NumSolverIterations, Gravity);
			Test.InitChain(NumBodies, FVec3(0, 0, -1));
			Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetUseSimd(bUseSimd);

			if (!Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetSettings().bUsePositionBasedDrives)
			{
				Test.Evolution.SetNumVelocityIterations(NumSolverIterations);
			}

			// Disable all limits
			Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };
			Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };

			// Set up the drive with a force limit
			// NOTE: Acceleration mode - no masses in expressions
			const FReal Stiffness = Gravity / Extension;
			const FReal Damping = 2 * FMath::Sqrt(Stiffness);
			Test.JointSettings[0].bLinearPositionDriveEnabled = { false, false, true };
			Test.JointSettings[0].bLinearVelocityDriveEnabled = { false, false, true };
			Test.JointSettings[0].LinearDriveForceMode = EJointForceMode::Acceleration;
			Test.JointSettings[0].LinearDriveStiffness = FVec3(0, 0, Stiffness);
			Test.JointSettings[0].LinearDriveDamping = FVec3(0, 0, Damping);
			Test.JointSettings[0].LinearDriveMaxForce = FVec3(0, 0, 0.5 * Stiffness * Extension);

			Test.Create();

			FConstGenericParticleHandle P1 = Test.GetParticle(1);

			// Run the sim - the dangling box should reach a steady state
			for (int32 i = 0; i < NumSteps; ++i)
			{
				Test.Evolution.AdvanceOneTimeStep(Dt);
				Test.Evolution.EndFrame(Dt);

				const FReal ForceZ = -Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstraintLinearImpulse(0).Z / Dt;
				EXPECT_LT(ForceZ, Test.JointSettings[0].LinearDriveMaxForce.Z * Test.ParticleMasses[1] + 0.1);
			}
		}
	}
}