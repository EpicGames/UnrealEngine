// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestJoint.h"
#include "Modules/ModuleManager.h"

namespace ChaosTest {

	using namespace Chaos;


	// 1 Kinematic Body with 1 Dynamic body held horizontally by a plastic angular constraint.
	// Constraint plasticity limit is larger than resulting rotational settling so constraint will not bend.
	template <typename TEvolution>
	void JointPlasticity_UnderAngularPlasticityThreshold()
	{
		const FReal PlasticityAngle = 10;
		const int32 NumIterations = 8;
		const FReal Gravity = 980;
		const FReal Dt = 0.01f;
		int32 NumIts = 100;

		FJointChainTest<TEvolution> Test(NumIterations, Gravity);
		Test.InitChain(2, FVec3(0, 1, 0));

		// Joint should break only if Threshold < MGL
		// So not in this test
		Test.JointSettings[0].bCollisionEnabled = false;
		Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Locked, EJointMotionType::Locked , EJointMotionType::Locked };

		Test.JointSettings[0].bAngularSLerpPositionDriveEnabled = true;
		Test.JointSettings[0].bAngularSLerpVelocityDriveEnabled = true;
		Test.JointSettings[0].AngularDriveDamping = FVec3(500);
		Test.JointSettings[0].AngularDriveStiffness = FVec3(500000.f);
		
		Test.JointSettings[0].AngularPlasticityLimit = PlasticityAngle * (PI/180.);

		Test.Create();
		Test.AddParticleBox(FVec3(0, 30, 50), FRotation3::Identity, FVec3(10.f), 100.f);

		FReal Angle = Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstraintSettings(0).AngularDrivePositionTarget.GetAngle() * (180. / PI);
		EXPECT_TRUE(FMath::IsNearlyEqual(Angle, (FReal)0.));

		// Run the sim
		for (int32 i = 0; i < NumIts; ++i)
		{

			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			//FReal Angle = Test.Evolution.GetJointCombinedConstraints().GetConstraintSettings(0).AngularDrivePositionTarget.GetAngle() * (180. / PI);
			//FVec3 Pos = Test.SOAs.GetDynamicParticles().X(0);
			//std::cout << "["<< Angle <<"]" << Pos.X << "," << Pos.Y << "," << Pos.Z << std::endl;
		}
		Angle = Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstraintSettings(0).AngularDrivePositionTarget.GetAngle() * (180. / PI);

		// Nothing should have been reset
		EXPECT_TRUE(FMath::IsNearlyEqual(Angle, (FReal)0.));
		
	}

	GTEST_TEST(AllEvolutions, JointPlasticity_UnderAngularPlasticityThreshold)
	{
		JointPlasticity_UnderAngularPlasticityThreshold<FPBDRigidsEvolution>();
	}


	// 1 Kinematic Body with 1 Dynamic body held horizontally by a plastic angular constraint.
	// Constraint plasticity limit is larger than resulting rotational settling so constraint will not bend.
	template <typename TEvolution>
	void JointPlasticity_OverAngularPlasticityThreshold()
	{
		const FReal PlasticityAngle = 10;
		const int32 NumIterations = 8;
		const FReal Gravity = 980;
		const FReal Dt = 0.01f;
		int32 NumIts = 1000;

		FJointChainTest<TEvolution> Test(NumIterations, Gravity);
		Test.InitChain(2, FVec3(0, 1, 0));

		// Joint should break only if Threshold < MGL
		// So not in this test
		Test.JointSettings[0].bCollisionEnabled = false;
		Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Locked, EJointMotionType::Locked , EJointMotionType::Locked };

		Test.JointSettings[0].bAngularSLerpPositionDriveEnabled = true;
		Test.JointSettings[0].bAngularSLerpVelocityDriveEnabled = true;
		Test.JointSettings[0].AngularDriveDamping = FVec3(50);
		Test.JointSettings[0].AngularDriveStiffness = FVec3(10000.f);

		Test.JointSettings[0].AngularPlasticityLimit = PlasticityAngle * (PI / 180.);

		Test.Create();
		Test.AddParticleBox(FVec3(0, 30, 50), FRotation3::Identity, FVec3(10.f), 100.f);

		FReal Angle = Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstraintSettings(0).AngularDrivePositionTarget.GetAngle() * (180. / PI);
		EXPECT_NEAR(Angle, (FReal)0., (FReal)KINDA_SMALL_NUMBER);

		// Run the sim
		for (int32 i = 0; i < NumIts; ++i)
		{

			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			//FReal Angle = Test.Evolution.GetJointCombinedConstraints().GetConstraintSettings(0).AngularDrivePositionTarget.GetAngle() * (180. / PI);
			//FVec3 Pos = Test.SOAs.GetDynamicParticles().X(0);
			//std::cout << "["<< Angle <<"]" << Pos.X << "," << Pos.Y << "," << Pos.Z << std::endl;
		}

		Test.Evolution.AdvanceOneTimeStep(Dt);
		Test.Evolution.EndFrame(Dt);

		Angle = Test.Evolution.GetJointCombinedConstraints().LinearConstraints.GetConstraintSettings(0).AngularDrivePositionTarget.GetAngle() * (180. / PI);

		// The angle should have reset. 
		EXPECT_GE(Angle, PlasticityAngle);

	}

	GTEST_TEST(AllEvolutions, JointPlasticity_OverAngularPlasticityThreshold)
	{
		JointPlasticity_OverAngularPlasticityThreshold<FPBDRigidsEvolution>();
	}


	// 1 Kinematic Body with 1 Dynamic body held horizontally by a plastic angular constraint.
	// Constraint plasticity limit is larger than resulting linear setting so constraint will not reset.
	template <typename TEvolution>
	void JointPlasticity_UnderLinearPlasticityThreshold()
	{
		const FReal PlasticityLimit = 10;
		const int32 NumIterations = 8;
		const FReal Gravity = 980;
		const FReal Dt = 0.01f;
		int32 NumIts = 200;

		FJointChainTest<TEvolution> Test(NumIterations, Gravity);
		Test.InitChain(2, FVec3(0, 0, 1), 10.f, 50.f);
		Test.ParticleMasses[1] = 10.f;

		Test.JointSettings[0].bCollisionEnabled = false;
		Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Locked, EJointMotionType::Locked , EJointMotionType::Locked };
		Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Limited, EJointMotionType::Limited , EJointMotionType::Limited };
		Test.JointSettings[0].bSoftLinearLimitsEnabled = true;
		Test.JointSettings[0].bLinearPositionDriveEnabled = { true, true, true };
		Test.JointSettings[0].LinearLimit = 0;
		Test.JointSettings[0].LinearSoftForceMode = EJointForceMode::Force;
		Test.JointSettings[0].SoftLinearStiffness = 10000;
		Test.JointSettings[0].SoftLinearDamping = 100;

		Test.JointSettings[0].LinearPlasticityLimit = PlasticityLimit;

		Test.Create();

		FReal Z = Test.SOAs.GetDynamicParticles().GetX(0)[2];
		EXPECT_TRUE(FMath::IsNearlyEqual(Z, (FReal)50.));

		// Run the sim
		for (int32 i = 0; i < NumIts; ++i)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			//FVector Pos = Test.SOAs.GetDynamicParticles().X(0);
			//std::cout << Pos.X << "," << Pos.Y << "," << Pos.Z << std::endl;
		}
		FReal ZPost = Test.SOAs.GetDynamicParticles().GetX(0)[2];

		// Nothing should have reset
		EXPECT_TRUE(FMath::IsNearlyEqual(Z, ZPost, (FReal)5.));
	}

	GTEST_TEST(AllEvolutions, JointPlasticity_UnderLinearPlasticityThreshold)
	{
		JointPlasticity_UnderLinearPlasticityThreshold<FPBDRigidsEvolution>();
	}


	// 1 Kinematic Body with 1 Dynamic body held horizontally by a plastic angular constraint.
// Constraint plasticity limit is larger than resulting linear setting so constraint will not reset.
	template <typename TEvolution>
	void JointPlasticity_OverLinearPlasticityThreshold()
	{
		const FReal PlasticityLimit = 0.1;
		const int32 NumIterations = 8;
		const FReal Gravity = 980;
		const FReal Dt = 0.01f;
		int32 NumIts = 1000;

		FJointChainTest<TEvolution> Test(NumIterations, Gravity);
		Test.InitChain(2, FVec3(0, 0, 1), 10.f, 50.f);
		Test.ParticleMasses[1] = 10.f;

		Test.JointSettings[0].bCollisionEnabled = false;
		Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Locked, EJointMotionType::Locked , EJointMotionType::Locked };
		Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Limited, EJointMotionType::Limited , EJointMotionType::Limited };
		Test.JointSettings[0].bSoftLinearLimitsEnabled = true;
		Test.JointSettings[0].bLinearPositionDriveEnabled = {true, true, true};
		Test.JointSettings[0].LinearLimit = 0;
		Test.JointSettings[0].LinearSoftForceMode = EJointForceMode::Force;
		Test.JointSettings[0].SoftLinearStiffness = 10000;
		Test.JointSettings[0].SoftLinearDamping = 100;

		Test.JointSettings[0].LinearPlasticityLimit = PlasticityLimit;

		Test.Create();

		FReal Z = Test.SOAs.GetDynamicParticles().GetX(0)[2];
		EXPECT_TRUE(FMath::IsNearlyEqual(Z, (FReal)50.));

		// Run the sim
		for (int32 i = 0; i < NumIts; ++i)
		{

			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			//FVector Pos = Test.SOAs.GetDynamicParticles().X(0);
			//std::cout << Pos.X << "," << Pos.Y << "," << Pos.Z << std::endl;
		}
		FReal ZPost = Test.SOAs.GetDynamicParticles().GetX(0)[2];

		// The linear spring should have reset. 
		EXPECT_TRUE(ZPost < Z - PlasticityLimit);
		EXPECT_TRUE(ZPost > 0.f);
	}

	GTEST_TEST(AllEvolutions, JointPlasticity_OverLinearPlasticityThreshold)
	{
		JointPlasticity_OverLinearPlasticityThreshold<FPBDRigidsEvolution>();
	}
}