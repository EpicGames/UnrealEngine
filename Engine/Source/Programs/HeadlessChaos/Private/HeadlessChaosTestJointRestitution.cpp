// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestJoint.h"
#include "Modules/ModuleManager.h"

namespace ChaosTest {

	using namespace Chaos;

	// One kinematic, one dynamic particle arrange horizontally.
	// Locked position constraint and limited swing constraint
	template <typename TEvolution>
	void JointRestitution_Cone()
	{
		const int32 NumSolverIterations = 10;
		const FReal Gravity = 0.0f;
		const FReal Dt = 0.01f;

		FJointChainTest<TEvolution> Test(NumSolverIterations, Gravity);

		FPBDJointSolverSettings JointSolverSettings;
		Test.Evolution.GetJointCombinedConstraints().LinearConstraints.SetSettings(JointSolverSettings);

		Test.InitChain(2, FVec3(1,0,0));

		const FReal SwingLimit = FMath::DegreesToRadians(30.0f);
		Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Free, EJointMotionType::Limited, EJointMotionType::Limited };
		Test.JointSettings[0].AngularLimits = { 0.0f, SwingLimit, SwingLimit };
		Test.JointSettings[0].SwingRestitution = 1.0f;
		Test.JointSettings[0].bProjectionEnabled = false;


		Test.Create();

		// Initialize the dynamic body as if it has rotated to the point that it will hit the swing limit at the next iteration (with a specific angular velocity)
		const FReal AngVelY = FMath::DegreesToRadians(90.0f);	// rad/s
		const FReal HitTime = 0.5f;
		const FVec3 Offset1 = Test.ParticlePositions[1] - Test.JointPositions[0];
		const FRotation3 InitialRot1 = FRotation3::FromAxisAngle(FVec3(0,1,0), SwingLimit - HitTime * AngVelY * Dt);
		const FVec3 InitialOffset1 = InitialRot1 * Offset1;
		const FVec3 InitialPos1 = Test.JointPositions[0] + InitialOffset1;
		const FVec3 InitialAngVel1 = FVec3(0.0f, AngVelY, 0.0f);
		const FVec3 InitialVel1 = FVec3::CrossProduct(InitialAngVel1, InitialOffset1);
		Test.ResetParticle(Test.GetParticle(1), InitialPos1, InitialRot1, InitialVel1, InitialAngVel1);

		Test.Evolution.AdvanceOneTimeStep(Dt);
		Test.Evolution.EndFrame(Dt);

		// Check that the velocity and angular velocity was affected by restitution
		if (auto KinParticle = Test.GetParticle(1)->CastToKinematicParticle())
		{
			const FVec3 ResultVel = KinParticle->GetV();
			const FVec3 ResultAngVel = KinParticle->GetW();

			const FVec3 ExpectedVel = -Test.JointSettings[0].SwingRestitution * InitialVel1;
			const FVec3 ExpectedAngVel = -Test.JointSettings[0].SwingRestitution * InitialAngVel1;

			// We need an insane number of iterations (~500) to get the velocity almost correct unless 
			// we do a matrix solve, but event though the linear velocity is not impacted enough, 
			// we still get decent results in practice because the angvel is correct.
			//EXPECT_NEAR(ResultVel.X, ExpectedVel.X, 1.0f);
			//EXPECT_NEAR(ResultVel.Y, ExpectedVel.Y, 1.0f);
			//EXPECT_NEAR(ResultVel.Z, ExpectedVel.Z, 1.0f);

			EXPECT_NEAR(ResultAngVel.X, ExpectedAngVel.X, 0.1f);
			EXPECT_NEAR(ResultAngVel.Y, ExpectedAngVel.Y, 0.1f);
			EXPECT_NEAR(ResultAngVel.Z, ExpectedAngVel.Z, 0.1f);
		}

	}

	GTEST_TEST(AllEvolutions, DISABLED_JointRestitutionTests_Cone)
	{
		JointRestitution_Cone<FPBDRigidsEvolutionGBF>();
	}

}