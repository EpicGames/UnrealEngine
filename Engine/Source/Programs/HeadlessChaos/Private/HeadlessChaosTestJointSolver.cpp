// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"

#include "Chaos/Utilities.h"
#include "Chaos/Joint/PBDJointSolverGaussSeidel.h"
#include "ChaosLog.h"

namespace ChaosTest 
{
	using namespace Chaos;

	static_assert((int32)EJointAngularConstraintIndex::Swing1 == 2, "Tests assume Swing1 axis is Z");

	// @todo(ccaulfield): fix these tests
	// All the tests where gravity is involved do not behave as expected - the spring strength is always
	// higher than expected if Torque = -K.Theta. Is this a bug in the test or in the solver??

	// Single joint minimal test framework
	class FJointSolverTest
	{
	public:
		FPBDJointSolver Solver;
		FSolverBody Body0;
		FSolverBody Body1;

		// Settings
		int32 NumPairIts;
		FVec3 Gravity;
		FReal Mass0;
		FReal Mass1;
		FVec3 Inertia0;
		FVec3 Inertia1;
		FRigidTransform3 Connector0;
		FRigidTransform3 Connector1;
		FPBDJointSolverSettings SolverSettings;
		FPBDJointSettings JointSettings;
		FReal SolverStiffness;

		// External accelerations
		FVec3 ExtAcc1;
		FVec3 ExtAngAcc1;

		bool bInitialized;

		FJointSolverTest()
			: Solver()
			, Body0(FSolverBody::MakeInitialized())
			, Body1(FSolverBody::MakeInitialized())
			, NumPairIts(1)
			, Gravity(FVec3(0))
			, Mass0(0)
			, Mass1(0)
			, Inertia0(FVec3(0))
			, Inertia1(FVec3(0))
			, Connector0(FRigidTransform3(FVec3(0), FRotation3::FromIdentity()))
			, Connector1(FRigidTransform3(FVec3(0), FRotation3::FromIdentity()))
			, SolverStiffness(1)
			, ExtAcc1(FVec3(0))
			, ExtAngAcc1(FVec3(0))
			, bInitialized(false)
		{
		}

		void Init()
		{
			if (Mass0 > 0)
			{
				Body0.SetInvM(1.0f / Mass0);
				Body0.SetInvILocal(FVec3(1.0f / Inertia0.X, 1.0f / Inertia0.Y, 1.0f / Inertia0.Z));
			}
			if (Mass1 > 0)
			{
				Body1.SetInvM(1.0f / Mass1);
				Body1.SetInvILocal(FVec3(1.0f / Inertia1.X, 1.0f / Inertia1.Y, 1.0f / Inertia1.Z));
			}
			bInitialized = true;
		}

		void Tick(const FReal Dt)
		{
			EXPECT_TRUE(bInitialized);

			Body0.SetX(Body0.P());
			Body0.SetR(Body0.Q());
			Body1.SetX(Body1.P());
			Body1.SetR(Body1.Q());

			// Apply Forces
			if (Mass0 > 0)
			{
				FVec3 A0 = Gravity;
				Body0.ApplyLinearVelocityDelta(A0 * Dt);
				Body0.ApplyTransformDelta(Body0.V() * Dt, Body0.W() * Dt);
				Body1.ApplyCorrections();
				Body0.UpdateRotationDependentState();
			}

			if (Mass1 > 0)
			{
				FVec3 Acc1 = Gravity + ExtAcc1;
				FVec3 AngAcc1 = ExtAngAcc1;
				Body1.ApplyVelocityDelta(Acc1 * Dt, AngAcc1 * Dt);
				Body1.ApplyTransformDelta(Body1.V() * Dt, Body1.W() * Dt);
				Body1.ApplyCorrections();
				Body1.UpdateRotationDependentState();
			}

			Solver.SetSolverBodies(&Body0, &Body1);

			// Apply Constraints
			Solver.Init(
				Dt,
				SolverSettings,
				JointSettings,
				Connector0,
				Connector1);

			Solver.Update(
				Dt,
				SolverSettings,
				JointSettings);

			for (int32 PairIt = 0; PairIt < NumPairIts; ++PairIt)
			{
				Solver.ApplyConstraints(Dt, SolverStiffness, SolverSettings, JointSettings);
			}

			if (Mass0 > 0)
			{
				Body0.SetImplicitVelocity(Dt);
				Body0.ApplyCorrections();
			}

			if (Mass1 > 0)
			{
				Body1.SetImplicitVelocity(Dt);
				Body1.ApplyCorrections();
			}

		}
	};

	// Set up a soft position constraint between a dynamic and kinematic particle.
	// Verify that F = -KX
	GTEST_TEST(JointSolverTests, TestJointSolver_KinematicDynamic_SoftPositionConstraint_ForceMode)
	{
		FJointSolverTest SolverTest;

		FReal Dt = 0.02f;
		int32 NumIts = 1000;
		SolverTest.Gravity = FVec3(0, 0, -1000);
		SolverTest.Mass1 = 100.0f;
		SolverTest.Inertia1 = FVec3(10000, 10000, 10000);

		// Set up a heavily damped position drive so that it settles quickly
		SolverTest.JointSettings.LinearMotionTypes = { EJointMotionType::Limited, EJointMotionType::Limited, EJointMotionType::Limited };
		SolverTest.JointSettings.LinearLimit = 1.0f;
		SolverTest.JointSettings.bSoftLinearLimitsEnabled = true;
		SolverTest.JointSettings.SoftLinearStiffness = 1000.0f;
		SolverTest.JointSettings.SoftLinearDamping = 1000.0f;
		SolverTest.JointSettings.LinearSoftForceMode = EJointForceMode::Force;

		// Particle 0 is Kinematic
		// Particle 1 is Dynamic

		SolverTest.Init();

		int32 RollingIts = 5;
		int32 It = 0;
		FVec3 OutDelta1 = FVec3(0);
		FReal AverageOutDelta1Z = 0.0f;
		while (It++ < NumIts)
		{
			SolverTest.Tick(Dt);

			// Measure Distance
			OutDelta1 = SolverTest.Body1.P();

			// Moving average delta (Z Axis)
			AverageOutDelta1Z = AverageOutDelta1Z + (OutDelta1.Z - AverageOutDelta1Z) / (FMath::Min(It, RollingIts));

			// Check for settling
			//UE_LOGF(LogChaos, Warning, "%d: %f %f %f: %f", It, OutDelta1.X, OutDelta1.Y, OutDelta1.Z, AverageOutDelta1Z);
			if ((It > 20) && FMath::IsNearlyEqual(AverageOutDelta1Z, (FReal)OutDelta1.Z, (FReal)KINDA_SMALL_NUMBER))
			{
				break;
			}
		}

		// Verify that X and Y offsets are zero, and that Z is negative
		EXPECT_NEAR(OutDelta1.X, 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(OutDelta1.Y, 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_LT(OutDelta1.Z, -5);

		// Verify that we stabilized
		EXPECT_LT(It, NumIts);

		// Verify that the force at the current position is the same for both gravity and the spring
		// For force-mode springs:
		//   F = -Stiffness * PosError = -MG
		FReal GravityForce = SolverTest.Mass1 * SolverTest.Gravity.Z;
		FReal SpringForce = -SolverTest.JointSettings.SoftLinearStiffness * (OutDelta1.Z + SolverTest.JointSettings.LinearLimit);
		EXPECT_NEAR(SpringForce, -GravityForce, 1.0f);
	}


	// Set up a soft position constraint between a dynamic and kinematic particle.
	// Verify that F = -KX
	GTEST_TEST(JointSolverTests, TestJointSolver_KinematicDynamic_SoftPositionConstraint_AccMode)
	{
		FJointSolverTest SolverTest;

		FReal Dt = 0.02f;
		int32 NumIts = 1000;
		SolverTest.Gravity = FVec3(0, 0, -1000);
		SolverTest.Mass1 = 100.0f;
		SolverTest.Inertia1 = FVec3(10000, 10000, 10000);

		// Set up a heavily damped position drive so that it settles quickly
		SolverTest.JointSettings.LinearMotionTypes = { EJointMotionType::Limited, EJointMotionType::Limited, EJointMotionType::Limited };
		SolverTest.JointSettings.LinearLimit = 1.0f;
		SolverTest.JointSettings.bSoftLinearLimitsEnabled = true;
		SolverTest.JointSettings.SoftLinearStiffness = 100.0f;
		SolverTest.JointSettings.SoftLinearDamping = 10.0f;
		SolverTest.JointSettings.LinearSoftForceMode = EJointForceMode::Acceleration;

		// Particle 0 is Kinematic
		// Particle 1 is Dynamic

		SolverTest.Init();

		int32 RollingIts = 5;
		int32 It = 0;
		FVec3 OutDelta1 = FVec3(0);
		FReal AverageOutDelta1Z = 0.0f;
		while (It++ < NumIts)
		{
			SolverTest.Tick(Dt);

			// Measure Distance
			OutDelta1 = SolverTest.Body1.P();

			// Moving average delta (Z Axis)
			AverageOutDelta1Z = AverageOutDelta1Z + (OutDelta1.Z - AverageOutDelta1Z) / (FMath::Min(It, RollingIts));

			// Check for settling
			//UE_LOGF(LogChaos, Warning, "%d: %f %f %f: %f", It, OutDelta1.X, OutDelta1.Y, OutDelta1.Z, AverageOutDelta1Z);
			if ((It > 20) && FMath::IsNearlyEqual(AverageOutDelta1Z, (FReal)OutDelta1.Z, (FReal)KINDA_SMALL_NUMBER))
			{
				break;
			}
		}

		// Verify that X and Y offsets are zero, and that Z is negative
		EXPECT_NEAR(OutDelta1.X, 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(OutDelta1.Y, 0.0f, KINDA_SMALL_NUMBER);
		EXPECT_LT(OutDelta1.Z, -5);

		// Verify that we stabilized
		EXPECT_LT(It, NumIts);

		// Verify that the force at the current position is the same for both gravity and the spring
		// For acceleration-mode springs:
		//   A = -Stiffness * PosError
		FReal GravityAcc = SolverTest.Gravity.Z;
		FReal SpringAcc = -SolverTest.JointSettings.SoftLinearStiffness * (OutDelta1.Z + SolverTest.JointSettings.LinearLimit);
		EXPECT_NEAR(SpringAcc, -GravityAcc, 1.0f);
	}

	// Set up a soft swing constraint between a dynamic and kinematic particle.
	// Verify that the movement is equivalent to a applying forces from a damped spring.
	// Verify that changing the mass affects the movement.
	GTEST_TEST(JointSolverTests, TestJointSolver_KinematicDynamic_SoftSwingConstraint_ForceMode)
	{
		FJointSolverTest SolverTestA;
		FJointSolverTest SolverTestB;

		FReal Dt = 0.02f;

		FVec3 Offset1 = FVec3(10, 0, 0);				// Particle1 distance from connector
		FReal Angle1 = -FMath::DegreesToRadians(10);	// Particle1 rotation through connector
		FRotation3 Rotation1 = FRotation3::FromAxisAngle(FJointConstants::Swing1Axis(), Angle1);

		SolverTestA.NumPairIts = 4;
		SolverTestA.Mass1 = 1.0f;
		SolverTestA.Inertia1 = FVec3(100, 100, 100);
		SolverTestA.Connector1 = FRigidTransform3(-Offset1, FRotation3::FromIdentity());

		SolverTestA.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] = EJointMotionType::Limited;
		SolverTestA.JointSettings.AngularLimits[(int32)EJointAngularConstraintIndex::Swing1] = 0;
		SolverTestA.JointSettings.bSoftSwingLimitsEnabled = true;
		SolverTestA.JointSettings.SoftSwingStiffness = 100.0f;
		SolverTestA.JointSettings.SoftSwingDamping = 0.0f;
		SolverTestA.JointSettings.AngularSoftForceMode = EJointForceMode::Force;

		// Particle 0 is Kinematic
		// Particle 1 is Dynamic, rotated by 10 degrees about the Swing1(Z) axis through its connector
		SolverTestA.Body1.SetQ(Rotation1);
		SolverTestA.Body1.SetP(Rotation1 * Offset1);

		FReal MassScale = 5;
		SolverTestB = SolverTestA;
		SolverTestB.Mass1 = MassScale * SolverTestA.Mass1;
		SolverTestB.Inertia1 = MassScale * SolverTestA.Inertia1;

		SolverTestA.Init();
		SolverTestB.Init();

		SolverTestA.Tick(Dt);
		SolverTestB.Tick(Dt);

		// For force-mode springs:
		//   F = InvI * DW/DT = -Stiffness * AngleError
		//   DW = DR/DT = -InvI * Stiffness * AngleError * Dt
		//   DR = -InvI * Stiffness * AngleError * Dt * Dt

		FReal EffectiveInertiaA1 = SolverTestA.Inertia1.Z + SolverTestA.Mass1 * Offset1.X * Offset1.X;
		FReal EffectiveInertiaB1 = SolverTestB.Inertia1.Z + SolverTestB.Mass1 * Offset1.X * Offset1.X;

		// Verify that the angle change is as expected
		FReal OutAngle1A = 2.0f * FMath::Asin(SolverTestA.Body1.Q().Z);
		FReal OutAngleDelta1A = OutAngle1A - Angle1;
		FReal IIAxis1A = 1.0f / EffectiveInertiaA1;
		FReal ExpectedAngle1DeltaA = -IIAxis1A * SolverTestA.JointSettings.SoftSwingStiffness * Angle1 * Dt * Dt;
		EXPECT_NEAR(OutAngleDelta1A, ExpectedAngle1DeltaA, 1.e-6f);

		// Verify that the angle change is as expected
		FReal OutAngle1B = 2.0f * FMath::Asin(SolverTestB.Body1.Q().Z);
		FReal OutAngleDelta1B = OutAngle1B - Angle1;
		FReal IIAxis1B = 1.0f / EffectiveInertiaB1;
		FReal ExpectedAngle1DeltaB = -IIAxis1B * SolverTestB.JointSettings.SoftSwingStiffness * Angle1 * Dt * Dt;
		EXPECT_NEAR(OutAngleDelta1B, ExpectedAngle1DeltaB, 1.e-6f);

		// Verify that the angle change is proportional to inverse mass
		EXPECT_NEAR(OutAngleDelta1A, MassScale * OutAngleDelta1B, 1.e-6f);
	}


	// Set up a soft swing constraint between a dynamic and kinematic particle.
	// Verify that a SLerp drive pushing against gravity results in the correct angle
	// assuming the drive torque is T = -k.I.Theta.
	void KinematicDynamic_SLerpDrive(FJointSolverTest& SolverTestA, FReal Mass, FReal Inertia, EJointForceMode ForceMode, FReal Stiffness, FReal Damping, FReal AngAcc, FReal Offset)
	{
		FReal Dt = 0.02f;
		int32 NumIts = 1000;

		FVec3 Offset1 = FVec3(Offset, 0, 0);				// Particle1 distance from connector
		SolverTestA.Mass1 = Mass;
		SolverTestA.Inertia1 = FVec3(Inertia, Inertia, Inertia);
		SolverTestA.Connector1 = FRigidTransform3(-Offset1, FRotation3::FromIdentity());
		SolverTestA.Body1.SetP(Offset1);

		// Set up a heavily damped SLerp drive so that it settles quickly
		SolverTestA.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist] = EJointMotionType::Free;
		SolverTestA.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] = EJointMotionType::Free;
		SolverTestA.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2] = EJointMotionType::Free;
		SolverTestA.JointSettings.bAngularSLerpPositionDriveEnabled = true;
		SolverTestA.JointSettings.bAngularSLerpVelocityDriveEnabled = true;
		SolverTestA.JointSettings.AngularDriveStiffness = FVec3(Stiffness);
		SolverTestA.JointSettings.AngularDriveDamping = FVec3(Damping);
		SolverTestA.JointSettings.AngularDrivePositionTarget = FRotation3::FromIdentity();
		SolverTestA.JointSettings.AngularDriveForceMode = ForceMode;

		SolverTestA.Init();

		int32 RollingIts = 20;
		int32 It = 0;
		FVec3 AverageW = FVec3(0);
		while (It++ < NumIts)
		{
			// Apply an angular acceleration about the connector to body 1
			SolverTestA.ExtAngAcc1 = FVec3(0, AngAcc, 0);
			SolverTestA.ExtAcc1 = FVec3::CrossProduct(SolverTestA.ExtAngAcc1, SolverTestA.Body1.Q() * Offset1);

			SolverTestA.Tick(Dt);

			// Measure Angle
			FVec3 OutAngles1 = FVec3(
				2.0f * FMath::Asin(SolverTestA.Body1.Q().X),
				2.0f * FMath::Asin(SolverTestA.Body1.Q().Y),
				2.0f * FMath::Asin(SolverTestA.Body1.Q().Z));

			// We should only be rotating about Y Axis
			EXPECT_NEAR(OutAngles1.X, 0.0f, 0.01f);
			EXPECT_NEAR(OutAngles1.Z, 0.0f, 0.01f);

			// Make sure our test is set up so that the angle is reasonable (i.e., the torque is not so high it rotates forever)
			EXPECT_LT(OutAngles1.Y, PI);
			EXPECT_GT(OutAngles1.Y, -PI);

			// Moving average angular velocity
			AverageW = SolverTestA.Body1.W() + (SolverTestA.Body1.W() - AverageW) / (FMath::Min(It, RollingIts));

			// Check for settling
			//UE_LOGF(LogChaos, Warning, "%d: %f deg : %f deg/s", It, FMath::RadiansToDegrees(OutAngles1.Y), FMath::RadiansToDegrees(SolverTestA.W1.Y));
			if ((It > 20) && FMath::IsNearlyEqual(SolverTestA.Body1.W().Y, FReal(0), FReal(KINDA_SMALL_NUMBER)) && FMath::IsNearlyEqual(AverageW.Y, FReal(0), FReal(KINDA_SMALL_NUMBER)))
			{
				break;
			}
		}

		// Verify that we stabilized
		EXPECT_LT(It, NumIts);
	}

	// Set up a soft swing constraint between a dynamic and kinematic particle.
	// Verify that a SLerp drive pushing against a torque results in the correct angle
	// assuming the drive torque is T = -K.Theta.
	GTEST_TEST(JointSolverTests, TestJointSolver_KinematicDynamic_SLerpDrive_ForceMode)
	{
		FReal DistanceAngAccs[][2] = {
			//{ 0.0f, 10.0f },
			//{ 0.0f, 100.0f },
			//{ 0.0f, 200.0f },
			//{ 1.0f, 10.0f },
			//{ 1.0f, 100.0f },
			//{ 1.0f, 200.0f },
			//{ 10.0f, 10.0f },
			{ 10.0f, 100.0f },
			{ 10.0f, 200.0f },
		};

		for (int32 Index = 0; Index < UE_ARRAY_COUNT(DistanceAngAccs); ++Index)
		{
			FReal Distance = DistanceAngAccs[Index][0];
			FReal AngAcc = DistanceAngAccs[Index][1];
			FReal Mass = 5.0f;
			FReal Inertia = 200.0f;
			FReal Stiffness = AngAcc * 1000.0f;
			FReal Damping = AngAcc * 300.0f;

			FJointSolverTest SolverTestA;
			KinematicDynamic_SLerpDrive(SolverTestA, Mass, Inertia, EJointForceMode::Force, Stiffness, Damping, AngAcc, Distance);

			FVec3 OutAngles1 = FVec3(
				2.0f * FMath::Asin(SolverTestA.Body1.Q().X),
				2.0f * FMath::Asin(SolverTestA.Body1.Q().Y),
				2.0f * FMath::Asin(SolverTestA.Body1.Q().Z));

			FReal EffectiveInertia1 = Inertia + Mass * Distance * Distance;

			// Calculate the expected angle for the given torque
			// Check for setup errors - if the torque leads to 180 degree rotation, it will keep spinning
			FReal ExpectedAngleDeg = FMath::RadiansToDegrees(EffectiveInertia1 * AngAcc / Stiffness);
			EXPECT_LT(ExpectedAngleDeg, 180);

			FReal AngleDeg = FMath::RadiansToDegrees(OutAngles1.Y);
			EXPECT_NEAR(AngleDeg, ExpectedAngleDeg, 3.0f) << "Distance: " << Distance << "; AngAcc: " << AngAcc;
		}
	}


	// Set up a soft swing constraint between a dynamic and kinematic particle.
	// Verify that a SLerp drive pushing against a torque results in the correct angle
	// assuming the drive acceleration is dW/dT = -K.Theta.
	// @todo(ccaulfield): fix tests
	GTEST_TEST(JointSolverTests, DISABLED_TestJointSolver_KinematicDynamic_SLerpDrive_AccMode)
	{
		FReal Distances[] = {
			0.0f,
			1.0f,
			10.0f,
			100.0f,
			1000.0f
		};

		FReal AngAccs[] = {
			10.0f,
			100.0f,
			200.0f,
		};

		FReal Masses[] = {
			1.0f,
			5.0f,
			10.0f,
			100.0f,
		};

		FReal Inertias[] = {
			100.0f,
			200.0f,
			10000.0f,
			100000.0f,
		};
		static_assert(UE_ARRAY_COUNT(Masses) == UE_ARRAY_COUNT(Inertias), "Mass-Inertia array mismatch");

		for (int32 AccIndex = 0; AccIndex < UE_ARRAY_COUNT(AngAccs); ++AccIndex)
		{
			for (int32 DistanceIndex = 0; DistanceIndex < UE_ARRAY_COUNT(Distances); ++DistanceIndex)
			{
				for (int32 MassIndex = 0; MassIndex < UE_ARRAY_COUNT(Masses); ++MassIndex)
				{
					FReal Mass = Masses[MassIndex];
					FReal Inertia = Inertias[MassIndex];
					FReal Distance = Distances[DistanceIndex];
					FReal AngAcc = AngAccs[AccIndex];
					FReal Stiffness = AngAcc * 1.0f;
					FReal Damping = AngAcc * 0.3f;

					FJointSolverTest SolverTestA;
					KinematicDynamic_SLerpDrive(SolverTestA, Mass, Inertia, EJointForceMode::Acceleration, Stiffness, Damping, AngAcc, Distance);

					FVec3 OutAngles1 = FVec3(
						2.0f * FMath::Asin(SolverTestA.Body1.Q().X),
						2.0f * FMath::Asin(SolverTestA.Body1.Q().Y),
						2.0f * FMath::Asin(SolverTestA.Body1.Q().Z));

					// Calculate the expected angle for the given torque
					FReal ExpectedAngleDeg = FMath::RadiansToDegrees(AngAcc / Stiffness);
					FReal AngleDeg = FMath::RadiansToDegrees(OutAngles1.Y);
					EXPECT_NEAR(AngleDeg, ExpectedAngleDeg, 3.0f) << "Distance: " << Distance << "; AngAcc: " << AngAcc << "; Mass: " << Mass << "; Inertia: " << Inertia;
				}
			}
		}
	}


	// This test reproduces and issue seen in game.
	// A dynamic body is positioned just above a kinematic one, with a SLerp spring maintaining
	// the dynamic body in a near vertical orientation. Initialize the dynamic body at a 90deg
	// rotation about its connector and verify that it ends up vertical.
	GTEST_TEST(JointSolverTests, TestJointSolver_KinematicDynamic_SLerpDrive_Gravity)
	{
		FReal Dt = 0.033f;
		int32 NumIts = 1000;
		FJointSolverTest SolverTestA;

		SolverTestA.NumPairIts = 1;
		SolverTestA.Gravity = FVec3(0, 0, -980);
		SolverTestA.Mass1 = 2.0f;
		SolverTestA.Inertia1 = FVec3(5.62034369f, 5.62034369f, 5.48915672f);

		// Set up a SLerp drive
		SolverTestA.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist] = EJointMotionType::Free;
		SolverTestA.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] = EJointMotionType::Free;
		SolverTestA.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2] = EJointMotionType::Free;
		SolverTestA.JointSettings.bAngularSLerpPositionDriveEnabled = true;
		SolverTestA.JointSettings.bAngularSLerpVelocityDriveEnabled = true;
		SolverTestA.JointSettings.AngularDriveStiffness = FVec3(80.0f);
		SolverTestA.JointSettings.AngularDriveDamping = FVec3(1.0f);
		SolverTestA.JointSettings.AngularDrivePositionTarget = FRotation3::FromIdentity();

		// Particle 0 is Kinematic
		// Particle 1 is Dynamic, with CoM vertically above the connector by a small amount
		FVec3 Offset1 = FVec3(0, 0, 0.25f);				// Particle1 CoM distance from connector
		SolverTestA.Connector0 = FRigidTransform3(FVec3(0, 0, 0), FRotation3::FromIdentity());
		SolverTestA.Connector1 = FRigidTransform3(-Offset1, FRotation3::FromIdentity());
		SolverTestA.Body1.SetQ(FRotation3::FromAxisAngle(FVec3(0, 1, 0), FMath::DegreesToRadians(45)));	// Start rotated 90 degrees
		SolverTestA.Body1.SetP(SolverTestA.Body1.Q() * Offset1);

		SolverTestA.Init();

		int32 RollingIts = 20;
		int32 It = 0;
		FVec3 OutAngles1 = FVec3(0);
		FVec3 AverageW = FVec3(0);
		while (It++ < NumIts)
		{
			SolverTestA.Tick(Dt);

			// Measure Angle
			OutAngles1 = FVec3(
				2.0f * FMath::Asin(SolverTestA.Body1.Q().X),
				2.0f * FMath::Asin(SolverTestA.Body1.Q().Y),
				2.0f * FMath::Asin(SolverTestA.Body1.Q().Z));

			// We should only be rotating about Y Axis
			EXPECT_NEAR(OutAngles1.X, 0.0f, 0.01f);
			EXPECT_NEAR(OutAngles1.Z, 0.0f, 0.01f);

			// Make sure our test is set up so that the angle is reasonable (i.e., the torque is not so high it rotates forever)
			EXPECT_LT(OutAngles1.Y, PI);
			EXPECT_GT(OutAngles1.Y, -PI);

			// Moving average angular velocity
			AverageW = SolverTestA.Body1.W() + (SolverTestA.Body1.W() - AverageW) / (FMath::Min(It, RollingIts));

			// Check for settling
			//UE_LOGF(LogChaos, Warning, "%d: %f deg : %f deg/s", It, FMath::RadiansToDegrees(OutAngles1.Y), FMath::RadiansToDegrees(SolverTestA.W1.Y));
			if ((It > 20) && FMath::IsNearlyEqual(SolverTestA.Body1.W().Y, FReal(0), FReal(KINDA_SMALL_NUMBER)) && FMath::IsNearlyEqual(AverageW.Y, FReal(0), FReal(KINDA_SMALL_NUMBER)))
			{
				break;
			}
		}

		FReal GravityAngAcc = SolverTestA.Body1.InvI().M[1][1] * SolverTestA.Mass1 * FVec3::CrossProduct(SolverTestA.Body1.P(), SolverTestA.Gravity).Y;
		FReal SpringAngAcc = -SolverTestA.JointSettings.AngularDriveStiffness.X * OutAngles1.Y;
		EXPECT_NEAR(SpringAngAcc, -GravityAngAcc, 5.0f);

		// Verify that X and Z angles are zero, and that Y is almost zero
		// In PhysX this setup leads to the body being near vertical, but that is not the case for Chaos, and it looks like we are correct, so
		// there must be something else going on in PhysX that we do not have the equivalent of...
		//EXPECT_NEAR(OutAngles1.X, 0.0f, KINDA_SMALL_NUMBER);
		//EXPECT_LT(OutAngles1.Y, FMath::DegreesToRadians(1));
		//EXPECT_NEAR(OutAngles1.Z, 0.0f, KINDA_SMALL_NUMBER);
	}

	// Linear Drive Stiffness and Damping test
	// A dynamic body attached to a kinematic boidy by a spring with different stiffness and damping on each axis.
	GTEST_TEST(JointSolverTests, TestJointSolver_KinematicDynamic_LinearDrive)
	{
		FJointSolverTest SolverTest;

		FReal Dt = 0.02f;
		int32 NumIts = 100;
		SolverTest.Gravity = FVec3(0, 0, -1000);
		SolverTest.Mass1 = 100.0f;
		SolverTest.Inertia1 = FVec3(10000, 10000, 10000);

		SolverTest.JointSettings.LinearMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };
		SolverTest.JointSettings.bSoftLinearLimitsEnabled = false;
		SolverTest.JointSettings.LinearDriveStiffness = FVec3(1000, 0, 100);
		SolverTest.JointSettings.LinearDriveDamping = FVec3(70, 0, 20);
		SolverTest.JointSettings.LinearDriveForceMode = EJointForceMode::Acceleration;
		SolverTest.JointSettings.bLinearPositionDriveEnabled = TVec3<bool>(true);
		SolverTest.JointSettings.bLinearVelocityDriveEnabled = TVec3<bool>(true);

		// Particle 0 is Kinematic
		// Particle 1 is Dynamic
		SolverTest.Body0.SetP(FVec3(0, 0, 0));
		SolverTest.Body1.SetP(FVec3(100, 100, 100));

		SolverTest.Init();

		int32 It = 0;
		while (It++ < NumIts)
		{
			SolverTest.Tick(Dt);
		}

		// Dynamic body should be close to X = 0, Y = 100, Z = G/K
		const FReal Tolerance = UE_KINDA_SMALL_NUMBER;
		EXPECT_NEAR(SolverTest.Body1.P().X, 0, Tolerance);
		EXPECT_NEAR(SolverTest.Body1.P().Y, 100, Tolerance);
		EXPECT_NEAR(SolverTest.Body1.P().Z, SolverTest.Gravity.Z / SolverTest.JointSettings.LinearDriveStiffness.Z, Tolerance);
	}

	// A dynamic body connected to a kinematic body by a twist spring.
	// Body starts rotated about twist and swing and the twist angle should decrease to 0, leaving the swing untouched.
	GTEST_TEST(JointSolverTests, TestJointSolver_KinematicDynamic_TwistDrive)
	{
		FJointSolverTest SolverTest;

		FReal Dt = 0.02f;
		int32 NumIts = 50;
		FReal Stiffness = 1000;
		FReal Damping = 70;
		FVec3 Connector = FVec3(100, 0, 0);
		FReal InitSwingAngle = FMath::DegreesToRadians(45);
		FReal InitTwistAngle = FMath::DegreesToRadians(45);
		FRotation3 InitRot = FRotation3::FromAxisAngle(FVec3(0,1,0), InitSwingAngle) * FRotation3::FromAxisAngle(FVec3(1, 0, 0), InitTwistAngle);

		SolverTest.Gravity = FVec3(0);
		SolverTest.Mass1 = 100.0f;
		SolverTest.Inertia1 = FVec3(10000, 10000, 10000);
		SolverTest.Body1.SetQ(InitRot);

		// Set up a damped Twist drive. Swing drive is also active but with 0 stiffness and damping
		SolverTest.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist] = EJointMotionType::Free;
		SolverTest.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] = EJointMotionType::Free;
		SolverTest.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2] = EJointMotionType::Free;
		SolverTest.JointSettings.bAngularTwistPositionDriveEnabled = true;
		SolverTest.JointSettings.bAngularTwistVelocityDriveEnabled = true;
		SolverTest.JointSettings.bAngularSwingPositionDriveEnabled = true;
		SolverTest.JointSettings.bAngularSwingVelocityDriveEnabled = true;
		SolverTest.JointSettings.AngularDriveStiffness = FVec3(Stiffness, 0, 0);
		SolverTest.JointSettings.AngularDriveDamping = FVec3(Damping, 0, 0);
		SolverTest.JointSettings.AngularDrivePositionTarget = FRotation3::FromIdentity();
		SolverTest.JointSettings.AngularDriveForceMode = EJointForceMode::Acceleration;

		SolverTest.Init();

		FReal TwistAngle, Swing1Angle, Swing2Angle;
		FPBDJointUtilities::GetSwingTwistAngles(FRotation3::FromIdentity(), SolverTest.Body1.Q(), TwistAngle, Swing1Angle, Swing2Angle);
		EXPECT_NEAR(TwistAngle, InitTwistAngle, UE_KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Swing1Angle, 0, UE_KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Swing2Angle, InitSwingAngle, UE_KINDA_SMALL_NUMBER);

		int32 It = 0;
		while (It++ < NumIts)
		{
			SolverTest.Tick(Dt);

			FPBDJointUtilities::GetSwingTwistAngles(FRotation3::FromIdentity(), SolverTest.Body1.Q(), TwistAngle, Swing1Angle, Swing2Angle);
		}

		const FReal Tolerance = UE_KINDA_SMALL_NUMBER;
		EXPECT_NEAR(TwistAngle, 0, Tolerance);
		EXPECT_NEAR(Swing1Angle, 0, Tolerance);
		EXPECT_NEAR(Swing2Angle, InitSwingAngle, Tolerance);
	}

	// A dynamic body connected to a kinematic body by a swing spring.
	// Body starts rotated about twist and swing and the swing angle should decrease to 0, leaving the twist untouched.
	GTEST_TEST(JointSolverTests, TestJointSolver_KinematicDynamic_Swing2Drive)
	{
		FJointSolverTest SolverTest;

		FReal Dt = 0.02f;
		int32 NumIts = 50;
		FReal Stiffness = 1000;
		FReal Damping = 70;
		FReal InitSwingAngle = FMath::DegreesToRadians(45);
		FReal InitTwistAngle = FMath::DegreesToRadians(0);
		FRotation3 InitRot = FRotation3::FromAxisAngle(FVec3(0, 1, 0), InitSwingAngle) * FRotation3::FromAxisAngle(FVec3(1, 0, 0), InitTwistAngle);

		SolverTest.Gravity = FVec3(0);
		SolverTest.Mass1 = 100.0f;
		SolverTest.Inertia1 = FVec3(10000, 10000, 10000);
		SolverTest.Body1.SetQ(InitRot);

		// Set up a damped Twist drive. Swing drive is also active but with 0 stiffness and damping
		SolverTest.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Twist] = EJointMotionType::Free;
		SolverTest.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing1] = EJointMotionType::Free;
		SolverTest.JointSettings.AngularMotionTypes[(int32)EJointAngularConstraintIndex::Swing2] = EJointMotionType::Free;
		SolverTest.JointSettings.bAngularTwistPositionDriveEnabled = true;
		SolverTest.JointSettings.bAngularTwistVelocityDriveEnabled = true;
		SolverTest.JointSettings.bAngularSwingPositionDriveEnabled = true;
		SolverTest.JointSettings.bAngularSwingVelocityDriveEnabled = true;
		SolverTest.JointSettings.AngularDriveStiffness = FVec3(0, Stiffness, Stiffness);
		SolverTest.JointSettings.AngularDriveDamping = FVec3(0, Damping, Damping);
		SolverTest.JointSettings.AngularDrivePositionTarget = FRotation3::FromIdentity();
		SolverTest.JointSettings.AngularDriveForceMode = EJointForceMode::Acceleration;

		SolverTest.Init();

		FReal TwistAngle, Swing1Angle, Swing2Angle;
		FPBDJointUtilities::GetSwingTwistAngles(FRotation3::FromIdentity(), SolverTest.Body1.Q(), TwistAngle, Swing1Angle, Swing2Angle);
		EXPECT_NEAR(TwistAngle, InitTwistAngle, UE_KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Swing1Angle, 0, UE_KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Swing2Angle, InitSwingAngle, UE_KINDA_SMALL_NUMBER);

		int32 It = 0;
		while (It++ < NumIts)
		{
			SolverTest.Tick(Dt);

			FPBDJointUtilities::GetSwingTwistAngles(FRotation3::FromIdentity(), SolverTest.Body1.Q(), TwistAngle, Swing1Angle, Swing2Angle);
		}

		EXPECT_NEAR(TwistAngle, InitTwistAngle, UE_KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Swing1Angle, 0, UE_KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Swing2Angle, 0, UE_KINDA_SMALL_NUMBER);
	}

}