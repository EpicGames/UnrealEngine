// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"

#include "Chaos/PBDJointConstraintUtilities.h"
#include "ChaosLog.h"

namespace ChaosTest
{
	using namespace Chaos;

	struct SwingTwistCase
	{
	public:
		FVec3 SwingAxis;
		FReal SwingAngleDeg;
		FReal TwistAngleDeg;

		friend std::ostream& operator<<(std::ostream& s, const SwingTwistCase& Case)
		{
			return s << "Twist/Swing: " << Case.TwistAngleDeg << "/" << Case.SwingAngleDeg << " Swing Axis: (" << Case.SwingAxis.X << ", " << Case.SwingAxis.Y << ", " << Case.SwingAxis.Z << ")";
		}
	};

	void TestAnglesDeg(const SwingTwistCase& Case, FReal A0, FReal A1, FReal Tolerance)
	{
		if (!(FMath::IsNearlyEqual(FMath::Abs(A0 - A1), (FReal)0., Tolerance) || FMath::IsNearlyEqual(FMath::Abs(A0 - A1), (FReal)360., Tolerance)))
		{
			GTEST_FAIL() << "Angle Test Fail: " << A0 << " != " << A1 << " " << Case;
		}
	}

	void TestSwingTwistOrder(const SwingTwistCase& Case)
	{
		FRotation3 SwingRot = FRotation3::FromAxisAngle(Case.SwingAxis, FMath::DegreesToRadians(Case.SwingAngleDeg));

		FVec3 TwistAxis = FVec3(1, 0, 0);
		FRotation3 TwistRot = FRotation3::FromAxisAngle(TwistAxis, FMath::DegreesToRadians(Case.TwistAngleDeg));

		FRotation3 RST = SwingRot * TwistRot;
		FRotation3 RS = SwingRot;

		// Verify that a vector along the X Axis is unaffected by twist
		FVec3 X = FVec3(100, 0, 0);
		FVec3 XST = RST * X;	// Swing and Twist applied
		FVec3 XS = RS * X;		// Just Swing applied

		EXPECT_NEAR(XS.X, XST.X, KINDA_SMALL_NUMBER) << Case;
		EXPECT_NEAR(XS.Y, XST.Y, KINDA_SMALL_NUMBER) << Case;
		EXPECT_NEAR(XS.Z, XST.Z, KINDA_SMALL_NUMBER) << Case;
	}

	void TestSwingTwistDecomposition(const SwingTwistCase& Case)
	{
		FRotation3 SwingRot = FRotation3::FromAxisAngle(Case.SwingAxis, FMath::DegreesToRadians(Case.SwingAngleDeg));

		FVec3 TwistAxis = FVec3(1, 0, 0);
		FRotation3 TwistRot = FRotation3::FromAxisAngle(TwistAxis, FMath::DegreesToRadians(Case.TwistAngleDeg));

		FRotation3 R0 = FRotation3::Identity;
		FRotation3 R1 = R0 * SwingRot * TwistRot;

		FVec3 OutTwistAxis, OutSwingAxisLocal;
		FReal OutTwistAngle, OutSwingAngle;

		FPBDJointUtilities::GetTwistAxisAngle(R0, R1, OutTwistAxis, OutTwistAngle);
		FPBDJointUtilities::GetConeAxisAngleLocal(R0, R1,  1.e-6f, OutSwingAxisLocal, OutSwingAngle);
		FReal OutTwistAngleDeg = FMath::RadiansToDegrees(OutTwistAngle);
		FReal OutSwingAngleDeg = FMath::RadiansToDegrees(OutSwingAngle);

		// Degenerate behavior at 180 degrees
		if (Case.SwingAngleDeg == 180)
		{
			TestAnglesDeg(Case, 180.0f, OutSwingAngleDeg, 0.1f);
			TestAnglesDeg(Case, 0.0f, OutTwistAngleDeg, 0.1f);
			return;
		}

		// If we expect a non-zero swing, make sure we recovered the swing axis
		FReal ExpectedSwingAngleDeg = (FVec3::DotProduct(Case.SwingAxis, OutSwingAxisLocal) >= 0.0f) ? Case.SwingAngleDeg : 360 - Case.SwingAngleDeg;
		if (ExpectedSwingAngleDeg != 0)
		{
			EXPECT_NEAR(FMath::Abs(FVec3::DotProduct(Case.SwingAxis, OutSwingAxisLocal)), 1.0f, 1.e-2f) << Case;
		}

		TestAnglesDeg(Case, ExpectedSwingAngleDeg, OutSwingAngleDeg, 0.1f);
		TestAnglesDeg(Case, Case.TwistAngleDeg, OutTwistAngleDeg, 0.1f);
	}

	GTEST_TEST(JointUtilitiesTests, TestSwingTwistDecomposition)
	{
		FVec3 SwingAxes[] = 
		{
			FVec3(0, 1, 0),
			FVec3(0, 0, 1),
			FVec3(0, 1, 1).GetSafeNormal(),
		};

		//int32 TwistIndex = 3;
		for (int32 TwistIndex = 0; TwistIndex < 360; ++TwistIndex)
		{
			FReal TwistAngleDeg = (FReal)TwistIndex;

			//int32 SwingIndex = 1;
			for (int32 SwingIndex = 0; SwingIndex < 360; ++SwingIndex)
			{
				FReal SwingAngleDeg = (FReal)SwingIndex;

				for (int32 SwingAxisIndex = 0; SwingAxisIndex < UE_ARRAY_COUNT(SwingAxes); ++SwingAxisIndex)
				{
					FVec3 SwingAxis = SwingAxes[SwingAxisIndex];
					TestSwingTwistOrder({ SwingAxis, SwingAngleDeg, TwistAngleDeg });
					TestSwingTwistDecomposition({ SwingAxis, SwingAngleDeg, TwistAngleDeg });
				}
			}
		}
	}

	void TestEllipticalConeAxisError(const FVec3& Axis, FReal AngleDeg, FReal LimitYDeg, FReal LimitZDeg, const FVec3& ExpectedAxis, FReal ExpectedErrorDeg)
	{
		FReal Angle = FMath::DegreesToRadians(AngleDeg);
		FReal LimitY = FMath::DegreesToRadians(LimitYDeg);
		FReal LimitZ = FMath::DegreesToRadians(LimitZDeg);
		FRotation3 RTwist = FRotation3::FromIdentity();
		FRotation3 RSwing = FRotation3::FromAxisAngle(Axis, Angle);

		FVec3 AxisLocal;
		FReal Error;
		FPBDJointUtilities::GetEllipticalConeAxisErrorLocal(RTwist, RSwing, LimitY, LimitZ, AxisLocal, Error);

		FReal ErrorDeg = FMath::RadiansToDegrees(Error);

		// About 0.5deg tolerance on error angle
		EXPECT_NEAR(ExpectedErrorDeg, ErrorDeg, 0.5f) << "Axis: (" << Axis.X << ", " << Axis.Y << ", " << Axis.Z << "); Angle = " << AngleDeg;
		EXPECT_NEAR(AxisLocal.Size(), 1.0f, KINDA_SMALL_NUMBER);

		// About 1deg error tolerance on axis
		FReal AxisDot = FVec3::DotProduct(AxisLocal, ExpectedAxis);
		EXPECT_NEAR(AxisDot, 1.0f, 0.02f);
	}


	GTEST_TEST(JointUtilitiesTes, TestEllipticalConeAxisError)
	{
		// Test Swing along Y Minor Axis
		{
			FVec3 Axis = FVec3(0, 1, 0);
			FReal AngleDeg = 40.0f;
			FReal LimitYDeg = 20.0f;
			FReal LimitZDeg = 30.0f;
			TestEllipticalConeAxisError(Axis, AngleDeg, LimitYDeg, LimitZDeg, Axis, AngleDeg - LimitYDeg);
		}

		// Test Swing along Y Major Axis
		{
			FVec3 Axis = FVec3(0, 1, 0);
			FReal AngleDeg = 40.0f;
			FReal LimitYDeg = 30.0f;
			FReal LimitZDeg = 20.0f;
			TestEllipticalConeAxisError(Axis, AngleDeg, LimitYDeg, LimitZDeg, Axis, AngleDeg - LimitYDeg);
		}

		// Test Swing along Z Minor Axis
		{
			FVec3 Axis = FVec3(0, 0, 1);
			FReal AngleDeg = 40.0f;
			FReal LimitYDeg = 30.0f;
			FReal LimitZDeg = 20.0f;
			TestEllipticalConeAxisError(Axis, AngleDeg, LimitYDeg, LimitZDeg, Axis, AngleDeg - LimitZDeg);
		}

		// Test Swing along Z Major Axis
		{
			FVec3 Axis = FVec3(0, 0, 1);
			FReal AngleDeg = 40.0f;
			FReal LimitYDeg = 20.0f;
			FReal LimitZDeg = 30.0f;
			TestEllipticalConeAxisError(Axis, AngleDeg, LimitYDeg, LimitZDeg, Axis, AngleDeg - LimitZDeg);
		}

		// Test Cicular
		{
			for (int32 Deg = 0; Deg < 360; ++Deg)
			{
				FReal AxisDeg = (FReal)Deg;
				FVec3 Axis = FRotation3::FromAxisAngle(FVec3(1, 0, 0), FMath::DegreesToRadians(AxisDeg)) *  FVec3(0, 1, 0);
				FReal AngleDeg = 40.0f;
				FReal LimitDeg = 20.0f;
				TestEllipticalConeAxisError(Axis, AngleDeg, LimitDeg, LimitDeg, Axis, AngleDeg - LimitDeg);
			}
		}

		// Test Elliptical
		{
			FReal ExpectedErrorDeg = 20.0f;
			FReal LimitYDeg = 20.0f;
			FReal LimitZDeg = 30.0f;
			const int32 NumSteps = 360;
			for (int32 Step = 0; Step < NumSteps; ++Step)
			{
				FReal S = (FReal)Step / (FReal)NumSteps;

				// Find the point on the ellipse for parameter S, and then a point with the specified error along the normal
				// https://mathworld.wolfram.com/Ellipse.html
				FReal CosS = FMath::Cos(S * 2.0f * PI);
				FReal SinS = FMath::Sin(S * 2.0f * PI);
				FReal TDenom = FMath::Sqrt(LimitYDeg * LimitYDeg * CosS * CosS + LimitZDeg * LimitZDeg * SinS * SinS);
				FVec2 P = FVec2(LimitZDeg * CosS, LimitYDeg * SinS);						// point on elipse for parameter S
				FVec2 T = FVec2(-LimitZDeg * SinS / TDenom, LimitYDeg * CosS / TDenom);		// tangent to ellipse at P
				FVec2 N = FVec2(T.Y, -T.X).GetSafeNormal();									// normal to ellipse at P

				// Point "error" degrees away from ellipse
				FVec3 NetP = FVec3(0.0f, P.X, P.Y) + FVec3(0.0f, N.X, N.Y) * ExpectedErrorDeg;

				// Get the axis and to rotate about to get to the off-ellipse point we want
				FReal AngleDeg = NetP.Size();
				FVec3 NetPDir = NetP / AngleDeg;
				FVec3 Axis = FVec3::CrossProduct(FVec3(1, 0, 0), NetPDir);

				FVec3 ExpectedAxis = FVec3(0.0f, T.X, T.Y);
				TestEllipticalConeAxisError(Axis, AngleDeg, LimitYDeg, LimitZDeg, ExpectedAxis, ExpectedErrorDeg);
			}
		}

	}

}