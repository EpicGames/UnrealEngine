// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestJoint.h"
#include "Modules/ModuleManager.h"
#include "Chaos/ParticleHandle.h"

namespace ChaosTest {

	using namespace Chaos;

	// Create a joint between a kinematic and dynamic body set up as a linear spring (using the drive)
	// and wait for it to sleep. Then move the kinematic and verify that the dynamic gets woken.
	GTEST_TEST(JointTests, TestWakeOnKinematicMove)
	{
		const int32 NumSolverIterations = 10;
		const FReal Gravity = 1000;
		const FReal Dt = 0.01f;

		FJointChainTest<FPBDRigidsEvolutionGBF> Test(NumSolverIterations, Gravity);
		Test.EnableSleeping();
		Test.InitChain(2, FVec3(0, 0, -1));

		// Set up the joint as a critically damped spring with a known resting extension under gravity
		const FReal Extension = 10;
		const FReal Stiffness = Gravity / Extension;
		const FReal Damping = 2 * FMath::Sqrt(Stiffness);
		Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };
		Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };
		Test.JointSettings[0].bLinearPositionDriveEnabled = { false, false, true };
		Test.JointSettings[0].bLinearVelocityDriveEnabled = { false, false, true };
		Test.JointSettings[0].LinearDriveForceMode = EJointForceMode::Acceleration;
		Test.JointSettings[0].LinearDriveStiffness = FVec3(0, 0, Stiffness);
		Test.JointSettings[0].LinearDriveDamping = FVec3(0, 0, Damping);

		Test.Create();

		FGenericParticleHandle P0 = Test.GetParticle(0);
		FConstGenericParticleHandle P1 = Test.GetParticle(1);
		FPBDJointConstraintHandle* J01 = Test.GetJoint(0);

		// Run the sim till the dynamic sleeps (should be less than 1s of sim time)
		const int32 MaxSteps = 200;
		for (int32 Step = 0; Step < MaxSteps; ++Step)
		{
			Test.Advance(Dt);

			if (P1->IsSleeping())
			{
				break;
			}
		}

		EXPECT_TRUE(P1->IsSleeping());
		if (P1->IsSleeping())
		{
			// Verify that the joint we created is still in the constraint graph
			// This was the source of a wake bug in the past
			EXPECT_TRUE(J01->IsInConstraintGraph());

			// Move the kinematic and tick
			const FKinematicTarget KinematicTarget = FKinematicTarget::MakePositionTarget(FRigidTransform3(FVec3(50, 0, 0), FRotation3::FromIdentity()));
			Test.Evolution.SetParticleKinematicTarget(P0->CastToKinematicParticle(), KinematicTarget);

			Test.Advance(Dt);

			EXPECT_FALSE(P1->IsSleeping());
		}
	}

	// Verify that a joint between 2 kinematics is removed from the graph anmd re-added
	// if one of the particles becomes dynamic again.
	GTEST_TEST(JointTests, TestKinematicKinematicGraph)
	{
		const int32 NumSolverIterations = 10;
		const FReal Gravity = 1000;
		const FReal Dt = 0.01f;

		FJointChainTest<FPBDRigidsEvolutionGBF> Test(NumSolverIterations, Gravity);
		Test.EnableSleeping();
		Test.InitChain(2, FVec3(0, 0, -1));

		// Set up the joint as a critically damped spring with a known resting extension under gravity
		const FReal Extension = 10;
		const FReal Stiffness = Gravity / Extension;
		const FReal Damping = 2 * FMath::Sqrt(Stiffness);
		Test.JointSettings[0].LinearMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };
		Test.JointSettings[0].AngularMotionTypes = { EJointMotionType::Free, EJointMotionType::Free, EJointMotionType::Free };
		Test.JointSettings[0].bLinearPositionDriveEnabled = { false, false, true };
		Test.JointSettings[0].bLinearVelocityDriveEnabled = { false, false, true };
		Test.JointSettings[0].LinearDriveForceMode = EJointForceMode::Acceleration;
		Test.JointSettings[0].LinearDriveStiffness = FVec3(0, 0, Stiffness);
		Test.JointSettings[0].LinearDriveDamping = FVec3(0, 0, Damping);

		Test.Create();

		FGenericParticleHandle P0 = Test.GetParticle(0);
		FGenericParticleHandle P1 = Test.GetParticle(1);
		FPBDJointConstraintHandle* J01 = Test.GetJoint(0);

		// Run the sim till the dynamic sleeps (should be less than 1s of sim time)
		const int32 MaxSteps = 200;
		for (int32 Step = 0; Step < MaxSteps; ++Step)
		{
			Test.Advance(Dt);

			if (P1->IsSleeping())
			{
				break;
			}
		}

		// Turn the dynamic into a kinematic and tick.
		// The joint should still be removed from the graph.
		// NOTE: This is different to the behaviour of collisions where we keep kinematic-kinematic collisions
		// in the graph. That is because collisions are transient, but we want the collision to remain allocated
		// for when the particle becomes dynamic again. We don't need this for Joints because the joints are
		// persistent and we can just re-add them when the particle becomes dynamic again.
		Test.Evolution.SetParticleObjectState(P1->CastToRigidParticle(), EObjectStateType::Kinematic);
		Test.Advance(Dt);

		EXPECT_TRUE(P1->IsKinematic());
		EXPECT_FALSE(J01->IsInConstraintGraph());

		// Now make the particle dynamic again. 
		// The joint should get re-added to the graph.
		Test.Evolution.SetParticleObjectState(P1->CastToRigidParticle(), EObjectStateType::Dynamic);
		Test.Advance(Dt);

		EXPECT_TRUE(P1->IsDynamic());
		EXPECT_FALSE(P1->IsSleeping());
		EXPECT_TRUE(J01->IsInConstraintGraph());
	}

}