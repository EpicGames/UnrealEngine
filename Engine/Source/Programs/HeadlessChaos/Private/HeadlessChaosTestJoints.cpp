// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestConstraints.h"

#include "Modules/ModuleManager.h"
#include "Chaos/ChaosEngineInterface.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/PBDRigidDynamicSpringConstraints.h"
#include "Chaos/PBDRigidSpringConstraints.h"
#include "Chaos/Box.h"
#include "Chaos/Plane.h"
#include "Chaos/Sphere.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Levelset.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Utilities.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/JointConstraintProxy.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"

namespace ChaosTest {

	using namespace Chaos;


	/**
	 * Base class for joint constraint tests.
	 * Initialize the particle and joint data in the test code and call Create()
	 */
	template <typename TEvolution>
	class FJointConstraintsTest : public FConstraintsTest<TEvolution>
	{
	public:
		using Base = FConstraintsTest<TEvolution>;
		using Base::Evolution;
		using Base::AddParticleBox;
		using Base::GetParticle;

		FJointConstraintsTest(const int32 NumIterations, const FReal Gravity)
			: Base(NumIterations, Gravity)
		{
		}

		FPBDJointConstraintHandle* AddJoint(const TVec2<FGeometryParticleHandle*>& InConstrainedParticles, const FVec3& InLocation)
		{
			FPBDJointSettings SettingsTemp;
			SetTestJointSetting(SettingsTemp);

			FPBDJointConstraintHandle* Constraint = nullptr;

			if (SettingsTemp.bUseLinearSolver)
			{
				Constraint = Evolution.GetJointCombinedConstraints().LinearConstraints.AddConstraint(InConstrainedParticles, FRigidTransform3(InLocation, FRotation3::FromIdentity()));
			}
			else
			{
				Constraint = Evolution.GetJointCombinedConstraints().NonLinearConstraints.AddConstraint(InConstrainedParticles, FRigidTransform3(InLocation, FRotation3::FromIdentity()));
			}

			FPBDJointSettings Settings = Constraint->GetSettings();			
			Settings.LinearProjection = 1.0;
			SetTestJointSetting(Settings);
			Constraint->SetSettings(Settings);

			// @todo(chaos): this should be automatic, but it's handled by the proxy. Fix this
			InConstrainedParticles[0]->ParticleConstraints().Add(Constraint);
			InConstrainedParticles[1]->ParticleConstraints().Add(Constraint);

			return Constraint;
		}

		virtual void Create()
		{
			for (int32 ParticleIndex = 0; ParticleIndex < ParticlePositions.Num(); ++ParticleIndex)
			{
				auto* Particle = AddParticleBox(ParticlePositions[ParticleIndex], FRotation3::MakeFromEuler(FVec3(0.f, 0.f, 0.f)).GetNormalized(), ParticleSizes[ParticleIndex], ParticleMasses[ParticleIndex]);
				Evolution.EnableParticle(Particle);
			}

			for (int32 JointIndex = 0; JointIndex < JointPositions.Num(); ++JointIndex)
			{
				const TVec2<FGeometryParticleHandle*> ConstraintedParticleIds(GetParticle(JointParticleIndices[JointIndex][0]), GetParticle(JointParticleIndices[JointIndex][1]));
				FPBDJointConstraintHandle*  Joint = AddJoint(ConstraintedParticleIds, JointPositions[JointIndex]);
				Joints.Add(Joint);
			}
		}

		int32 GetNumJoints() const
		{
			return Joints.Num();
		}

		FPBDJointConstraintHandle* GetJoint(int32 Index)
		{
			return Joints[Index];
		}

		TFunction<void(FPBDJointSettings&)>& SetSetTestJointSetting() { return SetTestJointSetting; }

		// Initial particles setup
		TArray<FVec3> ParticlePositions;
		TArray<FVec3> ParticleSizes;
		TArray<FReal> ParticleMasses;

		// Initial joints setup
		TArray<FVec3> JointPositions;
		TArray<TVec2<int32>> JointParticleIndices;

		TFunction<void(FPBDJointSettings&)> SetTestJointSetting = [](FPBDJointSettings& InputSetting) {};

		// Created joints
		TArray<FPBDJointConstraintHandle*> Joints;
	};

	/**
	 * One kinematic, one dynamic particle connected by a ball-socket joint in the middle.
	 */
	template <typename TEvolution>
	void JointConstraint_Single()
	{

		const int32 NumIterations = 1;
		const FReal Gravity = 980;

		FJointConstraintsTest<TEvolution> Test(NumIterations, Gravity);

		Test.ParticlePositions =
		{
			{ (FReal)0, (FReal)0, (FReal)1000 },
			{ (FReal)500, (FReal)0, (FReal)1000 },
		};
		Test.ParticleSizes =
		{
			{ (FReal)100, (FReal)100, (FReal)100 },
			{ (FReal)100, (FReal)100, (FReal)100 },
		};
		Test.ParticleMasses =
		{
			(FReal)0,
			(FReal)1,
		};

		Test.JointPositions =
		{
			{ (FReal)250, (FReal)0, (FReal)1000 },
		};
		Test.JointParticleIndices =
		{
			{ 0, 1 },
		};

		Test.SetSetTestJointSetting() = [](FPBDJointSettings& JointSettings)
			{
				JointSettings.bUseLinearSolver = false;
			};


		Test.Create();

		const int32 Box1Id = 0;
		const int32 Box2Id = 1;
		const FReal ExpectedDistance = (Test.ParticlePositions[1] - Test.ParticlePositions[0]).Size();
		const FVec3 Box2LocalSpaceJointPosition = Test.JointPositions[0] - Test.ParticlePositions[1];

		const FReal Dt = 0.01f;
		for (int32 i = 0; i < 100; ++i)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);


			// Joint position calculated from pose and local-space joint pos
			const FVec3 Box2WorldSpaceJointPosition = Test.GetParticle(Box2Id)->GetR().RotateVector(Box2LocalSpaceJointPosition) + Test.GetParticle(Box2Id)->GetX();
			EXPECT_LT((Box2WorldSpaceJointPosition - Test.JointPositions[0]).Size(), (FReal)0.1);

			// Kinematic particle should not have moved
			EXPECT_LT((Test.GetParticle(Box1Id)->GetX() - Test.ParticlePositions[0]).Size(), (FReal)0.1);
		}
	}

	template <typename TEvolution>
	void JointConstraint_SingleMoveRoot()
	{
		const int32 NumIterations = 10;
		const FReal Gravity = 0;
		const FReal BoxSize = 1;
		const FReal BoxMass = 1;
		const FReal Dt = (FReal)1 / 20;
		const FVec3 RootDelta(1 * BoxSize, 0, 0);

		FJointConstraintsTest<TEvolution> Test(NumIterations, Gravity);

		Test.ParticlePositions =
		{
			{ (FReal)0, (FReal)0, (FReal)10 * BoxSize },
			{ (FReal)0, (FReal)0, (FReal)5 * BoxSize },
		};
		Test.ParticleSizes =
		{
			{ (FReal)BoxSize, (FReal)BoxSize, (FReal)BoxSize },
			{ (FReal)BoxSize, (FReal)BoxSize, (FReal)BoxSize },
		};
		Test.ParticleMasses =
		{
			(FReal)0,
			(FReal)BoxMass,
		};

		Test.JointPositions =
		{
			Test.ParticlePositions[0],
		};
		Test.JointParticleIndices =
		{
			{ 0, 1 },
		};

		Test.Create();

		const int32 Box1Id = 0;
		const int32 Box2Id = 1;
		const FReal ExpectedDistance = (Test.ParticlePositions[1] - Test.ParticlePositions[0]).Size();
		const FVec3 Box2LocalSpaceJointPosition = Test.JointPositions[0] - Test.ParticlePositions[1];

		// Everything should be in a stable state
		for (int32 i = 0; i < 10; ++i)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			// Nothing should have moved
			for (int32 ParticleIndex = 0; ParticleIndex < Test.ParticlePositions.Num(); ++ParticleIndex)
			{
				EXPECT_LT((Test.GetParticle(ParticleIndex)->GetX() - Test.ParticlePositions[ParticleIndex]).Size(), (FReal)0.1) << "Initial configuration instability on frame " << i;
			}
		}

		// Move the kinematic body
		const FVec3 RootPosition = Test.ParticlePositions[0] + RootDelta;
		Test.Evolution.SetParticleKinematicTarget(Test.GetParticle(Box1Id)->CastToKinematicParticle(), FKinematicTarget::MakePositionTarget(RootPosition, Test.GetParticle(Box1Id)->GetR()));

		for (int32 i = 0; i < 1000; ++i)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			// Kinematic particle should have moved to animated position
			EXPECT_LT((Test.GetParticle(Box1Id)->GetX() - RootPosition).Size(), (FReal)0.1 * BoxSize) << "Post-move instability on frame " << i;

			// Particles should remain fixed distance apart (joint point is at Box1 location)
			// NOTE: when using linear joints the error can be moderately large
			const FVec3 Delta = Test.GetParticle(Box2Id)->GetX() - Test.GetParticle(Box1Id)->GetX();
			const FReal Distance = Delta.Size();
			EXPECT_NEAR(Distance, ExpectedDistance, (FReal)0.15 * BoxSize) << "Post-move instability on frame " << i;

			// Joint position calculted from pose and local-space joint pos
			const FVec3 Box2WorldSpaceJointPosition = Test.GetParticle(Box2Id)->GetR().RotateVector(Box2LocalSpaceJointPosition) + Test.GetParticle(Box2Id)->GetX();
			EXPECT_LT((Box2WorldSpaceJointPosition - RootPosition).Size(), (FReal)0.15 * BoxSize) << "Post-move instability on frame " << i;
		}
	}


	/**
	 * Pendulum with animated root.
	 */
	template <typename TEvolution>
	void JointConstraint_SingleAnimated()
	{
		const int32 NumIterations = 5;
		const FReal Gravity = 980;
		const FReal BoxSize = 100;
		const FReal BoxMass = 1000;
		const FReal Dt = (FReal)1 / 20;
		const FReal AnimPeriod = (FReal)2;
		const FVec3 AnimDelta = FVec3(BoxSize, 0, 0);

		FJointConstraintsTest<TEvolution> Test(NumIterations, Gravity);

		Test.ParticlePositions =
		{
			{ (FReal)0, (FReal)0, (FReal)10 * BoxSize },
			{ (FReal)0, (FReal)2 * BoxSize, (FReal)10 * BoxSize },
		};
		Test.ParticleSizes =
		{
			{ (FReal)BoxSize, (FReal)BoxSize, (FReal)BoxSize },
			{ (FReal)BoxSize, (FReal)BoxSize, (FReal)BoxSize },
		};
		Test.ParticleMasses =
		{
			(FReal)0,
			(FReal)BoxMass,
		};

		Test.JointPositions =
		{
			Test.ParticlePositions[0],
		};
		Test.JointParticleIndices =
		{
			{ 0, 1 },
		};

		Test.Create();

		const int32 Box1Id = 0;
		const int32 Box2Id = 1;
		const FReal ExpectedDistance = (Test.ParticlePositions[1] - Test.ParticlePositions[0]).Size();
		const FVec3 Box2LocalSpaceJointPosition = Test.JointPositions[0] - Test.ParticlePositions[1];

		for (int32 i = 0; i < 1000; ++i)
		{
			const FReal Time = i * Dt;
			const FVec3 RootOffset = FMath::Sin((FReal)2 * PI * Time / AnimPeriod) * AnimDelta;
			const FVec3 RootPosition = Test.ParticlePositions[0] + RootOffset;

			Test.Evolution.SetParticleKinematicTarget(Test.GetParticle(Box1Id)->CastToKinematicParticle(), FKinematicTarget::MakePositionTarget(RootPosition, Test.GetParticle(Box1Id)->GetR()));

			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			// Kinematic particle should have moved to animated position
			EXPECT_LT((Test.GetParticle(Box1Id)->GetX() - RootPosition).Size(), (FReal)1) << "Failed on frame " << i;

			// Particles should remain fixed distance apart (joint point is at Box1 location)
			const FVec3 Delta = Test.GetParticle(Box2Id)->CastToRigidParticle()->GetP() - Test.GetParticle(Box1Id)->GetX();
			const FReal Distance = Delta.Size();
			EXPECT_NEAR(Distance, ExpectedDistance, (FReal)1) << "Failed on frame " << i;

			// Joint position calculated from pose and local-space joint pos
			const FVec3 Box2WorldSpaceJointPosition = Test.GetParticle(Box2Id)->GetR().RotateVector(Box2LocalSpaceJointPosition) + Test.GetParticle(Box2Id)->GetX();
			EXPECT_LT((Box2WorldSpaceJointPosition - RootPosition).Size(), (FReal)1) << "Failed on frame " << i;
		}
	}

	/**
	 * Pendulum with animated root.
	 */
	template <typename TEvolution>
	void JointConstraint_ShortChainAnimated(const bool bProjectionEnabled)
	{
		const int32 NumIterations = 10;
		const FReal Gravity = 980;
		const FReal BoxSize = 100;
		const FReal BoxMass = 1000;
		const FReal Dt = (FReal)1 / 100;
		const FReal AnimPeriod = (FReal)1;
		const FVec3 AnimDelta = FVec3(5 * BoxSize, 0, 0);
		const FReal AcceptableDistanceError = 5;

		FJointConstraintsTest<TEvolution> Test(NumIterations, Gravity);

		Test.ParticlePositions =
		{
			{ (FReal)0, (FReal)0, (FReal)20 * BoxSize },
			{ (FReal)0, (FReal)2 * BoxSize, (FReal)20 * BoxSize },
			{ (FReal)0, (FReal)4 * BoxSize, (FReal)20 * BoxSize },
		};
		Test.ParticleSizes =
		{
			{ (FReal)BoxSize, (FReal)BoxSize, (FReal)BoxSize },
			{ (FReal)BoxSize, (FReal)BoxSize, (FReal)BoxSize },
			{ (FReal)BoxSize, (FReal)BoxSize, (FReal)BoxSize },
		};
		Test.ParticleMasses =
		{
			(FReal)0,
			(FReal)BoxMass,
			(FReal)BoxMass,
		};

		Test.JointPositions =
		{
			Test.ParticlePositions[0],
			Test.ParticlePositions[1],
		};
		Test.JointParticleIndices =
		{
			{ 0, 1 },
			{ 1, 2 },
		};

		Test.Create();

		for (FPBDJointConstraintHandle* Joint : Test.Joints)
		{
			FPBDJointSettings Settings = Joint->GetSettings();
			Settings.bProjectionEnabled = bProjectionEnabled;
			Joint->SetSettings(Settings);
		}

		const FVec3 Box2LocalSpaceJointPosition = Test.JointPositions[0] - Test.ParticlePositions[1];

		FReal MaxDistanceError = 0.0f;
		int32 MaxDistanceErrorFrameIndex = INDEX_NONE;
		for (int32 FrameIndex = 0; FrameIndex < 1000; ++FrameIndex)
		{
			const FReal Time = FrameIndex * Dt;
			const FVec3 RootOffset = FMath::Sin((FReal)2 * PI * Time / AnimPeriod) * AnimDelta;
			const FVec3 RootPosition = Test.ParticlePositions[0] + RootOffset;

			Test.Evolution.SetParticleKinematicTarget(Test.GetParticle(0)->CastToKinematicParticle(), FKinematicTarget::MakePositionTarget(RootPosition, Test.GetParticle(0)->GetR()));

			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			// Particles should remain fixed distance apart
			for (int JointIndex = 0; JointIndex < Test.JointPositions.Num(); ++JointIndex)
			{
				const int32 ParticleIndex1 = Test.JointParticleIndices[JointIndex][0];
				const int32 ParticleIndex2 = Test.JointParticleIndices[JointIndex][1];
				const FVec3 Delta = Test.GetParticle(ParticleIndex2)->CastToRigidParticle()->GetP() - Test.GetParticle(ParticleIndex1)->GetX();
				const FReal Distance = Delta.Size();
				const FReal ExpectedDistance = (Test.ParticlePositions[ParticleIndex2] - Test.ParticlePositions[ParticleIndex1]).Size();
				EXPECT_NEAR(Distance, ExpectedDistance, AcceptableDistanceError) << "Joint " << JointIndex << " on frame " << FrameIndex;

				const FReal DistanceError = FMath::Abs(Distance - ExpectedDistance);
				if (DistanceError > MaxDistanceError)
				{
					MaxDistanceError = DistanceError;
					MaxDistanceErrorFrameIndex = FrameIndex;
				}
			}
		}
		EXPECT_LT(MaxDistanceError, AcceptableDistanceError) << "On frame " << MaxDistanceErrorFrameIndex;
	}

	/**
	 * Pendulum with animated root.
	 */
	template <typename TEvolution>
	void JointConstraint_LongChainAnimated(const bool bProjectionEnabled)
	{
		const int NumParticles = 10;
		const int32 NumIterations = 20;
		const FReal Gravity = 980;
		const FReal BoxSize = 100;
		const FReal BoxMass = 1000;
		const FReal Dt = (FReal)1 / 20;
		const FReal AnimPeriod = (FReal)1;
		const FVec3 AnimDelta = FVec3(1 * BoxSize, 0, 0);
		const FReal AcceptableDistanceError = bProjectionEnabled ? 6.5 : 5;
		const FReal Separation = 2 * BoxSize;
		const FVec3 Begin = FVec3(0, 0, (NumParticles + 10) * Separation);
		const FVec3 Dir = FVec3(0, 1, 0);
		const bool bRandomizeConstraintOrder = true;

		FMath::RandInit(1048604845);

		// Create a chain of connected particles, with particle 0 fixed
		FJointConstraintsTest<TEvolution> Test(NumIterations, Gravity);
		for (int32 ParticleIndex = 0; ParticleIndex < NumParticles; ++ParticleIndex)
		{
			Test.ParticlePositions.Add(Begin + ParticleIndex * Separation * Dir);
			Test.ParticleSizes.Add({ BoxSize, BoxSize, BoxSize });
			Test.ParticleMasses.Add((ParticleIndex == 0) ? (FReal)0 : BoxMass);
		}
		for (int32 JointIndex = 0; JointIndex < NumParticles - 1; ++JointIndex)
		{
			Test.JointPositions.Add(Test.ParticlePositions[JointIndex]);
			Test.JointParticleIndices.Add({ JointIndex, JointIndex + 1 });
		}

		// Randomize constraint order
		if (bRandomizeConstraintOrder)
		{
			for (int32 JointIndex = 0; JointIndex < Test.JointParticleIndices.Num(); ++JointIndex)
			{
				const int32 Index0 = JointIndex;
				const int32 Index1 = FMath::RandRange(0, Test.JointParticleIndices.Num() - 1);
				Test.JointPositions.Swap(Index0, Index1);
				Test.JointParticleIndices.Swap(Index0, Index1);
			}
		}

		Test.Create();

		for (FPBDJointConstraintHandle* Joint : Test.Joints)
		{
			FPBDJointSettings Settings = Joint->GetSettings();
			Settings.bProjectionEnabled = bProjectionEnabled;
			Joint->SetSettings(Settings);
		}

		const FVec3 Box2LocalSpaceJointPosition = Test.JointPositions[0] - Test.ParticlePositions[1];

		FReal MaxDistanceError = 0.0f;
		int32 MaxDistanceErrorFrameIndex = INDEX_NONE;
		for (int32 FrameIndex = 0; FrameIndex < 1000; ++FrameIndex)
		{
			const FReal Time = FrameIndex * Dt;
			const FVec3 RootOffset = FMath::Sin((FReal)2 * PI * Time / AnimPeriod) * AnimDelta;
			const FVec3 RootPosition = Test.ParticlePositions[0] + RootOffset;

			Test.Evolution.SetParticleKinematicTarget(Test.GetParticle(0)->CastToKinematicParticle(), FKinematicTarget::MakePositionTarget(RootPosition, Test.GetParticle(0)->GetR()));

			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			// Particles should remain fixed distance apart
			for (int JointIndex = 0; JointIndex < Test.JointPositions.Num(); ++JointIndex)
			{
				const int32 ParticleIndex1 = Test.JointParticleIndices[JointIndex][0];
				const int32 ParticleIndex2 = Test.JointParticleIndices[JointIndex][1];
				const FVec3 Delta = Test.GetParticle(ParticleIndex2)->CastToRigidParticle()->GetP() - Test.GetParticle(ParticleIndex1)->GetX();
				const FReal Distance = Delta.Size();
				const FReal ExpectedDistance = (Test.ParticlePositions[ParticleIndex2] - Test.ParticlePositions[ParticleIndex1]).Size();
				EXPECT_NEAR(Distance, ExpectedDistance, AcceptableDistanceError) << "Joint " << JointIndex << " on frame " << FrameIndex;

				// Track largest error that exceeds threshold
				const FReal DistanceError = FMath::Abs(Distance - ExpectedDistance);
				if (DistanceError > MaxDistanceError)
				{
					MaxDistanceError = DistanceError;
					MaxDistanceErrorFrameIndex = FrameIndex;
				}
			}
		}
		// Report the largest error and when it occurred if it exceeded the threshold
		EXPECT_LT(MaxDistanceError, AcceptableDistanceError) << "On frame " << MaxDistanceErrorFrameIndex;
	}

	/**
	 * Pendulum with animated root with varying number of iterations.
	 */
	template <typename TEvolution>
	void JointConstraint_LongChainVaryingIterations()
	{
		const int NumParticles = 10;
		const int32 NumIterations = 20;
		const FReal Gravity = 980;
		const FReal BoxSize = 100;
		const FReal BoxMass = 1000;
		const FReal Dt = (FReal)1 / 20;
		const FReal AcceptableDistanceError = 5;
		const FReal Separation = 2 * BoxSize;
		const FVec3 Begin = FVec3(0, 0, (NumParticles + 10) * Separation);
		const FVec3 Dir = FVec3(0, 1, 0);

		// Create a chain of connected particles, with particle 0 fixed
		FJointConstraintsTest<TEvolution> Test(NumIterations, Gravity);
		for (int32 ParticleIndex = 0; ParticleIndex < NumParticles; ++ParticleIndex)
		{
			Test.ParticlePositions.Add(Begin + ParticleIndex * Separation * Dir);
			Test.ParticleSizes.Add({ BoxSize, BoxSize, BoxSize });
			Test.ParticleMasses.Add((ParticleIndex == 0) ? (FReal)0 : BoxMass);
		}
		for (int32 JointIndex = 0; JointIndex < NumParticles - 1; ++JointIndex)
		{
			Test.JointPositions.Add(Test.ParticlePositions[JointIndex]);
			Test.JointParticleIndices.Add({ JointIndex, JointIndex + 1 });
		}

		Test.Create();

		const FVec3 Box2LocalSpaceJointPosition = Test.JointPositions[0] - Test.ParticlePositions[1];

		FReal MaxDistanceErrorBigIterations = 0.0f;
		FReal MaxDistanceErrorSmallIterations = 0.0f;

		const int32 SmallPositionIterations = 1;
		const int32 SmallVelocityIterations = 1;
		const int32 SmallProjectionIterations = 1;
		const int32 BigPositionIterations = 100;
		const int32 BigVelocityIterations = 100;
		const int32 BigProjectionIterations = 100;

		Test.Evolution.SetNumPositionIterations(BigPositionIterations);
		Test.Evolution.SetNumVelocityIterations(BigVelocityIterations);
		Test.Evolution.SetNumProjectionIterations(BigProjectionIterations);

		const FVec3 RootPosition = Test.ParticlePositions[0];
		Test.Evolution.SetParticleKinematicTarget(Test.GetParticle(0)->CastToKinematicParticle(), FKinematicTarget::MakePositionTarget(RootPosition, Test.GetParticle(0)->GetR()));


		for (int32 FrameIndex = 0; FrameIndex < 10; ++FrameIndex)
		{
			const FReal Time = FrameIndex * Dt;
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			Chaos::Private::FIterationSettings GroupIterations = Test.Evolution.GetIslandGroupManager().GetIslandGroupIterations(0);

			EXPECT_EQ(GroupIterations.GetNumPositionIterations(), 100);
			EXPECT_EQ(GroupIterations.GetNumVelocityIterations(), 100);
			EXPECT_EQ(GroupIterations.GetNumProjectionIterations(), 100);
		}

		// Particles should remain fixed distance apart
		for (int JointIndex = 0; JointIndex < Test.JointPositions.Num(); ++JointIndex)
		{
			const int32 ParticleIndex1 = Test.JointParticleIndices[JointIndex][0];
			const int32 ParticleIndex2 = Test.JointParticleIndices[JointIndex][1];
			const FVec3 Delta = Test.GetParticle(ParticleIndex2)->CastToRigidParticle()->GetP() - Test.GetParticle(ParticleIndex1)->GetX();
			const FReal Distance = Delta.Size();
			const FReal ExpectedDistance = (Test.ParticlePositions[ParticleIndex2] - Test.ParticlePositions[ParticleIndex1]).Size();
			EXPECT_NEAR(Distance, ExpectedDistance, AcceptableDistanceError) << "Joint " << JointIndex;

			// Track largest error that exceeds threshold
			const FReal DistanceError = FMath::Abs(Distance - ExpectedDistance);
			if (DistanceError > MaxDistanceErrorBigIterations)
			{
				MaxDistanceErrorBigIterations = DistanceError;
			}
		}

		// Report the largest error and when it occurred if it exceeded the threshold
		EXPECT_LT(MaxDistanceErrorBigIterations, AcceptableDistanceError);

		Test.Evolution.SetNumPositionIterations(SmallPositionIterations);
		Test.Evolution.SetNumVelocityIterations(SmallVelocityIterations);
		Test.Evolution.SetNumProjectionIterations(SmallProjectionIterations);

		for (int32 FrameIndex = 0; FrameIndex < 10; ++FrameIndex)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			Chaos::Private::FIterationSettings GroupIterations = Test.Evolution.GetIslandGroupManager().GetIslandGroupIterations(0);

			EXPECT_EQ(GroupIterations.GetNumPositionIterations(), 1);
			EXPECT_EQ(GroupIterations.GetNumVelocityIterations(), 1);
			EXPECT_EQ(GroupIterations.GetNumProjectionIterations(), 1);

		}
		// Check max distance error between particles
		for (int JointIndex = 0; JointIndex < Test.JointPositions.Num(); ++JointIndex)
		{
			const int32 ParticleIndex1 = Test.JointParticleIndices[JointIndex][0];
			const int32 ParticleIndex2 = Test.JointParticleIndices[JointIndex][1];
			const FVec3 Delta = Test.GetParticle(ParticleIndex2)->CastToRigidParticle()->GetP() - Test.GetParticle(ParticleIndex1)->GetX();
			const FReal Distance = Delta.Size();
			const FReal ExpectedDistance = (Test.ParticlePositions[ParticleIndex2] - Test.ParticlePositions[ParticleIndex1]).Size();
			const FReal DistanceError = FMath::Abs(Distance - ExpectedDistance);
			if (DistanceError > MaxDistanceErrorSmallIterations)
			{
				MaxDistanceErrorSmallIterations = DistanceError;
			}
		}
		// Report if distance error from many iterations is bigger than that from small iterations. 
		EXPECT_LT(MaxDistanceErrorBigIterations, MaxDistanceErrorSmallIterations);
	}

	/**
	 * Two pendulums, one solved using the linear solver, the other using nonlinear solver
	 */
	template <typename TEvolution>
	void JointConstraint_LongChainLinearJoints()
	{
		const int NumParticles = 10;
		const int32 NumIterations = 5;
		const FReal Gravity = 980;
		const FReal BoxLength = 0.01;
		const FReal BoxWidth = 0.005;
		const FReal BoxMass = 10;
		const FReal Dt = (FReal)1 / 20;
		const FReal Separation = 2 * BoxLength;
		const FVec3 Begin = FVec3(0, 0, (NumParticles + 10) * Separation);
		const FVec3 Dir = FVec3(0, 1, 0);

		// Create two chains of connected particles, with particle 0 fixed
		FJointConstraintsTest<TEvolution> LinearJointTest(NumIterations, Gravity), NonLinearJointTest(NumIterations, Gravity);
		for (int32 ParticleIndex = 0; ParticleIndex < NumParticles; ++ParticleIndex)
		{
			LinearJointTest.ParticlePositions.Add(Begin + ParticleIndex * Separation * Dir);
			LinearJointTest.ParticleSizes.Add({ BoxWidth, BoxLength, BoxWidth });
			LinearJointTest.ParticleMasses.Add((ParticleIndex == 0) ? (FReal)0 : BoxMass);
			NonLinearJointTest.ParticlePositions.Add(Begin + ParticleIndex * Separation * Dir);
			NonLinearJointTest.ParticleSizes.Add({ BoxWidth, BoxLength, BoxWidth });
			NonLinearJointTest.ParticleMasses.Add((ParticleIndex == 0) ? (FReal)0 : BoxMass);
		}
		for (int32 JointIndex = 0; JointIndex < NumParticles - 1; ++JointIndex)
		{
			LinearJointTest.JointPositions.Add(LinearJointTest.ParticlePositions[JointIndex]);
			LinearJointTest.JointParticleIndices.Add({ JointIndex, JointIndex + 1 });
			NonLinearJointTest.JointPositions.Add(NonLinearJointTest.ParticlePositions[JointIndex]);
			NonLinearJointTest.JointParticleIndices.Add({ JointIndex, JointIndex + 1 });
		}

		LinearJointTest.SetSetTestJointSetting() = [](FPBDJointSettings& JointSettings)
			{
				JointSettings.bUseLinearSolver = true;
				JointSettings.AngularMotionTypes = TVector<EJointMotionType, 3>(EJointMotionType::Locked);
			};

		NonLinearJointTest.SetSetTestJointSetting() = [](FPBDJointSettings& JointSettings)
			{
				JointSettings.bUseLinearSolver = false;
				JointSettings.AngularMotionTypes = TVector<EJointMotionType, 3>(EJointMotionType::Locked);
			};

		LinearJointTest.Create();
		NonLinearJointTest.Create();
		for (int32 ParticleIndex = 0; ParticleIndex < NumParticles; ++ParticleIndex)
		{
			LinearJointTest.GetParticle(ParticleIndex)->CastToRigidParticle()->SetInertiaConditioningEnabled(false);
			NonLinearJointTest.GetParticle(ParticleIndex)->CastToRigidParticle()->SetInertiaConditioningEnabled(false);
		}

		const FVec3 Box2LocalSpaceJointPosition = LinearJointTest.JointPositions[0] - LinearJointTest.ParticlePositions[1];

		FReal MaxDistanceError = 0.0f;
		int32 MaxDistanceErrorFrameIndex = INDEX_NONE;
		for (int32 FrameIndex = 0; FrameIndex < 100; ++FrameIndex)
		{
			const FReal Time = FrameIndex * Dt;

			LinearJointTest.Evolution.SetParticleKinematicTarget(LinearJointTest.GetParticle(0)->CastToKinematicParticle(), FKinematicTarget::MakePositionTarget(LinearJointTest.ParticlePositions[0], LinearJointTest.GetParticle(0)->GetR()));

			LinearJointTest.Evolution.AdvanceOneTimeStep(Dt);
			LinearJointTest.Evolution.EndFrame(Dt);

			NonLinearJointTest.Evolution.SetParticleKinematicTarget(NonLinearJointTest.GetParticle(0)->CastToKinematicParticle(), FKinematicTarget::MakePositionTarget(NonLinearJointTest.ParticlePositions[0], NonLinearJointTest.GetParticle(0)->GetR()));

			NonLinearJointTest.Evolution.AdvanceOneTimeStep(Dt);
			NonLinearJointTest.Evolution.EndFrame(Dt);
		}

		// The chain solved using the nonlinear solver should have less error than the one using linear solver. 
		for (int JointIndex = 0; JointIndex < LinearJointTest.JointPositions.Num(); ++JointIndex)
		{
			const int32 ParticleIndex1 = LinearJointTest.JointParticleIndices[JointIndex][0];
			const int32 ParticleIndex2 = LinearJointTest.JointParticleIndices[JointIndex][1];
			const FVec3 LinearDelta = LinearJointTest.GetParticle(ParticleIndex2)->CastToRigidParticle()->GetP() - LinearJointTest.GetParticle(ParticleIndex1)->GetX();
			const FReal LinearDistance = LinearDelta.Size();
			const FVec3 NonLinearDelta = NonLinearJointTest.GetParticle(ParticleIndex2)->CastToRigidParticle()->GetP() - NonLinearJointTest.GetParticle(ParticleIndex1)->GetX();
			const FReal NonLinearDistance = NonLinearDelta.Size();
			const FReal ExpectedDistance = (LinearJointTest.ParticlePositions[ParticleIndex2] - LinearJointTest.ParticlePositions[ParticleIndex1]).Size();
			const FReal LinearDistanceError = FMath::Abs(ExpectedDistance - LinearDistance);
			const FReal NonLinearDistanceError = FMath::Abs(ExpectedDistance - NonLinearDistance);
			EXPECT_LT(NonLinearDistanceError, LinearDistanceError);

			//// Uncomment for debugging
			//printf("ParticleIndex1: %d\n", ParticleIndex1);
			//printf("Linear Particle1 X: %f, %f, %f\n", LinearJointTest.GetParticle(ParticleIndex1)->GetX().X, LinearJointTest.GetParticle(ParticleIndex1)->GetX().Y, LinearJointTest.GetParticle(ParticleIndex1)->GetX().Z);
			//printf("Linear Particle1 P: %f, %f, %f\n", LinearJointTest.GetParticle(ParticleIndex1)->CastToRigidParticle()->GetP().X, LinearJointTest.GetParticle(ParticleIndex1)->CastToRigidParticle()->GetP().Y, LinearJointTest.GetParticle(ParticleIndex1)->CastToRigidParticle()->GetP().Z);
			//printf("Nonlinear Particle1 X: %f, %f, %f\n", NonLinearJointTest.GetParticle(ParticleIndex1)->GetX().X, NonLinearJointTest.GetParticle(ParticleIndex1)->GetX().Y, NonLinearJointTest.GetParticle(ParticleIndex1)->GetX().Z);
			//printf("Nonlinear Particle1 P: %f, %f, %f\n", NonLinearJointTest.GetParticle(ParticleIndex1)->CastToRigidParticle()->GetP().X, NonLinearJointTest.GetParticle(ParticleIndex1)->CastToRigidParticle()->GetP().Y, NonLinearJointTest.GetParticle(ParticleIndex1)->CastToRigidParticle()->GetP().Z);
			//printf("Expected Particle1: %f, %f, %f\n", LinearJointTest.ParticlePositions[ParticleIndex1].X, LinearJointTest.ParticlePositions[ParticleIndex1].Y, LinearJointTest.ParticlePositions[ParticleIndex1].Z);
			//printf("ParticleIndex2: %d\n", ParticleIndex2);
			//printf("Linear Particle2 X: %f, %f, %f\n", LinearJointTest.GetParticle(ParticleIndex2)->GetX().X, LinearJointTest.GetParticle(ParticleIndex2)->GetX().Y, LinearJointTest.GetParticle(ParticleIndex2)->GetX().Z);
			//printf("Linear Particle2 P: %f, %f, %f\n", LinearJointTest.GetParticle(ParticleIndex2)->CastToRigidParticle()->GetP().X, LinearJointTest.GetParticle(ParticleIndex2)->CastToRigidParticle()->GetP().Y, LinearJointTest.GetParticle(ParticleIndex2)->CastToRigidParticle()->GetP().Z);
			//printf("Nonlinear Particle2 X: %f, %f, %f\n", NonLinearJointTest.GetParticle(ParticleIndex2)->GetX().X, NonLinearJointTest.GetParticle(ParticleIndex2)->GetX().Y, NonLinearJointTest.GetParticle(ParticleIndex2)->GetX().Z);
			//printf("Nonlinear Particle2 P: %f, %f, %f\n", NonLinearJointTest.GetParticle(ParticleIndex2)->CastToRigidParticle()->GetP().X, NonLinearJointTest.GetParticle(ParticleIndex2)->CastToRigidParticle()->GetP().Y, NonLinearJointTest.GetParticle(ParticleIndex2)->CastToRigidParticle()->GetP().Z);
			//printf("Expected Particle2: %f, %f, %f\n", LinearJointTest.ParticlePositions[ParticleIndex2].X, LinearJointTest.ParticlePositions[ParticleIndex2].Y, LinearJointTest.ParticlePositions[ParticleIndex2].Z);
			//printf("JointIndex: %d\n", JointIndex);
			//printf("LinearDistanceError: %f\n", LinearDistanceError);
			//printf("NonLinearDistanceError: %f\n", NonLinearDistanceError);
			//printf("LinearDistance: %f\n", LinearDistance);
			//printf("NonLinearDistance: %f\n", NonLinearDistance);
			//printf("ExpectedDistance: %f\n", ExpectedDistance);
			//printf("\n");
		}
	}


	template <typename TEvolution>
	void SpringConstraint()
	{
		TUniquePtr<FChaosPhysicsMaterial> PhysicalMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicalMaterial->Friction = 0;
		PhysicalMaterial->Restitution = 0;
		PhysicalMaterial->SleepingLinearThreshold = 0;
		PhysicalMaterial->SleepingAngularThreshold = 0;
		PhysicalMaterial->DisabledLinearThreshold = 0;
		PhysicalMaterial->DisabledAngularThreshold = 0;

		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);

		auto StaticBox = AppendStaticParticleBox(Particles, FVec3((FReal)100, (FReal)100, (FReal)100));
		auto Box2 = AppendDynamicParticleBox(Particles, FVec3((FReal)100, (FReal)100, (FReal)100));
		StaticBox->SetX(FVec3((FReal)0, (FReal)0, (FReal)1000));

		Box2->SetX(FVec3((FReal)500, (FReal)0, (FReal)1000));
		Box2->SetP(Box2->GetX());
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		TEvolution Evolution(Particles, PhysicalMaterials);
		TVec2<FGeometryParticleHandle*> ConstrainedParticles = TVec2<FGeometryParticleHandle*>(StaticBox, Box2);
		TVec2<FVec3> Points = { FVec3((FReal)100, (FReal)0, (FReal)1000), FVec3((FReal)400, (FReal)0, (FReal)1000) };

		Evolution.SetPhysicsMaterial(StaticBox, MakeSerializable(PhysicalMaterial));
		Evolution.SetPhysicsMaterial(Box2, MakeSerializable(PhysicalMaterial));

		Evolution.EnableParticle(StaticBox);
		Evolution.EnableParticle(Box2);

		FPBDRigidSpringConstraints JointConstraints;
		JointConstraints.AddConstraint(ConstrainedParticles, Points, 1.0f, 0.0f, (Points[0] - Points[1]).Size());
		Evolution.AddConstraintContainer(JointConstraints);

		
		const FReal Dt = 0.01f;
		for (int32 i = 0; i < 100; ++i)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
			EXPECT_LT(FMath::Abs((Box2->GetR().RotateVector(FVec3((FReal)-100, (FReal)0, (FReal)0)) + Box2->GetX() - Points[0]).Size() - 300.f), 0.1);
		}
	}

	template <typename TEvolution>
	void DynamicSpringConstraint()
	{
		TUniquePtr<FChaosPhysicsMaterial> PhysicalMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicalMaterial->Friction = 0;
		PhysicalMaterial->Restitution = 0;
		PhysicalMaterial->SleepingLinearThreshold = 0;
		PhysicalMaterial->SleepingAngularThreshold = 0;
		PhysicalMaterial->DisabledLinearThreshold = 0;
		PhysicalMaterial->DisabledAngularThreshold = 0;
		PhysicalMaterial->SleepCounterThreshold = 20;

		{
			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs Particles(UniqueIndices);

			auto& StaticBox = *AppendStaticParticleBox(Particles, FVec3((FReal)100, (FReal)100, (FReal)100));
			auto& Box2 = *AppendDynamicParticleBox(Particles, FVec3((FReal)100, (FReal)100, (FReal)100));
			StaticBox.SetX(FVec3((FReal)0, (FReal)0, (FReal)500));

			Box2.SetX(FVec3((FReal)500, (FReal)0, (FReal)1000));
			Box2.SetP(Box2.GetX());

			THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
			TEvolution Evolution(Particles, PhysicalMaterials);
			TArray<TVec2<FGeometryParticleHandle*>> Constraints = { TVec2<FGeometryParticleHandle*>(&StaticBox, &Box2) };

			Evolution.SetPhysicsMaterial(&StaticBox, MakeSerializable(PhysicalMaterial));
			Evolution.SetPhysicsMaterial(&Box2, MakeSerializable(PhysicalMaterial));

			Evolution.EnableParticle(&StaticBox);
			Evolution.EnableParticle(&Box2);

			Chaos::FPBDRigidDynamicSpringConstraints SpringConstraints(MoveTemp(Constraints));
			Evolution.AddConstraintContainer(SpringConstraints);

			const FReal Dt = 0.01f;
			for (int32 i = 0; i < 200; ++i)
			{
				Evolution.AdvanceOneTimeStep(Dt);
				Evolution.EndFrame(Dt);
			}
			EXPECT_LT(Box2.GetX()[2], 0);
		}

		{
			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs Particles(UniqueIndices);

			auto& StaticBox = *AppendStaticParticleBox(Particles, FVec3((FReal)100, (FReal)100, (FReal)100));
			auto& Box2 = *AppendDynamicParticleBox(Particles, FVec3((FReal)100, (FReal)100, (FReal)100));
			StaticBox.SetX(FVec3((FReal)0, (FReal)0, (FReal)500));

			Box2.SetX(FVec3((FReal)500, (FReal)0, (FReal)1000));
			Box2.SetP(Box2.GetX());

			THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
			TEvolution Evolution(Particles, PhysicalMaterials);
			TArray<TVec2<FGeometryParticleHandle*>> Constraints = { TVec2<FGeometryParticleHandle*>(&StaticBox, &Box2) };

			Evolution.SetPhysicsMaterial(&StaticBox, MakeSerializable(PhysicalMaterial));
			Evolution.SetPhysicsMaterial(&Box2, MakeSerializable(PhysicalMaterial));

			Evolution.EnableParticle(&StaticBox);
			Evolution.EnableParticle(&Box2);

			Chaos::FPBDRigidDynamicSpringConstraints SpringConstraints(MoveTemp(Constraints), 400);
			Evolution.AddConstraintContainer(SpringConstraints);

			const FReal Dt = 0.01f;
			for (int32 i = 0; i < 200; ++i)
			{
				Evolution.AdvanceOneTimeStep(Dt);
				Evolution.EndFrame(Dt);
			}
			EXPECT_GT(Box2.GetX()[2], 0);
		}
	}

	// check that joints don't simulate if a constrained particle is disabled
	template <typename TEvolution>
	void JointConstraint_DisableOneConstrainedParticle()
	{

		const int32 NumIterations = 1;
		const FReal Gravity = 980;

		FJointConstraintsTest<TEvolution> Test(NumIterations, Gravity);

		Test.ParticlePositions = {
			{ (FReal)0, (FReal)0, (FReal)1000 },
			{ (FReal)500, (FReal)0, (FReal)1000 },
		};
		Test.ParticleSizes =
		{
			{ (FReal)100, (FReal)100, (FReal)100 },
			{ (FReal)100, (FReal)100, (FReal)100 },
		};
		Test.ParticleMasses =
		{
			(FReal)1,
			(FReal)1,
		};

		Test.JointPositions =
		{
			{ (FReal)250, (FReal)0, (FReal)1000 },
		};
		Test.JointParticleIndices =
		{
			{ 0, 1 },
		};

		Test.Create();

		const int32 Box1Id = 0;
		const int32 Box2Id = 1;
		const FReal ExpectedDistance = (Test.ParticlePositions[1] - Test.ParticlePositions[0]).Size();
		const FVec3 Box2LocalSpaceJointPosition = Test.JointPositions[0] - Test.ParticlePositions[1];

		// box 1 disabled
		Test.Evolution.DisableParticle(Test.GetParticle(Box1Id));

		const FReal Dt = 0.01f;
		for (int32 i = 0; i < 100; ++i)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			// box 1 not simulating so would expect to not have moved
			EXPECT_LT(FMath::Abs(Test.ParticlePositions[0].X - Test.GetParticle(Box1Id)->GetX().X), (FReal)0.1);
			EXPECT_LT(FMath::Abs(Test.ParticlePositions[0].Y - Test.GetParticle(Box1Id)->GetX().Y), (FReal)0.1);
			EXPECT_LT(FMath::Abs(Test.ParticlePositions[0].Z - Test.GetParticle(Box1Id)->GetX().Z), (FReal)0.09);

			// box 2 should fall under gravity & not have moved in X or Y, constraint should not 'Apply' if other particle is disabled
			EXPECT_LT(FMath::Abs(Test.ParticlePositions[1].X - Test.GetParticle(Box2Id)->GetX().X), (FReal)0.1);
			EXPECT_LT(FMath::Abs(Test.ParticlePositions[1].Y - Test.GetParticle(Box2Id)->GetX().Y), (FReal)0.1);
			EXPECT_GT(FMath::Abs(Test.ParticlePositions[1].Z - Test.GetParticle(Box2Id)->GetX().Z), (FReal)0.09);
		}
	}


	GTEST_TEST(AllEvolutions, JointTests_TestSingleConstraint) {
		JointConstraint_Single<FPBDRigidsEvolutionGBF>();
	}

	GTEST_TEST(AllEvolutions, JointTests_TestSingleConstraintWithLateralTranslation) {
		JointConstraint_SingleMoveRoot<FPBDRigidsEvolutionGBF>();
	}

	GTEST_TEST(AllEvolutions, JointTests_TestSingleConstraintWithAnimatedRoot) {
		JointConstraint_SingleAnimated<FPBDRigidsEvolutionGBF>();
	}

	GTEST_TEST(AllEvolutions, JointTests_TestShortJointChainWithAnimatedRootProjectionEnabled) {
		JointConstraint_ShortChainAnimated<FPBDRigidsEvolutionGBF>(true);
	}	
	
	GTEST_TEST(AllEvolutions, JointTests_TestShortJointChainWithAnimatedRootProjectionDisabled) {
		JointConstraint_ShortChainAnimated<FPBDRigidsEvolutionGBF>(false);
	}

	GTEST_TEST(AllEvolutions, JointTests_TestLongJointChainWithAnimatedRootProjectionEnabled) {
		JointConstraint_LongChainAnimated<FPBDRigidsEvolutionGBF>(true);
	}

	GTEST_TEST(AllEvolutions, JointTests_TestLongJointChainWithAnimatedRootProjectionDisabled) {
		JointConstraint_LongChainAnimated<FPBDRigidsEvolutionGBF>(false);
	}

	GTEST_TEST(AllEvolutions, JointTests_TestLongJointChainWithVaryingIterations) {
		JointConstraint_LongChainVaryingIterations<FPBDRigidsEvolutionGBF>();
	}

	GTEST_TEST(AllEvolutions, JointTests_TestLongJointChainWithNonLinearSolvers) {
		JointConstraint_LongChainLinearJoints<FPBDRigidsEvolutionGBF>();
	}

	GTEST_TEST(AllEvolutions, JointTests_TestSingleSpringConstraint) {
		SpringConstraint<FPBDRigidsEvolutionGBF>();
	}

	GTEST_TEST(AllEvolutions, JointTests_TestSingleDynamicSpringConstraint) {
		DynamicSpringConstraint<FPBDRigidsEvolutionGBF>();
	}

	GTEST_TEST(AllEvolutions, JointConstraint_TestDisableOneConstrainedParticle) {
		JointConstraint_DisableOneConstrainedParticle<FPBDRigidsEvolutionGBF>();
	}

	// Create a kinematic-dynamic particle chain with a center of mass offset on the kinematic
	// and verify that the joint offsets are used correctly. The two particles are arranged vertically
	// with the joint between them, the center of mass offset should not affect behaviour.
	//
	// NOTE: We create both particles as dynamics and then change one to be kinematic so that we
	// can alter its mass properties.
	//
	GTEST_TEST(JointTests, TestJointCoMOffset)
	{
		const int32 NumIterations = 1;
		const FReal Gravity = 980;
		FJointConstraintsTest<FPBDRigidsEvolutionGBF> Test(NumIterations, Gravity);

		Test.ParticlePositions =
		{
			{ (FReal)0, (FReal)0, (FReal)0 },
			{ (FReal)0, (FReal)0, (FReal)1000 },
		};
		Test.ParticleSizes =
		{
			{ (FReal)100, (FReal)100, (FReal)100 },
			{ (FReal)100, (FReal)100, (FReal)100 },
		};
		Test.ParticleMasses =
		{
			(FReal)1000,
			(FReal)1000,
		};

		Test.JointPositions =
		{
			{ (FReal)0, (FReal)0, (FReal)500 },
		};
		Test.JointParticleIndices =
		{
			{ 0, 1 },
		};

		Test.Create();

		// Move the center of mass of the soon-to-be kinematic
		Test.GetParticle(0)->CastToRigidParticle()->SetCenterOfMass(FVec3(0, 0, -100));
		EXPECT_NEAR((Test.GetParticle(0)->CastToRigidParticle()->CenterOfMass() - FVec3(0, 0, -100)).Size(), 0, UE_KINDA_SMALL_NUMBER);

		// Make the root body kinematic
		Test.Evolution.SetParticleObjectState(Test.GetParticle(0)->CastToRigidParticle(), EObjectStateType::Kinematic);

		// The kinematic will now report zero center of mass via the GenericParticle API, but internally it will still be set
		EXPECT_NEAR(FGenericParticleHandle(Test.GetParticle(0))->CenterOfMass().Size(), 0, UE_KINDA_SMALL_NUMBER);
		EXPECT_NEAR((Test.GetParticle(0)->CastToRigidParticle()->CenterOfMass() - FVec3(0, 0, -100)).Size(), 0, UE_KINDA_SMALL_NUMBER);

		const FReal Dt = 0.01f;
		for (int32 i = 0; i < 100; ++i)
		{
			Test.Evolution.AdvanceOneTimeStep(Dt);
			Test.Evolution.EndFrame(Dt);

			// Neither particle should have moved
			EXPECT_LT((Test.GetParticle(0)->GetX() - Test.ParticlePositions[0]).Size(), (FReal)0.1);
			EXPECT_LT((Test.GetParticle(1)->GetX() - Test.ParticlePositions[1]).Size(), (FReal)0.1);
		}
	}

	// Check that constraints end up in the same island when graph is fully connected
	GTEST_TEST(JointTests, TestJointConstraintGraph_Connected)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs ParticleContainer(UniqueIndices);
		FPBDJointConstraints JointContainer;
		JointContainer.SetSortEnabled(true);

		// Create 3 particles
		TArray<FPBDRigidParticleHandle*> Rigids = ParticleContainer.CreateDynamicParticles(3);

		// Connect particles with 2 joints
		TArray<FPBDJointConstraintHandle*> Joints =
		{
			JointContainer.AddConstraint({ Rigids[0], Rigids[1] }, { FRigidTransform3(), FRigidTransform3() }),
			JointContainer.AddConstraint({ Rigids[0], Rigids[2] }, { FRigidTransform3(), FRigidTransform3() })
		};

		// This sets up the joint container, including generating islands etc
		JointContainer.PrepareTick();

		// Both joints should be in an island
		EXPECT_GE(JointContainer.GetConstraintIsland(0), 0);
		EXPECT_GE(JointContainer.GetConstraintIsland(1), 0);

		// Both joints should be in same island
		EXPECT_EQ(JointContainer.GetConstraintIsland(0), JointContainer.GetConstraintIsland(1));

		// Joints should have different colors
		EXPECT_NE(JointContainer.GetConstraintColor(0), JointContainer.GetConstraintColor(1));
	}

	// Check that constraints islands are not merged through shared kinematic particles
	GTEST_TEST(JointTests, TestJointConstraintGraph_NotConnected)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs ParticleContainer(UniqueIndices);
		FPBDJointConstraints JointContainer;
		JointContainer.SetSortEnabled(true);

		// Create 3 particles
		TArray<FPBDRigidParticleHandle*> Rigids = ParticleContainer.CreateDynamicParticles(3);

		// Connect particles with 2 joints
		TArray<FPBDJointConstraintHandle*> Joints =
		{
			JointContainer.AddConstraint({ Rigids[0], Rigids[1] }, { FRigidTransform3(), FRigidTransform3() }),
			JointContainer.AddConstraint({ Rigids[0], Rigids[2] }, { FRigidTransform3(), FRigidTransform3() })
		};

		// Set the particle in the middle of the two joints to kinematic
		Rigids[0]->SetObjectStateLowLevel(EObjectStateType::Kinematic);

		// This sets up the joint container, including generating islands etc
		JointContainer.PrepareTick();

		// Both joints should be in an island
		EXPECT_GE(JointContainer.GetConstraintIsland(0), 0);
		EXPECT_GE(JointContainer.GetConstraintIsland(1), 0);

		// Joints should be in different islands
		EXPECT_NE(JointContainer.GetConstraintIsland(0), JointContainer.GetConstraintIsland(1));

		// Both joints should be at level 0
		EXPECT_EQ(JointContainer.GetConstraintLevel(0), 0);
		EXPECT_EQ(JointContainer.GetConstraintLevel(1), 0);
	}

	// Change the linear/non-linear solver mode of a constraint after creation using the GT Proxy API.
	// Bug Fix: changing the solver type after creation would result in an attempt to remove the constraint
	// from the wrong Container on destruction, leading to an OOB array access, or destruction of the wrong
	// constraint and a later OOB access attempting to destroy that constraint.
	GTEST_TEST(JointTests, TestSetUseLinearJointSolver)
	{
		using namespace Chaos;
		auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();

		// Make a solver
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);

		// Make 2 particles for our constraint
		auto Proxy1 = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& Particle1 = Proxy1->GetGameThreadAPI();
		Particle1.SetGeometry(Sphere);
		Solver->RegisterObject(Proxy1);

		auto Proxy2 = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& Particle2 = Proxy2->GetGameThreadAPI();
		Particle2.SetGeometry(Sphere);
		Solver->RegisterObject(Proxy2);

		// Confirm solver doesn't have any joint constraint proxies yet.
		EXPECT_EQ(Solver->GetJointConstraintPhysicsProxies_Internal().Num(), 0);

		FTransform IdentityTransform(FTransform::Identity);
		FPhysicsConstraintHandle Constraint = FChaosEngineInterface::CreateConstraint(Proxy1, Proxy2, IdentityTransform, IdentityTransform);
		FChaosEngineInterface::SetUseLinearJointSolver_AssumesLocked(Constraint, false);

		// Tick physics thread to create the PT constraint
		Solver->AdvanceAndDispatch_External(/*Dt=*/1.0f);

		// Change the solver type for the joint
		FChaosEngineInterface::SetUseLinearJointSolver_AssumesLocked(Constraint, true);

		// Tick physics thread
		Solver->AdvanceAndDispatch_External(/*Dt=*/1.0f);

		// Release constraint
		FChaosEngineInterface::ReleaseConstraint(Constraint);

		// Tick physics thread
		// Bug Fix: This would crash in FPBDJointConstraints::RemoveConstraint
		Solver->AdvanceAndDispatch_External(/*Dt=*/1.0f);

		Solver->UnregisterObject(Proxy1);
		Solver->UnregisterObject(Proxy2);
		Module->DestroySolver(Solver);
	}
}

