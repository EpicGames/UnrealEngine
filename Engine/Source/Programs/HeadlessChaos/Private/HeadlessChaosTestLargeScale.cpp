// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestCollisions.h"

#include "HeadlessChaos.h"
#include "Chaos/Box.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Convex.h"
#include "Chaos/Sphere.h"
#include "Chaos/GJK.h"
#include "Chaos/Pair.h"
#include "Chaos/Utilities.h"


namespace ChaosTest {

	using namespace Chaos;

	// Test GJKPenetration for a normal sized Capsule against a large scaled Box.
	// We used to have a large error in GJK when testing against large shapes. This was fixed
	// in Simplex.h in the same CL that this line was added.
	GTEST_TEST(LargeScaleTests, TestSmallCapsuleLargeBoxGJKPenetration)
	{
		// Shape settings
		const FVec3 BoxSize = FVec3(100, 100, 100);
		const FVec3 BoxScale = FVec3(1000, 1000, 1);
		const FReal BoxMargin = 0.1f * (BoxSize * BoxScale).GetAbsMin();
		const FReal CapsuleHeight = 200.0f;
		const FReal CapsuleRadius = 50.0f;

		// Shape transforms
		const FReal Separation = 20.0f;
		const FVec3 BoxPos = FVec3(0, 0, -0.5 * (BoxSize * BoxScale).Z);
		const FVec3 CapsulePos = FVec3(0, 0, 0.5f * CapsuleHeight + Separation);
		const FRigidTransform3 AT(BoxPos, FRotation3::FromIdentity());
		const FRigidTransform3 BT(CapsulePos, FRotation3::FromIdentity());

		// Shapes
		FImplicitBox3 A(-0.5f * BoxSize * BoxScale, 0.5f * BoxSize * BoxScale, BoxMargin);
		FImplicitCapsule3 B(-(0.5f * CapsuleHeight - CapsuleRadius) * FVec3::UpVector, (0.5f * CapsuleHeight - CapsuleRadius) * FVec3::UpVector, CapsuleRadius);

		// Run GJK
		FReal Penetration;
		FVec3 ClosestA, ClosestBInA, Normal;
		int32 ClosestVertexIndexA, ClosestVertexIndexB;
		const FRigidTransform3 BTtoAT = BT.GetRelativeTransform(AT);
		bool bResult = GJKPenetration<true>(A, B, BTtoAT, Penetration, ClosestA, ClosestBInA, Normal, ClosestVertexIndexA, ClosestVertexIndexB);

		// Verify results
		const FReal MaxPositionError = 1.e-3f;
		EXPECT_TRUE(bResult);
		EXPECT_NEAR(-Penetration, Separation, MaxPositionError);
		EXPECT_NEAR(ClosestA.Z, 0.5f * (BoxSize * BoxScale).Z, MaxPositionError);			// Box space
		EXPECT_NEAR(ClosestBInA.Z, 0.5f * (BoxSize * BoxScale).Z + Separation, MaxPositionError);
	}

	// A large box centered on X,Y and positioned so that the top surface is at Z = 0.
	// A small capsule positioned according to the parameter CapsulePos.
	// Verify that a downward sweep with the specified initial direction returns an accurate result.
	// We used to have a large error in GJK when testing against large shapes. This was fixed
	// in Simplex.h in the same CL that this line was added.
	void SmallCapsuleLargeBoxGJKRaycast(const FVec3 &CapsuleBottomPos, const FVec3& InitialDir)
	{
		// Shape settings
		const FVec3 BoxSize = FVec3(100, 100, 100);
		const FVec3 BoxScale = FVec3(1000, 1000, 1);
		const FReal BoxMargin = 0.1f * (BoxSize * BoxScale).GetAbsMin();
		const FReal CapsuleHeight = 200.0f;
		const FReal CapsuleRadius = 50.0f;

		// Shape transforms
		const FVec3 BoxPos = FVec3(0, 0, -0.5 * (BoxSize * BoxScale).Z);
		const FReal Separation = CapsuleBottomPos.Z;
		const FVec3 CapsulePos = CapsuleBottomPos + 0.5f * CapsuleHeight * FVec3::UpVector;
		const FRigidTransform3 AT(BoxPos, FRotation3::FromIdentity());
		const FRigidTransform3 BT(CapsulePos, FRotation3::FromIdentity());

		// Ray settings
		const FVec3 RayDir = FVec3(0, 0, -1);
		const FReal RayLength = 2 * Separation;

		// Shapes
		FImplicitBox3 A(-0.5f * BoxSize * BoxScale, 0.5f * BoxSize * BoxScale, BoxMargin);
		FImplicitCapsule3 B(-(0.5f * CapsuleHeight - CapsuleRadius) * FVec3::UpVector, (0.5f * CapsuleHeight - CapsuleRadius) * FVec3::UpVector, CapsuleRadius);

		// Run GJK
		FReal Distance;
		FVec3 Position, Normal;
		const FRigidTransform3 BTtoAT = BT.GetRelativeTransform(AT);
		bool bResult = GJKRaycast2(A, B, BTtoAT, RayDir, RayLength, Distance, Position, Normal, (FReal)0., false, InitialDir, (FReal)0.);

		// Verify results
		const FReal MaxTimeError = 1.e-4f;
		const FReal MaxPositionError = 1.e-3f;
		const FReal MaxNormalError = 1.e-4f;
		EXPECT_TRUE(bResult);
		EXPECT_NEAR(Distance, CapsuleBottomPos.Z, MaxTimeError);
		EXPECT_NEAR(Position.Z, 0.5f * (BoxSize * BoxScale).Z, MaxPositionError);		// Box space
		EXPECT_NEAR(Normal.Z, 1.0f, MaxNormalError);
	}

	GTEST_TEST(LargeScaleTests, TestSmallCapsuleLargeBoxGJKRaycast_Vertical)
	{
		// Place capsule above the plane and shited laterally
		SmallCapsuleLargeBoxGJKRaycast(FVec3(3, 5, 20), FVec3(0, 0, 1));
		SmallCapsuleLargeBoxGJKRaycast(FVec3(100, 50, 20), FVec3(0, 0, 1));
		SmallCapsuleLargeBoxGJKRaycast(FVec3(300, 50, 20), FVec3(0, 0, 1));
		SmallCapsuleLargeBoxGJKRaycast(FVec3(400, 50, 20), FVec3(0, 0, 1));
		SmallCapsuleLargeBoxGJKRaycast(FVec3(500, 50, 20), FVec3(0, 0, 1));
	}



	// Test GJKRaycast for a normal sized Capsule against a large scaled Box.
	// We used to have a large error in GJK when testing against large shapes. This was fixed
	// in Simplex.h in the same CL that this line was added.
	GTEST_TEST(LargeScaleTests, TestSmallCapsuleLargeConvexGJKRaycast)
	{
		// Shape settings
		const FVec3 BoxSize = FVec3(100, 100, 100);
		const FVec3 BoxScale = FVec3(1000, 1000, 1);
		const FReal BoxMargin = 0.1f * (BoxSize * BoxScale).GetAbsMin();
		const FReal CapsuleHeight = 200.0f;
		const FReal CapsuleRadius = 50.0f;

		// Shape transforms
		const FReal Separation = 20.0f;
		const FVec3 BoxPos = FVec3(0, 0, -0.5 * (BoxSize * BoxScale).Z);
		const FVec3 CapsulePos = FVec3(0, 0, 0.5f * CapsuleHeight + Separation);
		const FRigidTransform3 AT(BoxPos, FRotation3::FromIdentity());
		const FRigidTransform3 BT(CapsulePos, FRotation3::FromIdentity());

		// Ray settings
		const FVec3 RayDir = FVec3(0, 0, -1);
		const FReal RayLength = 2 * Separation;

		// Shapes
		TImplicitObjectScaled<FImplicitConvex3> A = CreateScaledConvexBox(BoxSize, BoxScale, BoxMargin);
		FImplicitCapsule3 B(-(0.5f * CapsuleHeight - CapsuleRadius) * FVec3::UpVector, (0.5f * CapsuleHeight - CapsuleRadius) * FVec3::UpVector, CapsuleRadius);

		// Run GJK
		FReal Time;
		FVec3 Position, Normal;
		const FRigidTransform3 BTtoAT = BT.GetRelativeTransform(AT);
		bool bResult = GJKRaycast2(A, B, BTtoAT, RayDir, RayLength, Time, Position, Normal);

		// Verify results
		const FReal MaxTimeError = 1.e-4f;
		const FReal MaxPositionError = 1.e-3f;
		const FReal MaxNormalError = 1.e-4f;
		EXPECT_TRUE(bResult);
		EXPECT_NEAR(Time, Separation, MaxTimeError);
		EXPECT_NEAR(Position.Z, 0.5f * (BoxSize * BoxScale).Z, MaxPositionError);
		EXPECT_NEAR(Normal.Z, 1.0f, MaxNormalError);
	}

	GTEST_TEST(LargeScaleTests, TestSmallBoxLargeBoxGJKIntersection)
	{
		const FVec3 FloorMin(-250000.09155273438, -250000.01525878906, -50.000015258789062);
		const FVec3 FloorMax(250000.00000000000, 249999.98474121094, 2.8421709430404007e-14);
		const FVec3 BoxPos(6400.000000, 6400.000000, 95.304688);
		const FVec3 BoxMin(5040.0000000000000, 5040.0000000000000, -4548.6328119999998);
		const FVec3 BoxMax(7760.0000000000000, 7760.0000000000000, 4739.2421880000002);
		const FRigidTransform3 AT(FVec3::Zero(), FRotation3::FromIdentity());
		const FRigidTransform3 BT(FVec3::Zero(), FRotation3::FromIdentity());
		const FRigidTransform3 BTtoAT = BT.GetRelativeTransform(AT);
		FImplicitBox3 Box(BoxMin, BoxMax);
		FImplicitBox3 Floor(FloorMin, FloorMax);
		FVec3 InitialDir = -BoxPos;
		FReal Thickness = 0;
		const bool bResult = GJKIntersection(Floor, Box, BTtoAT, Thickness, InitialDir);
		EXPECT_TRUE(bResult);
	}

}