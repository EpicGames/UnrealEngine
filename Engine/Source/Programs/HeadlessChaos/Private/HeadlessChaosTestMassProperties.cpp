// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Box.h"
#include "Chaos/Matrix.h"
#include "Chaos/MassProperties.h"
#include "Chaos/TriangleMesh.h"
#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Modules/ModuleManager.h"

namespace ChaosTest {

	void TransformToLocalSpace1()
	{
		Chaos::FMatrix33 Inertia;
		Inertia.M[0][0] = 3628.83862f;
		Inertia.M[0][1] = 0.f;
		Inertia.M[0][2] = 1675.89563f;
		Inertia.M[0][3] = 0.0f;
		Inertia.M[1][0] = 0.f;
		Inertia.M[1][1] = 13133.3340;
		Inertia.M[1][2] = 0.f;
		Inertia.M[1][3] = 0.f;
		Inertia.M[2][0] = 1675.89563;
		Inertia.M[2][1] = 0.f;
		Inertia.M[2][2] = 12837.8281;
		Inertia.M[2][3] = 0.f;
		Inertia.M[3][0] = 0.f;
		Inertia.M[3][1] = 0.f;
		Inertia.M[3][2] = 0.f;
		Inertia.M[3][3] = 5.f;
		auto Rotation = Chaos::TransformToLocalSpace(Inertia);
	}

	Chaos::FMatrix33 RandInertia(FReal MinVal, FReal MaxVal)
	{
		return Chaos::FMatrix33(FMath::RandRange(MinVal, MaxVal), FMath::RandRange(MinVal, MaxVal), FMath::RandRange(MinVal, MaxVal));
	}

	// Check Chaos::TransformToLocalSpace
	// 
	// Starting from some diagonal Inertia matrices, apply various rotations to them and 
	// check that TransformToLocalSpace can recover the original inertia and the principal axis.
	void TransformToLocalSpace2()
	{
		FMath::SRandInit(347856243);
		FVector Axes[] = 
		{
			// Rotations about world-space axes
			FVector::UpVector,
			FVector::DownVector,
			FVector::ForwardVector,
			FVector::BackwardVector,
			FVector::LeftVector,
			FVector::RightVector,
			// Rotation about random axes
			RandAxis(),
			RandAxis(),
			RandAxis(),
			RandAxis(),
			RandAxis(),
			RandAxis(),
			RandAxis(),
			RandAxis(),
			RandAxis(),
			RandAxis(),
			RandAxis(),
			RandAxis(),
		};

		FReal Angles[] =
		{
			// No rotation
			0.0f,
			// Random rotation angles
			FMath::RandRange(-2.0f * PI, 2.0f * PI),
			FMath::RandRange(-2.0f * PI, 2.0f * PI),
			FMath::RandRange(-2.0f * PI, 2.0f * PI),
			FMath::RandRange(-2.0f * PI, 2.0f * PI),
			FMath::RandRange(-2.0f * PI, 2.0f * PI),
			FMath::RandRange(-2.0f * PI, 2.0f * PI),
			FMath::RandRange(-2.0f * PI, 2.0f * PI),
			FMath::RandRange(-2.0f * PI, 2.0f * PI),
			FMath::RandRange(-2.0f * PI, 2.0f * PI),
			// Random small rotation angles
			FMath::RandRange(-0.1f * PI, 0.1f * PI),
			FMath::RandRange(-0.1f * PI, 0.1f * PI),
			FMath::RandRange(-0.1f * PI, 0.1f * PI),
			FMath::RandRange(-0.1f * PI, 0.1f * PI),
			FMath::RandRange(-0.1f * PI, 0.1f * PI),
			FMath::RandRange(-0.1f * PI, 0.1f * PI),
			FMath::RandRange(-0.1f * PI, 0.1f * PI),
			FMath::RandRange(-0.1f * PI, 0.1f * PI),
			FMath::RandRange(-0.1f * PI, 0.1f * PI),
		};

		Chaos::FMatrix33 Inertias[] =
		{
			// Specific inertia tests
			Chaos::FMatrix33(1.0f, 1.0f, 1.0f),
			Chaos::FMatrix33(1000.0f, 1000.0f, 1000.0f),
			Chaos::FMatrix33(1234.0f, 222.0f, 4321.0f),
			Chaos::FMatrix33(2.0f, 1.0f, 1.0f),
			// Random largish inertia
			RandInertia(100.0f, 10000.0f),
			RandInertia(100.0f, 10000.0f),
			RandInertia(100.0f, 10000.0f),
			RandInertia(100.0f, 10000.0f),
			RandInertia(100.0f, 10000.0f),
			RandInertia(100.0f, 10000.0f),
			RandInertia(100.0f, 10000.0f),
			RandInertia(100.0f, 10000.0f),
			RandInertia(100.0f, 10000.0f),
			// Random smallish inertia
			RandInertia(0.01f, 1.0f),
			RandInertia(0.01f, 1.0f),
			RandInertia(0.01f, 1.0f),
			RandInertia(0.01f, 1.0f),
			RandInertia(0.1f, 1.0f),
			RandInertia(0.1f, 1.0f),
			RandInertia(0.1f, 1.0f),
			RandInertia(0.1f, 1.0f),
			RandInertia(0.1f, 1.0f),
		};

		for (Chaos::FMatrix33 InputInertiaLocal : Inertias)
		{
			for (FVector InputRotationAxis : Axes)
			{
				for (FReal InputRotationAngle : Angles)
				{
					FQuat InputRotation = FQuat(InputRotationAxis, InputRotationAngle);
					Chaos::FMatrix33 InputInertia = Utilities::ComputeWorldSpaceInertia(InputRotation, InputInertiaLocal);
					Chaos::FMatrix33 OutputInertiaLocal = InputInertia;
					Chaos::FRotation3 OutputRotation = Chaos::TransformToLocalSpace(OutputInertiaLocal);

					// We should have recovered the local inertia matrix, but the axes may be switched
					FVector OutputInertiaAxes[3], InputInertiaAxes[3];
					OutputInertiaLocal.GetUnitAxes(OutputInertiaAxes[0], OutputInertiaAxes[1], OutputInertiaAxes[2]);
					InputInertiaLocal.GetUnitAxes(InputInertiaAxes[0], InputInertiaAxes[1], InputInertiaAxes[2]);
					int32 MatchedInertiaAxis[3] = { INDEX_NONE, INDEX_NONE, INDEX_NONE };
					FReal MatchedInertiaDir[3] = { 0.0, 0.0, 0.0 };
					for (int OutputInertiaAxisIndex = 0; OutputInertiaAxisIndex < 3; ++OutputInertiaAxisIndex)
					{
						for (int InputInertiaAxisIndex = 0; InputInertiaAxisIndex < 3; ++InputInertiaAxisIndex)
						{
							FReal Dot = FVector::DotProduct(InputInertiaAxes[InputInertiaAxisIndex], OutputInertiaAxes[OutputInertiaAxisIndex]);
							if (FMath::IsNearlyEqual((FReal)FMath::Abs(Dot), (FReal)1., (FReal)KINDA_SMALL_NUMBER))
							{
								// We should only find each axis zero or one times
								EXPECT_TRUE(MatchedInertiaAxis[InputInertiaAxisIndex] == INDEX_NONE);
								MatchedInertiaAxis[InputInertiaAxisIndex] = OutputInertiaAxisIndex;
								MatchedInertiaDir[InputInertiaAxisIndex] = FMath::Sign(Dot);
								break;
							}
						}
					}
					EXPECT_TRUE((MatchedInertiaAxis[0] != INDEX_NONE) && (MatchedInertiaAxis[1] != INDEX_NONE) && (MatchedInertiaAxis[2] != INDEX_NONE));

					// Check that we recover the rotated input inertia matrix
					Chaos::FMatrix33 OutputInertia = Utilities::ComputeWorldSpaceInertia(OutputRotation, OutputInertiaLocal);
					if (!OutputInertia.Equals(InputInertia, 0.1f))
					{
						GTEST_FAIL();
					}
					EXPECT_TRUE(OutputInertia.Equals(InputInertia, 0.1f));
				}
			}
		}
	}


	void ComputeMassProperties()
	{
		Chaos::FParticles Particles;
		Particles.AddParticles(8);
		Particles.SetX(0, FVec3(-1, -1, -1));
		Particles.SetX(1, FVec3(-1, -1, 1));
		Particles.SetX(2, FVec3(-1, 1, -1));
		Particles.SetX(3, FVec3(-1, 1, 1));
		Particles.SetX(4, FVec3(1, -1, -1));
		Particles.SetX(5, FVec3(1, -1, 1));
		Particles.SetX(6, FVec3(1, 1, -1));
		Particles.SetX(7, FVec3(1, 1, 1));
		TArray<Chaos::TVec3<int32>> Faces;
		Faces.SetNum(12);
		Faces[0] = TVec3<int32>(0, 4, 5);
		Faces[1] = TVec3<int32>(5, 1, 0);
		Faces[2] = TVec3<int32>(7, 6, 2);
		Faces[3] = TVec3<int32>(2, 3, 7);
		Faces[4] = TVec3<int32>(0, 1, 3);
		Faces[5] = TVec3<int32>(3, 2, 0);
		Faces[6] = TVec3<int32>(7, 5, 4);
		Faces[7] = TVec3<int32>(4, 6, 7);
		Faces[8] = TVec3<int32>(0, 2, 6);
		Faces[9] = TVec3<int32>(6, 4, 0);
		Faces[10] = TVec3<int32>(7, 3, 1);
		Faces[11] = TVec3<int32>(1, 5, 7);
		Chaos::FTriangleMesh Surface(MoveTemp(Faces));
		Chaos::FMassProperties MassProperties = Chaos::CalculateMassProperties(Particles, Surface.GetElements(), (FReal)1.0);
		EXPECT_TRUE(MassProperties.CenterOfMass.Size() < SMALL_NUMBER);
		EXPECT_TRUE(MassProperties.RotationOfMass.Euler().Size() < SMALL_NUMBER);
		EXPECT_TRUE(MassProperties.InertiaTensor.M[0][0] - ((FReal)2 / 3) < SMALL_NUMBER);
		EXPECT_TRUE(MassProperties.InertiaTensor.M[1][1] - ((FReal)2 / 3) < SMALL_NUMBER);
		EXPECT_TRUE(MassProperties.InertiaTensor.M[2][2] - ((FReal)2 / 3) < SMALL_NUMBER);
	}


	TEST(MassPropertyTests, TestTransformToLocalSpace1) {
		ChaosTest::TransformToLocalSpace1();
	}

	TEST(MassPropertyTests, TestTransformToLocalSpace2) {
		ChaosTest::TransformToLocalSpace2();
	}

	TEST(MassPropertyTests, TestMassProperties) {
		ChaosTest::ComputeMassProperties();
	}

	TEST(MassPropertyTests, TestWorldSpaceInertia)
	{
		FVec3 ILocal = FVec3(10.0f, 1.0f, 0.1f);

		for (int32 RotIndex = 0; RotIndex < 10; ++RotIndex)
		{
			FRigidTransform3 Transform = FRigidTransform3(FVec3(0, 0, 0), RandomRotation(PI, PI, PI));

			// World space inertia
			FMatrix33 IWorld = Utilities::ComputeWorldSpaceInertia(Transform.GetRotation(), FMatrix33(ILocal.X, ILocal.Y, ILocal.Z));

			// Calculate Inertia about each Axis individually
			FMatrix33 RotM = FRotation3(Transform.GetRotation()).ToMatrix();
			FVec3 ILocal2 = FVec3(
				FVec3::DotProduct(RotM.GetAxis(0), IWorld * RotM.GetAxis(0)),
				FVec3::DotProduct(RotM.GetAxis(1), IWorld * RotM.GetAxis(1)),
				FVec3::DotProduct(RotM.GetAxis(2), IWorld * RotM.GetAxis(2)));

			EXPECT_NEAR(ILocal2.X, ILocal.X, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(ILocal2.Y, ILocal.Y, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(ILocal2.Z, ILocal.Z, KINDA_SMALL_NUMBER);
		}
	}

	TEST(MassPropertyTests, TestScaleInertia)
	{
		FReal Mass = 12.0f;
		FVec3 Dim = FVec3(2.0f, 5.0f, 10.0f);
		FVec3 UnscaledInertia = TBox<FReal, 3>::GetInertiaTensor(Mass, Dim).GetDiagonal();
		FReal UnscaledVolume = Dim.X * Dim.Y * Dim.Z;

		FVec3 Scales[] = {
			FVec3(1, 1, 1),
			FVec3(5, 5, 5),
			FVec3(5, 1, 1),
			FVec3(1, 5, 1),
			FVec3(1, 1, 5),
			FVec3(5, 3, 2),
			FVec3(0.1f, 0.1f, 0.1f),
			FVec3(0.1f, 0.5f, 0.3f),
			FVec3(0.5f, 2.3f, 1.5f),
		};

		bool bScaleMass = false;

		for (int32 MassScale = 0; MassScale < 2; ++MassScale)
		{
			for (int32 Index = 0; Index < UE_ARRAY_COUNT(Scales); ++Index)
			{
				const FVec3& Scale = Scales[Index];

				FReal ScaledVolume = Dim.X * Dim.Y * Dim.Z * Scale.X * Scale.Y * Scale.Z;
				FReal ScaledMass = (bScaleMass) ? Mass * ScaledVolume / UnscaledVolume : Mass;
				FVec3 ExpectedScaledInertia = TBox<FReal, 3>::GetInertiaTensor(ScaledMass, Dim * Scale).GetDiagonal();
				
				FVec3 ScaledInertia = Utilities::ScaleInertia(UnscaledInertia, Scale, bScaleMass);

				// Accurate to 0.1%?
				EXPECT_NEAR(ScaledInertia.X, ExpectedScaledInertia.X, ExpectedScaledInertia.X / 1000.0f);
				EXPECT_NEAR(ScaledInertia.Y, ExpectedScaledInertia.Y, ExpectedScaledInertia.Y / 1000.0f);
				EXPECT_NEAR(ScaledInertia.Z, ExpectedScaledInertia.Z, ExpectedScaledInertia.Z / 1000.0f);
			}

			bScaleMass = !bScaleMass;
		}
	}
}
