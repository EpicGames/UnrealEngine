// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/Math/Krylov.h"
#include "Chaos/Math/Poisson.h"
#include "Chaos/Matrix.h"
#include "Chaos/Utilities.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Vector.h"
#include "GenericPlatform/GenericPlatformMath.h"

namespace ChaosTest
{
	using namespace Chaos;

	TEST(MathTests, TestMatrixInverse)
	{
		FMath::RandInit(10695676);
		const FReal Tolerance = (FReal)0.001;

		for (int RandIndex = 0; RandIndex < 20; ++RandIndex)
		{
			FMatrix33 M = RandomMatrix(-10, 10);
			FMatrix33 MI = M.Inverse();

			FMatrix33 R = Utilities::Multiply(MI, M);

			EXPECT_TRUE(R.Equals(FMatrix33::Identity, Tolerance));
		}
	}

	template<int Order>
	void TestAsinEst(const FReal ExpectedMaxError, const FReal SmallError, const FReal SmallErrorX)
	{
		const int32 NumSteps = 21;
		for (int32 I = 0; I < NumSteps; ++I)
		{
			// X from 0 to +1 in steps
			const FReal X = FMath::Clamp(FReal(I) / FReal(NumSteps - 1), FReal(0), FReal(1));
			const FReal Y = Utilities::AsinEst<FReal, Order>(X);
			const FReal YNeg = Utilities::AsinEst<FReal, Order>(-X);
			const FReal YExact = FMath::Asin(X);

			// Asin(X) == -Asin(-X)
			EXPECT_NEAR(Y, -YNeg, UE_SMALL_NUMBER);

			// Total error percent less than expected
			const FReal Error = (YExact > 0) ? FMath::Abs(Y - YExact) / YExact : UE_SMALL_NUMBER;
			EXPECT_LT(Error, ExpectedMaxError);

			// Error percent should be less than SmallError for X < ExpectedXAtSmallError
			if (Error > SmallError)
			{
				EXPECT_GT(X, SmallErrorX - UE_SMALL_NUMBER);
			}
		}
	}

	TEST(MathTests, TestAsinEst3)
	{
		const FReal MaxError = FReal(0.26);			// expected error - 26% at X=1
		const FReal SmallError = FReal(0.01);		// expected error - 1% at X=0.6
		const FReal SmallErrorX = FReal(0.6);

		TestAsinEst<3>(MaxError, SmallError, SmallErrorX);
	}

	TEST(MathTests, TestAsinEst5)
	{
		const FReal MaxError = FReal(0.21);			// expected error - 21% at X=1
		const FReal SmallError = FReal(0.01);		// expected error - 1% at X=0.75
		const FReal SmallErrorX = FReal(0.75);

		TestAsinEst<5>(MaxError, SmallError, SmallErrorX);
	}

	TEST(MathTests, TestAsinEst7)
	{
		const FReal MaxError = FReal(0.19);			// expected error - 19% at X=1
		const FReal SmallError = FReal(0.01);		// expected error - 1% at X=0.8
		const FReal SmallErrorX = FReal(0.8);

		TestAsinEst<7>(MaxError, SmallError, SmallErrorX);
	}

	TEST(MathTests, TestAsinEstCrossover)
	{
		const FReal MaxError = FReal(0.01);	// 1%
		const int32 NumSteps = 21;
		for (int32 I = 0; I < NumSteps; ++I)
		{
			// X from 0 to +1 in steps
			const FReal X = FMath::Clamp(FReal(I) / FReal(NumSteps - 1), FReal(0), FReal(1));
			const FReal Y = Utilities::AsinEstCrossover(X);
			const FReal YNeg = Utilities::AsinEstCrossover(-X);
			const FReal YExact = FMath::Asin(X);

			// F(X) == -F(-X)
			EXPECT_NEAR(Y, -YNeg, UE_SMALL_NUMBER);

			// Total error less than expected
			const FReal Error = (YExact > 0) ? FMath::Abs(Y - YExact) / YExact : UE_SMALL_NUMBER;
			EXPECT_LT(Error, MaxError);
		}
	}

	Chaos::TVector<double, 4> 
	ToVec4(const TArray<double>& x)
	{
		return Chaos::TVector<double, 4>(x[0], x[1], x[2], x[3]);
	}

	TArray<double>
	ToArray4(const TVector<double, 4>& x)
	{
		return TArray<double>({x[0], x[1], x[2], x[3]});
	}

	TEST(MathTests, TestLanczosCGSolver) 
	{
		Chaos::PMatrix<double, 4, 4> A(0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0);
		A.M[0][0] = double(2);
		A.M[0][1] = -double(1);
		A.M[1][0] = -double(1);
		A.M[1][1] = double(2);
		A.M[1][2] = -double(1);
		A.M[2][1] = -double(1);
		A.M[2][2] = double(2);
		A.M[2][3] = -double(1);
		A.M[3][2] = -double(1);
		A.M[3][3] = double(2);
		A = A.GetTransposed(); // UE is col maj, not row

		TArray<double> x({ 1, 1, 1, 1 });
		TArray<double> x_cg;
		TArray<double> b = ToArray4(A.TransformFVector4(ToVec4(x)));

		Chaos::LanczosCG<double>(
			[&A](TArray<double>& y, const TArray<double>& x) { y = ToArray4(A.TransformFVector4(ToVec4(x))); },
			[](const TArray<double>& x, const TArray<double>& y) { return Dot4(ToVec4(x), ToVec4(y)); },
			[](TArray<double>& y, double a, const TArray<double>& x) { y = ToArray4(ToVec4(y) + a * ToVec4(x)); },
			[](TArray<double>& y, double a) { y = ToArray4(a * ToVec4(y)); },
			[](TArray<double>& y, const TArray<double>& x) { y = x; },
			x_cg,
			b,
			5);

		double error = 0.;
		for (int i = 0; i < 4; i++)
			error += (x[i] - x_cg[i]) * (x[i] - x_cg[i]);
		error = FGenericPlatformMath::Sqrt(error);
		double tol = 1.e-10;
		EXPECT_NEAR(error, double(0), tol);
	}

#if 0
	void ICRes(const Chaos::PMatrix<double, 4, 4>& L, const Chaos::TVector<double, 4>& d, Chaos::TVector<double, 4>& y, const Chaos::TVector<double, 4>& x)
	{
		Chaos::TVector<double, 4> u;
		for (int k = 0; k < 4; k++)
		{
			if (L.M[k][k])
			{
				double rly = L.M[k][k];
				for (int j = 0; j < k; j++)
				{
					rly -= L.M[k][j] * u[j];
				}
				u[k] = rly / L.M[k][k];
			}
		}
		for (int k = 4 - 1; k >= 0; k--)
		{
			double lu = 0.;
			for (int i = k + 1; i < 4; i++)
			{
				lu += L.M[i][k] * y[i];
			}
			y[k] = u[k] - d[k] * lu;
		}
	}

	TEST(MathTests, LanczosPCGSolver)
	{
		Chaos::PMatrix<double, 4, 4> A(2, 0, 0, 0, 0, 0, 0, -2.6, 0, 0, 4, 3.6, 0, -2.6, 3.6, 5);
		Chaos::PMatrix<double, 4, 4> M;
		Chaos::PMatrix<double, 3, 3> LMat;
		Chaos::TVector<double, 4> DVec;
		TVector<double, 4> x(1, 1, 1, 1);
		TVector<double, 4> x_pcg;
		TVector<double, 4> b;
		//b = A * x;
		b = A.TransformFVector4(x);

		// Calculate M
		auto ICU = [](Chaos::PMatrix<double, 4, 4>& AMat, Chaos::PMatrix<double, 4, 4>& L, Chaos::TVector<double, 4>& d)
		{
			double epsilon = .1;
			L.M[0][0] = AMat.M[0][0];
			
			d[0] = 1. / L.M[0][0];
			for (int k = 0; k < 4; k++) 
			{
				double value = 0.;
				for (int j =0; j < 4; j++) 
				{
					double aij = AMat.M[k][j];
					double ldl = 0.;
					for(int i=0; i < 4; i++)
					{
						ldl -= L.M[k][i] * L.M[j][i] * d[i];
					}
					value = aij + ldl;
					L.M[k][j] = value;
				}
				if (abs(L.M[k][k]) < epsilon)
					L.M[k][k] = epsilon;
				d[k] = 1. / L.M[k][k];
			}
		};

		auto multiplyA = [&A](Chaos::TVector<double, 4>& y, const Chaos::TVector<double, 4>& x) { y = A.TransformFVector4(x); };
		auto multiplyPrecond = [&LMat, &DVec, this](Chaos::TVector<double, 4>& y, const Chaos::TVector<double, 4>& x) { ICRes(LMat, DVec, y, x); };
		auto dotProduct = [](const Chaos::TVector<double, 4>& x, const Chaos::TVector<double, 4>& y) { return Dot4(x, y); };
		auto set = [](Chaos::TVector<double, 4>& y, const Chaos::TVector<double, 4>& x) { y = x; };
		auto setScaled = [](Chaos::TVector<double, 4>& a, const Chaos::TVector<double, 4>& b, const double s) { a = s * b; };        // setScaled(a,b,s): a <- s * b
		auto scaleAndAdd = [](Chaos::TVector<double, 4>& a, const double s, const Chaos::TVector<double, 4>& b) { a = s * a + b; };  // scaleAndAdd(a,s,b): a <- s * a + b
		auto addScaled = [](Chaos::TVector<double, 4>& a, const Chaos::TVector<double, 4>& b, const double s) { a += s * b; };       // addScaled(a,b,s): a <- a + s * b
		auto addScaled2 = [](Chaos::TVector<double, 4>& a, const Chaos::TVector<double, 4>& b1, const double s1, const Chaos::TVector<double, 4>& b2, const double s2) {
			a += s1 * b1 + s2 * b2;
		};  // addScaled2(a,b1,s1,b2,s2); a <- s1 * b1 + s2 * b2
		auto residual = [&](double& r, const Chaos::TVector<double, 4>& x, const Chaos::TVector<double, 4>& b) {
			Chaos::TVector<double, 4> res, Mres;
			multiplyA(res, x);
			addScaled(res, b, double(-1));
			multiplyPrecond(Mres, res);
			r = FGenericPlatformMath::Sqrt(dotProduct(Mres, res));
		};

		Chaos::LanczosPCG<T>(multiplyA, multiplyPrecond, dotProduct, set, setScaled, scaleAndAdd, addScaled, addScaled2, residual, x_pcg, b, 4);
		double r;
		residual(r, x_pcg, b);
		EXPECT_NEAR(r, 0., 1.e-10);
	}
#endif // 0

	TEST(MathTests, TestLaplacian)
	{
		int32 N = 3;
		double dx = 1. / N;
		TVector<double, 3> Origin(0, 0, 0);
		TVector<double, 3> MinCorner = Origin;
		TVector<double, 3> MaxCorner = Origin + TVector<double, 3>(dx * N, dx * N, dx * N);
		TUniformGrid<double, 3> Grid(MinCorner, MaxCorner, TVector<int32, 3>(N, N, N), 0);
		
		TArray<TVector<int, 4>> Mesh;
		TArray<TVector<double, 3>> X;
		Chaos::Utilities::TetMeshFromGrid<double>(Grid, Mesh, X);

		TArray<TArray<int>> IncidentElementsLocalIndex;
		TArray<TArray<int>> IncidentElements = Chaos::Utilities::ComputeIncidentElements(Mesh, &IncidentElementsLocalIndex);

		TArray<double> u; u.SetNum(X.Num());
		TVector<double, 3> A(1, 2, 3);
		for (int32 i = 0; i < X.Num(); i++)
		{
			u[i] = 0.;
			for (int32 alpha = 0; alpha < 3; alpha++)
			{
				u[i] += A[alpha] * X[i][alpha];
			}
		}

		TArray<double> De_inverse, measure;
		Chaos::ComputeDeInverseAndElementMeasures<double>(Mesh, X, De_inverse, measure);

		TArray<double> Lu; Lu.SetNum(X.Num());
		Chaos::Laplacian(Mesh, IncidentElements, IncidentElementsLocalIndex, De_inverse, measure, u, Lu);

		for (int32 i = 0; i < Grid.GetNumNodes(); i++)
		{
			Chaos::TVector<int32, 3> MIndex;
			Grid.FlatToMultiIndex(i, MIndex, true);
			if (Grid.InteriorNode(MIndex)) 
			{
				EXPECT_NEAR(Lu[i], 0., 1.e-14);
			}
		}

		double Energy = Chaos::LaplacianEnergy<double>(Mesh, De_inverse, measure, u);
		EXPECT_NEAR(Energy, .5 * Chaos::Utilities::DotProduct(u, Lu), 1e-12);

		srand(0);
		for (int32 i = 0; i < u.Num(); i++) 
		{
			u[i] = 2. * rand() / RAND_MAX - 1.;
		}

		Chaos::Laplacian(Mesh, IncidentElements, IncidentElementsLocalIndex, De_inverse, measure, u, Lu);
		Energy = Chaos::LaplacianEnergy(Mesh, De_inverse, measure, u);
		EXPECT_NEAR(Energy, .5 * Chaos::Utilities::DotProduct(u, Lu), 1.e-12);
	}

	TEST(PoissonTests, TestFiberField) 
	{
		//create regular grid
		int32 N = 3;
		double dx = 1. / N;
		TVector<double, 3> Origin(0, 0, 0);
		TVector<double, 3> MinCorner = Origin;
		TVector<double, 3> MaxCorner = Origin + TVector<double, 3>(dx * N, dx * N, dx * N);
		TUniformGrid<double, 3> Grid(MinCorner, MaxCorner, TVector<int32, 3>(N, N, N), 0);

		//create mesh from grid
		TArray<TVector<int, 4>> Mesh;
		TArray<TVector<double, 3>> X;
		Chaos::Utilities::TetMeshFromGrid<double>(Grid, Mesh, X);

		TArray<TArray<int>> IncidentElementsLocalIndex;
		TArray<TArray<int>> IncidentElements = Chaos::Utilities::ComputeIncidentElements(Mesh, &IncidentElementsLocalIndex);

		TArray<int32> Origins;
		TArray<int32> Insertions;
		for (int32 i = 0; i < X.Num(); i++) 
		{
			if (X[i][0] < MinCorner[0] + .1 * dx) 
			{
				Origins.Add(i);
			}
			else if (X[i][0] > MaxCorner[0] - .1 * dx)
			{
				Insertions.Add(i);
			}
		}

		TArray<Chaos::TVector<double, 3>> Directions;
		TArray<double> ScalarField;
		Chaos::ComputeFiberField<double>(Mesh, X, IncidentElements, IncidentElementsLocalIndex, Origins, Insertions, Directions, ScalarField);

		for(int32 e=0; e < Mesh.Num(); e++)
		{
			EXPECT_NEAR(Directions[e][0], 1., 1.e-12);
			for (int32 alpha = 1; alpha < 3; alpha++) 
			{
				EXPECT_NEAR(Directions[e][alpha], 0., 1.e-12);
			}
		}

	}

}
