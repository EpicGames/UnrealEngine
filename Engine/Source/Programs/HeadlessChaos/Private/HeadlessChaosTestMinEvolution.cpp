// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"

#include "Chaos/Collision/BasicCollisionDetector.h"
#include "Chaos/Evolution/PBDMinEvolution.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PBDRigidSpringConstraints.h"

#include "ChaosLog.h"

namespace ChaosTest
{
	using namespace Chaos;

	// Check thar spring constraints work with MinEvolution
	GTEST_TEST(MinEvolutionTests, DISABLED_TestSpringConstraints)
	{
		// @todo(ccaulfield): remove template parameters on collisions and other constraints
		using FCollisionConstraints = FPBDCollisionConstraints;
		using FCollisionDetector = FBasicCollisionDetector;
		using FRigidParticleSOAs = FPBDRigidsSOAs;
		using FParticleHandle = FPBDRigidParticleHandle;
		using FParticlePair = TVec2<FGeometryParticleHandle*>;

		// Particles
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FRigidParticleSOAs ParticlesContainer(UniqueIndices);

		// @todo(ccaulfield): we shouldn't require collisions to use an evolution...
		// Stuff needed for collisions
		TArray<FParticlePair> ActivePotentiallyCollidingPairs;
		TArrayCollectionArray<bool> CollidedParticles;
		TArrayCollectionArray<Chaos::TSerializablePtr<Chaos::FChaosPhysicsMaterial>> ParticleMaterials;
		TArrayCollectionArray<TUniquePtr<Chaos::FChaosPhysicsMaterial>> PerParticleMaterials;
		TArrayCollectionArray<FVec3> ParticlePrevXs;
		TArrayCollectionArray<FRotation3> ParticlePrevRs;
		FCollisionConstraints Collisions(ParticlesContainer, CollidedParticles, ParticleMaterials, PerParticleMaterials, nullptr);
		FBasicBroadPhase BroadPhase(&ActivePotentiallyCollidingPairs, nullptr, nullptr);
		FCollisionDetector CollisionDetector(BroadPhase, Collisions);
		// End collisions stuff

		// Springs
		FPBDRigidSpringConstraints Springs;

		// Evolution
		// @todo(ccaulfield): this should start with some reasonable default iterations
		FPBDMinEvolution Evolution(ParticlesContainer, ParticlePrevXs, ParticlePrevRs, CollisionDetector);
		Evolution.SetNumPositionIterations(6);
		Evolution.SetNumVelocityIterations(1);
		Evolution.SetNumProjectionIterations(1);

		Evolution.AddConstraintContainer(Springs);
		Evolution.SetGravity(FVec3(0));

		FReal Dt = 1.0f / 30.0f;

		// Add a couple dynamic particles connected by a spring
		ParticlesContainer.GetParticleHandles().AddArray(&ParticlePrevXs);
		ParticlesContainer.GetParticleHandles().AddArray(&ParticlePrevRs);
		TArray<FParticleHandle*> Particles = ParticlesContainer.CreateDynamicParticles(2);

		// Set up Particles
		// @todo(ccaulfield) this needs to be easier
		Particles[0]->SetX(FVec3(-50, 0, 0));
		Particles[0]->M() = 1.0f;
		Particles[0]->I() = TVec3<FRealSingle>(100.0f, 100.0f, 100.0f);
		Particles[0]->InvM() = 1.0f;
		Particles[0]->InvI() = TVec3<FRealSingle>(1.0f / 100.0f, 1.0f / 100.0f, 1.0f / 100.0f);
		Particles[0]->AuxilaryValue(ParticlePrevXs) = Particles[0]->GetX();
		Particles[0]->AuxilaryValue(ParticlePrevRs) = Particles[0]->GetR();

		Particles[1]->SetX(FVec3(50, 0, 0));
		Particles[1]->M() = 1.0f;
		Particles[1]->I() = TVec3<FRealSingle>(100.0f, 100.0f, 100.0f);
		Particles[1]->InvM() = 1.0f;
		Particles[1]->InvI() = TVec3<FRealSingle>(1.0f / 100.0f, 1.0f / 100.0f, 1.0f / 100.0f);
		Particles[1]->AuxilaryValue(ParticlePrevXs) = Particles[1]->GetX();
		Particles[1]->AuxilaryValue(ParticlePrevRs) = Particles[1]->GetR();

		// Spring connectors at particle centres
		TArray<FVec3> Locations =
		{
			FVec3(-50, 0, 0),
			FVec3(50, 0, 0)
		};

		// Create springs
		FPBDRigidSpringConstraintHandle* Spring = Springs.AddConstraint({ Particles[0], Particles[1] }, { Locations[0], Locations[1] }, 0.1f, 0.0f, 60.0f);

		for (int32 TimeIndex = 0; TimeIndex < 1000; ++TimeIndex)
		{
			//UE_LOGF(LogChaos, Warning, "%d: %f %f %f - %f %f %f", TimeIndex, Particles[0]->X().X, Particles[0]->X().Y, Particles[0]->X().Z, Particles[1]->X().X, Particles[1]->X().Y, Particles[1]->X().Z);
			Evolution.Advance(Dt, 1, 0.0f);
		}

		// Particles should be separated by the spring's rest length
		FVec3 P0 = Particles[0]->GetX();
		FVec3 P1 = Particles[1]->GetX();
		FReal Distance01 = (P0 - P1).Size();
		EXPECT_NEAR(Distance01, Spring->GetRestLength(), 0.1f);
	}

}
