// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

// for Module Unit Tests
#include "SimModule/ChassisModule.h"
#include "SimModule/AerofoilModule.h"
#include "SimModule/EngineModule.h"
#include "SimModule/ClutchModule.h"
#include "SimModule/TransmissionModule.h"
#include "SimModule/WheelModule.h"
#include "SimModule/SimModuleTree.h"
#include "SimModule/ModuleInput.h"

// for Simulation Tests
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/Box.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"

//////////////////////////////////////////////////////////////////////////
// These tests are mostly working in real word units rather than Unreal 
// units as it's easier to tell if the simulations are working close to 
// reality. i.e. Google stopping distance @ 30MPH ==> typically 15 metres
//////////////////////////////////////////////////////////////////////////

namespace ChaosTest
{
	using namespace Chaos;

	struct FInputsContainer
	{
		FInputsContainer(Chaos::FAllInputs& InInputs)
		{
			ConfigureControlInputs(InInputs);
		}
		
		void ConfigureControlInputs(Chaos::FAllInputs& Inputs)
		{
			FModuleInputSetup InputData1(TEXT("Throttle"), EModuleInputValueType::MAxis1D);
			InputSetupData.Add(InputData1);

			FModuleInputSetup InputData2(TEXT("Brake"), EModuleInputValueType::MAxis1D);
			InputSetupData.Add(InputData2);

			FModuleInputSetup InputData3(TEXT("Steering"), EModuleInputValueType::MAxis1D);
			InputSetupData.Add(InputData3);

			FModuleInputSetup InputData4(TEXT("Clutch"), EModuleInputValueType::MAxis1D);
			InputSetupData.Add(InputData4);

			FModuleInputSetup InputData5(TEXT("Handbrake"), EModuleInputValueType::MAxis1D);
			InputSetupData.Add(InputData5);

			FModuleInputSetup InputData6(TEXT("Pitch"), EModuleInputValueType::MAxis1D);
			InputSetupData.Add(InputData6);

			FModuleInputSetup InputData7(TEXT("Yaw"), EModuleInputValueType::MAxis1D);
			InputSetupData.Add(InputData7);

			FModuleInputSetup InputData8(TEXT("Roll"), EModuleInputValueType::MAxis1D);
			InputSetupData.Add(InputData8);

			FModuleInputSetup InputData9(TEXT("ChangeUp"), EModuleInputValueType::MBoolean);
			InputSetupData.Add(InputData9);

			FModuleInputSetup InputData10(TEXT("ChangeDown"), EModuleInputValueType::MBoolean);
			InputSetupData.Add(InputData10);

			ValueContainer.Initialize(InputSetupData, NameMap);

			ControlInputs = MakeUnique<FInputInterface>(NameMap, ValueContainer, EModuleInputQuantizationType::Default_16Bits);
			Inputs.ControlInputs = ControlInputs.Get();
		}

		FInputInterface::FInputNameMap NameMap;
		FModuleInputContainer ValueContainer;
		TArray<FModuleInputSetup> InputSetupData;
		TUniquePtr<FInputInterface> ControlInputs;
	};

	GTEST_TEST(AllTraits, ModularVehicleTest_Aerofoil)
	{
		FAerofoilSettings RWingSetup;
		RWingSetup.Offset.Set(-0.8f, 3.0f, 0.0f);
		RWingSetup.ForceAxis.Set(0.0f, 0.f, 1.0f);
		RWingSetup.ControlRotationAxis.Set(0.f, 1.f, 0.f);
		RWingSetup.Area = 8.2f;
		RWingSetup.Camber = 3.0f;
		RWingSetup.MaxControlAngle = 1.0f;
		RWingSetup.StallAngle = 16.0f;
		RWingSetup.Type = EAerofoil::Wing;

		FAerofoilSimModule RWing(RWingSetup);

		RWing.SetControlSurface(0.0f);
		RWing.SetDensityOfMedium(RealWorldConsts::AirDensity());

		float Altitude = 100.0f;
		float DeltaTime = 1.0f / 30.0f;

		//////////////////////////////////////////////////////////////////////////

		FVector Velocity(1.0f, 0.0f, 0.0f);

		float AOAFlat = RWing.CalcAngleOfAttackDegrees(FVector(0, 0, 1), FVector(-1, 0, 0));
		EXPECT_LT(AOAFlat, SMALL_NUMBER);

		float AOAFlat2 = RWing.CalcAngleOfAttackDegrees(FVector(0, 0, 1), FVector(1, 0, 0));
		EXPECT_LT(AOAFlat2, SMALL_NUMBER);

		float AOA90 = RWing.CalcAngleOfAttackDegrees(FVector(0, 0, 1), FVector(0, 0, 1));
		EXPECT_LT(AOA90 - 90.0f, SMALL_NUMBER);

		float AOA45 = RWing.CalcAngleOfAttackDegrees(FVector(0, 0, 1), FVector(0, 0.707, 0.707));
		EXPECT_LT(AOA45 - 45.0f, SMALL_NUMBER);

		//////////////////////////////////////////////////////////////////////////
		// Lift
		{
			float Zero = RWing.CalcLiftCoefficient(0, 0);
			EXPECT_LT(Zero, SMALL_NUMBER);

			float Two = RWing.CalcLiftCoefficient(2, 0);
			float NegTwo = RWing.CalcLiftCoefficient(-2, 0);
			EXPECT_GT(Two, SMALL_NUMBER);
			EXPECT_LT(NegTwo, SMALL_NUMBER);
			EXPECT_LT(Two - FMath::Abs(NegTwo), SMALL_NUMBER);

			float Three = RWing.CalcLiftCoefficient(0, 3);
			float NegThree = RWing.CalcLiftCoefficient(0, -3);
			EXPECT_GT(Three, SMALL_NUMBER);
			EXPECT_LT(NegThree, SMALL_NUMBER);
			EXPECT_LT(Three - FMath::Abs(NegThree), SMALL_NUMBER);

			float Nine = RWing.CalcLiftCoefficient(6, 3);
			float NegNine = RWing.CalcLiftCoefficient(-6, -3);
			EXPECT_GT(Nine, SMALL_NUMBER);
			EXPECT_LT(NegNine, SMALL_NUMBER);
			EXPECT_LT(Nine - FMath::Abs(NegNine), SMALL_NUMBER);

			float Stall = RWing.CalcLiftCoefficient(RWingSetup.StallAngle, 0);
			float StallPlus = RWing.CalcLiftCoefficient(RWingSetup.StallAngle, 5);
			EXPECT_GT(Stall, Nine);
			EXPECT_GT(Stall, Three);
			EXPECT_GT(Stall, Two);
			EXPECT_GT(Stall, StallPlus);
		}

		// Drag
		{
			float Two = RWing.CalcDragCoefficient(2, 0);
			float NegTwo = RWing.CalcDragCoefficient(-2, 0);
			EXPECT_GT(Two, SMALL_NUMBER);
			EXPECT_GT(NegTwo, SMALL_NUMBER);
			EXPECT_LT(Two - NegTwo, SMALL_NUMBER);

			float Six = RWing.CalcDragCoefficient(4, 2);
			float NegSix = RWing.CalcDragCoefficient(-4, -2);
			EXPECT_GT(Six, SMALL_NUMBER);
			EXPECT_GT(NegSix, SMALL_NUMBER);
			EXPECT_LT(Six - NegSix, SMALL_NUMBER);

			float AltNegTwo = RWing.CalcDragCoefficient(2, -4);
			EXPECT_GT(AltNegTwo, SMALL_NUMBER);
			EXPECT_LT(AltNegTwo - NegTwo, SMALL_NUMBER);
		}


		////////////////////////////////////////////////////////////////////////////

		FVector Velocity1(0.0f, 0.0f, 10.0f);
		FVector RWForceZero = RWing.GetForce(Velocity1, Altitude, DeltaTime);
		EXPECT_LT(FMath::Abs(RWForceZero.X), SMALL_NUMBER);
		EXPECT_LT(FMath::Abs(RWForceZero.Y), SMALL_NUMBER);
		EXPECT_LT(RWForceZero.Z, 0.f); // drag value opposes velocity direction

		FVector Velocity2(0.0f, 10.0f, 10.0f);
		FVector RWForce3 = RWing.GetForce(Velocity2, Altitude, DeltaTime);
		EXPECT_LT(FMath::Abs(RWForce3.X), SMALL_NUMBER);
		EXPECT_LT(RWForce3.Y, 0.0f);
		EXPECT_LT(RWForce3.Z, 0.0f);

		FVector Velocity3(10.0f, 0.0f, 0.0f);
		FVector RWForce4 = RWing.GetForce(Velocity3, Altitude, DeltaTime);
		EXPECT_LT(RWForce4.X, 0.0f);
		EXPECT_LT(FMath::Abs(RWForce4.Y), SMALL_NUMBER);
		EXPECT_GT(RWForce4.Z, 0.0f);

	}


	// Transmission

	class FTransmissionTestClass : public FTransmissionSimModule
	{
		public:

		FTransmissionTestClass(const FTransmissionSettings& Settings)
		: FTransmissionSimModule(Settings)
		{}

		void Test_TransmissionManualGearSelection()
		{
			FAllInputs Inputs;
			FInputsContainer IC(Inputs);
			FSimModuleTree Tree;

			EXPECT_EQ(GetCurrentGear(), 1);

			// Immediate Gear Change, since Setup.GearChangeTime = 0.0f
			ChangeUp();

			EXPECT_EQ(GetCurrentGear(), 2);
			ChangeUp();
			ChangeUp();
			ChangeUp();
			EXPECT_EQ(GetCurrentGear(), 5);

			ChangeUp();
			EXPECT_EQ(GetCurrentGear(), 5);

			SetGear(1);
			EXPECT_EQ(GetCurrentGear(), 1);

			ChangeDown();
			EXPECT_EQ(GetCurrentGear(), 0);

			ChangeDown();
			EXPECT_EQ(GetCurrentGear(), -1);

			ChangeDown();
			EXPECT_EQ(GetCurrentGear(), -2);

			ChangeDown();
			EXPECT_EQ(GetCurrentGear(), -2);

			ChangeUp();
			EXPECT_EQ(GetCurrentGear(), -1);

			ChangeUp();
			EXPECT_EQ(GetCurrentGear(), 0);

			SetGear(1);

			// Now change settings so we have a delay in the gear changing
			AccessSetup().GearChangeTime = 0.5f;

			ChangeUp();
			EXPECT_EQ(GetCurrentGear(), 0);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), 0);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), 2);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), 2);

			SetGear(4);
			EXPECT_EQ(GetCurrentGear(), 0);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), 0);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), 4);
		}

		void Test_TransmissionAutoGearSelection()
		{
			FAllInputs Inputs;
			FInputsContainer IC(Inputs);
			FSimModuleTree Tree;

			SetGear(1, true);

			SetRPM(1400);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), 1);

			SetRPM(2000);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), 1);

			SetRPM(3000);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), 2);

			SetRPM(2000);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), 2);

			SetRPM(1000);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), 1);

			// stays in first, doesn't change to neutral
			SetRPM(1000);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), 1);

			SetGear(-2, true);

			SetRPM(3000);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), -2);

			SetRPM(1000);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), -1);

			// stays in reverse first, doesn't change to neutral
			SetRPM(1000);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), -1);

			// changes to next reverse gear
			SetRPM(3000);
			Simulate(0.25f, Inputs, Tree);
			EXPECT_EQ(GetCurrentGear(), -2);
		}

		void Test_TransmissionGearRatios()
		{
			float Ratio = 0;
			Ratio = GetGearRatio(-1);
			EXPECT_LT(-12.f - Ratio, SMALL_NUMBER); // -ve output for reverse gears

			Ratio = GetGearRatio(0);
			EXPECT_LT(Ratio, SMALL_NUMBER);

			Ratio = GetGearRatio(1);
			EXPECT_LT(16.f - Ratio, SMALL_NUMBER);

			Ratio = GetGearRatio(2);
			EXPECT_LT(12.f - Ratio, SMALL_NUMBER);

			Ratio = GetGearRatio(3);
			EXPECT_LT(8.f - Ratio, SMALL_NUMBER);

			Ratio = GetGearRatio(4);
			EXPECT_LT(4.f - Ratio, SMALL_NUMBER);
		}
	};

	GTEST_TEST(AllTraits, ModularVehicleTest_TransmissionManualGearSelection)
	{
		// done this way so we can access protected function calls

		FTransmissionSettings Setup;
		{
			Setup.ForwardRatios.Empty();
			Setup.ReverseRatios.Empty();
			Setup.ForwardRatios.Add(4.f);
			Setup.ForwardRatios.Add(3.f);
			Setup.ForwardRatios.Add(2.f);
			Setup.ForwardRatios.Add(1.f);
			Setup.ForwardRatios.Add(0.8f);
			Setup.ReverseRatios.Add(3.f);
			Setup.ReverseRatios.Add(6.f);
			Setup.FinalDriveRatio = 4.f;
			Setup.ChangeUpRPM = 3000;
			Setup.ChangeDownRPM = 1200;
			Setup.GearChangeTime = 0.0f;
			Setup.TransmissionType = FTransmissionSettings::ETransType::ManualType;
			Setup.AutoReverse = true;
			Setup.TransmissionEfficiency = 1.0f;
		}

		FTransmissionTestClass Transmission(Setup);
		Transmission.Test_TransmissionManualGearSelection();
	}

	GTEST_TEST(AllTraits, ModularVehicleTest_TransmissionAutoGearSelection)
	{
		FTransmissionSettings Setup;
		{
			Setup.ForwardRatios.Empty();
			Setup.ReverseRatios.Empty();
			Setup.ForwardRatios.Add(4.f);
			Setup.ForwardRatios.Add(3.f);
			Setup.ForwardRatios.Add(2.f);
			Setup.ForwardRatios.Add(1.f);
			Setup.ReverseRatios.Add(3.f);
			Setup.ReverseRatios.Add(6.f);
			Setup.FinalDriveRatio = 4.f;
			Setup.ChangeUpRPM = 3000;
			Setup.ChangeDownRPM = 1200;
			Setup.GearChangeTime = 0.0f;
			Setup.GearHysteresisTime = 0.0f;
			Setup.TransmissionType = FTransmissionSettings::ETransType::AutomaticType;
			Setup.AutoReverse = false;
			Setup.TransmissionEfficiency = 1.0f;
		}

		FTransmissionTestClass Transmission(Setup);
		Transmission.Test_TransmissionAutoGearSelection();
	}

	GTEST_TEST(AllTraits, ModularVehicleTest_TransmissionGearRatios)
	{
		FTransmissionSettings Setup;
		{
			Setup.ForwardRatios.Empty();
			Setup.ReverseRatios.Empty();
			Setup.ForwardRatios.Add(4.f);
			Setup.ForwardRatios.Add(3.f);
			Setup.ForwardRatios.Add(2.f);
			Setup.ForwardRatios.Add(1.f);
			Setup.ReverseRatios.Add(3.f);
			Setup.FinalDriveRatio = 4.f;
			Setup.ChangeUpRPM = 3000;
			Setup.ChangeDownRPM = 1200;
			Setup.GearChangeTime = 0.0f;
			Setup.TransmissionType = FTransmissionSettings::ETransType::AutomaticType;
			Setup.AutoReverse = true;
			Setup.TransmissionEfficiency = 1.0f;
		}

		FTransmissionTestClass Transmission(Setup);
		Transmission.Test_TransmissionGearRatios();
	}

	// Wheel
	void SimulateBraking(FWheelSimModule& Wheel
		, const float Gravity
		, float VehicleSpeed
		, float DeltaTime
		, float& StoppingDistanceOut
		, float& SimulationTimeOut
		, bool bLoggingEnabled = false
		)
	{
		FAllInputs Inputs;
		FInputsContainer IC(Inputs);
		Inputs.GetControls().SetValue(TEXT("Throttle"), 0.0f);
		Inputs.GetControls().SetValue(TEXT("Brake"), 1.0f); // Apply full brake
		FSimModuleTree Tree;

		StoppingDistanceOut = 0.f;
		SimulationTimeOut = 0.f;
		
		float MaxSimTime = 15.0f;
		float VehicleMass = 1300.f;
		float VehicleMassPerWheel = 1300.f / 4.f;

		Wheel.SetForceIntoSurface(VehicleMassPerWheel * Gravity);

		// Road speed
		FVector Velocity = FVector(VehicleSpeed, 0.f, 0.f);

		// wheel rolling speed matches road speed
		Wheel.SetLinearSpeed(Velocity.X);

		if (bLoggingEnabled)
		{
			UE_LOGF(LogChaos, Warning, "--------------------START---------------------");
		}

		while (SimulationTimeOut < MaxSimTime)
		{
			// rolling speed matches road speed
			Wheel.SetLocalLinearVelocity(Velocity);
			Wheel.Simulate(DeltaTime, Inputs, Tree);

			// deceleration from brake, F = m * a, a = F / m, v = dt * F / m
			Velocity += DeltaTime * Wheel.GetForceFromFriction() / VehicleMassPerWheel;
			StoppingDistanceOut += Velocity.X * DeltaTime;

			if (bLoggingEnabled)
			{
				UE_LOGF(LogChaos, Warning, "Wheel.GetForceFromFriction() %ls", *Wheel.GetForceFromFriction().ToString());
			}

			if (FMath::Abs(Velocity.X) < 0.05f)
			{
				Velocity.X = 0.f;
				break; // break out early if already stopped
			}

			SimulationTimeOut += DeltaTime;
		}

		if (bLoggingEnabled)
		{
			UE_LOGF(LogChaos, Warning, "---------------------END----------------------");
		}


	}

	void SimulateAccelerating(FWheelSimModule& Wheel
		, const float Gravity
		, const float DriveTorque
		, float FinalVehicleSpeed
		, float DeltaTime
		, float& DistanceTravelledOut
		, float& SimulationTimeOut
		)
	{
		FAllInputs Inputs;
		FInputsContainer IC(Inputs);
		Inputs.GetControls().SetValue(TEXT("Throttle"), 1.0f); // Apply full throttle
		Inputs.GetControls().SetValue(TEXT("Brake"), 0.0f);
		FSimModuleTree Tree;

		DistanceTravelledOut = 0.f;
		SimulationTimeOut = 0.f;

		float MaxSimTime = 15.0f;
		float VehicleMass = 1300.f;
		float VehicleMassPerWheel = VehicleMass / 4.f;

		Wheel.SetForceIntoSurface(VehicleMassPerWheel * Gravity);

		// Road speed
		FVector Velocity = FVector(0.f, 0.f, 0.f);

		// start from stationary
		Wheel.SetLinearSpeed(Velocity.X);

		while (SimulationTimeOut < MaxSimTime)
		{
			Wheel.SetLocalLinearVelocity(Velocity);
			Wheel.SetDriveTorque(DriveTorque);
			Wheel.Simulate(DeltaTime, Inputs, Tree);

			Velocity += DeltaTime * Wheel.GetForceFromFriction() / VehicleMassPerWheel;
			DistanceTravelledOut += Velocity.X * DeltaTime;

			SimulationTimeOut += DeltaTime;

			if (FMath::Abs(Velocity.X) >= FinalVehicleSpeed)
			{
				break; // time is up
			}

		}
	}

	GTEST_TEST(AllTraits, ModularVehicleTest_WheelBrakingLongitudinalSlip)
	{
		FWheelSettings Setup;
		Setup.ABSEnabled = false;
		Setup.TractionControlEnabled = false;
		Setup.SteeringEnabled = true;
		Setup.HandbrakeEnabled = true;
		Setup.Radius = 0.3f;
		Setup.FrictionMultiplier = 1.0f;
		Setup.CorneringStiffness = 1000.0f;
		Setup.LateralSlipGraphMultiplier = 0.7f;

		FWheelSimModule Wheel(Setup);

		// Google braking distance at 30mph says 14m (not interested in the thinking distance part)
		// So using a range 10-20 to ensure we are in the correct ballpark.
		// If specified more accurately in the test, then modifying the code would break the test all the time.

		// units meters
		float Gravity = 9.8f;
		float StoppingDistanceTolerance = 0.5f;
		float DeltaTime = 1.f / 30.f;
		float StoppingDistanceA = 0.f;
		float SimulationTime = 0.0f;
		Wheel.SetSurfaceFriction(RealWorldConsts::DryRoadFriction());

		// reasonably ideal stopping distance - traveling forwards
		Wheel.AccessSetup().MaxBrakeTorque = 650.0f;
 		SimulateBraking(Wheel, Gravity, MPHToMS(30.f), DeltaTime, StoppingDistanceA, SimulationTime);
		EXPECT_GT(StoppingDistanceA, 10.f);
		EXPECT_LT(StoppingDistanceA, 20.f);

		// traveling backwards stops just the same
		float StoppingDistanceReverseDir = 0.f;
		Wheel.AccessSetup().MaxBrakeTorque = 650.0f;
		SimulateBraking(Wheel, Gravity, MPHToMS(-30.f), DeltaTime, StoppingDistanceReverseDir, SimulationTime);
		EXPECT_GT(StoppingDistanceReverseDir, -20.f);
		EXPECT_LT(StoppingDistanceReverseDir, -10.f);
		EXPECT_LT(StoppingDistanceA - FMath::Abs(StoppingDistanceReverseDir), StoppingDistanceTolerance);

		// Changing to units of Cm should yield the same results
		float MToCm = 100.0f;
		float StoppingDistanceCm = 0.f;
		Wheel.AccessSetup().MaxBrakeTorque = 650.0f * MToCm * MToCm;
		Wheel.AccessSetup().Radius = (0.3f * MToCm);
		SimulateBraking(Wheel, Gravity * MToCm, MPHToCmS(30.f), DeltaTime, StoppingDistanceCm, SimulationTime);
		EXPECT_NEAR(StoppingDistanceCm, StoppingDistanceA * MToCm, 1.0f);

		// Similar results with different delta time
		float StoppingDistanceDiffDT = 0.f;
		Wheel.AccessSetup().MaxBrakeTorque = 650.0f;
		Wheel.AccessSetup().Radius = 0.3f;
		SimulateBraking(Wheel, Gravity, MPHToMS(30.f), DeltaTime * 0.25f, StoppingDistanceDiffDT, SimulationTime);
		EXPECT_LT(StoppingDistanceA - StoppingDistanceDiffDT, StoppingDistanceTolerance);

		// barely touching the brake - going to take longer to stop
		float StoppingDistanceLightBraking = 0.f;
		Wheel.AccessSetup().MaxBrakeTorque = 150.0f;
		SimulateBraking(Wheel, Gravity, MPHToMS(30.f), DeltaTime, StoppingDistanceLightBraking, SimulationTime);
		EXPECT_GT(StoppingDistanceLightBraking, StoppingDistanceA);

		// locking the wheels / too much brake torque -> dynamic friction rather than static friction -> going to take longer to stop
		float StoppingDistanceTooHeavyBreaking = 0.f;
		Wheel.AccessSetup().MaxBrakeTorque = 5000.0f;
		SimulateBraking(Wheel, Gravity, MPHToMS(30.f), DeltaTime, StoppingDistanceTooHeavyBreaking, SimulationTime);
		EXPECT_GT(StoppingDistanceTooHeavyBreaking, StoppingDistanceA);

		// Would have locked the wheels but ABS prevents skidding
		Wheel.AccessSetup().ABSEnabled = true;
		float StoppingDistanceTooHeavyBreakingABS = 0.f;
		Wheel.AccessSetup().MaxBrakeTorque = 5000.0f;
		SimulateBraking(Wheel, Gravity, MPHToMS(30.f), DeltaTime, StoppingDistanceTooHeavyBreakingABS, SimulationTime);
		EXPECT_LT(StoppingDistanceTooHeavyBreakingABS, StoppingDistanceTooHeavyBreaking);
		Wheel.AccessSetup().ABSEnabled = false;

		// lower initial speed - stops more quickly
		float StoppingDistanceLowerSpeed = 0.f;
		Wheel.AccessSetup().MaxBrakeTorque = 650.0f;
		SimulateBraking(Wheel, Gravity, MPHToMS(20.f), DeltaTime, StoppingDistanceLowerSpeed, SimulationTime);
		EXPECT_LT(StoppingDistanceLowerSpeed, StoppingDistanceA);

		// higher initial speed - stops more slowly
		float StoppingDistanceHigherSpeed = 0.f;
		Wheel.AccessSetup().MaxBrakeTorque = 650.0f;
		SimulateBraking(Wheel, Gravity, MPHToMS(60.f), DeltaTime, StoppingDistanceHigherSpeed, SimulationTime);
		EXPECT_GT(StoppingDistanceHigherSpeed, StoppingDistanceA);

		// slippy surface - stops more slowly
		float StoppingDistanceLowFriction = 0.f;
		Wheel.SetSurfaceFriction(0.3f);
		Wheel.AccessSetup().MaxBrakeTorque = 650.0f;
		SimulateBraking(Wheel, Gravity, MPHToMS(30.f), DeltaTime, StoppingDistanceLowFriction, SimulationTime);
		EXPECT_GT(StoppingDistanceLowFriction, StoppingDistanceA);
	}

	GTEST_TEST(AllTraits, ModularVehicleTest_WheelAcceleratingLongitudinalSlip)
	{
		FWheelSettings Setup;
		Setup.ABSEnabled = false;
		Setup.TractionControlEnabled = false;
		Setup.SteeringEnabled = false;
		Setup.HandbrakeEnabled = false;
		Setup.Radius = 0.3f;
		Setup.FrictionMultiplier = 1.0f;
		Setup.CorneringStiffness = 1000.0f;
		Setup.LateralSlipGraphMultiplier = 0.7f;

		FWheelSimModule Wheel(Setup);

		// There could be one frame extra computation on the acceleration since the last frame of brake is not using the full 
		// amount of torque, it's clearing the last remaining velocity without pushing the vehicle back in the opposite direction
		// Hence a slightly larger tolerance for the result
		float AccelerationResultsTolerance = 1.0f; // meters

		// units meters
		float Gravity = 9.8f;
		float DeltaTime = 1.f / 30.f;
		float DriveTorque = 0.0;

		float StoppingDistanceA = 0.f;
		float SimulationTimeBrake = 0.0f;
		Wheel.SetSurfaceFriction(RealWorldConsts::DryRoadFriction());

		// How far & what time does it take to stop from 30MPH to rest
		Wheel.AccessSetup().MaxBrakeTorque = 650.0f;
		SimulateBraking(Wheel, Gravity, MPHToMS(30.0f), DeltaTime, StoppingDistanceA, SimulationTimeBrake);

		// How far and what time does it take to accelerate from rest to 30MPH
		float SimulationTimeAccel = 0.0f;
		float DrivingDistanceA = 0.f;
		DriveTorque = 650.0f;
		SimulateAccelerating(Wheel, Gravity, DriveTorque, MPHToMS(30.0f), DeltaTime, DrivingDistanceA, SimulationTimeAccel);

		// 0-30 MPH and 30-0 MPH should be the same if there's no slipping and accel torque was same as the brake torque run
		EXPECT_LT(DrivingDistanceA - StoppingDistanceA, AccelerationResultsTolerance);
		EXPECT_LT(SimulationTimeAccel - SimulationTimeBrake, AccelerationResultsTolerance);

		// same range as braking from 30MPH
		EXPECT_GT(DrivingDistanceA, 10.f);
		EXPECT_LT(DrivingDistanceA, 20.f);

		// Unreal units cm - Note for the same results the radius needs to remain at 0.3m and not also be scaled to 30(cm)
		float SimulationTimeAccelCM = 0.0f;
		float MToCm = 100.0f;
		float DrivingDistanceCM = 0.f;
		DriveTorque = 650.0f * MToCm * MToCm;
		Wheel.AccessSetup().Radius = (0.3f * MToCm);
		SimulateAccelerating(Wheel, Gravity * MToCm, DriveTorque, MPHToCmS(30.0f), DeltaTime, DrivingDistanceCM, SimulationTimeAccelCM);
		EXPECT_GT(DrivingDistanceCM, 10.f * MToCm);
		EXPECT_LT(DrivingDistanceCM, 20.f * MToCm);
		EXPECT_NEAR(DrivingDistanceCM, DrivingDistanceA * MToCm, AccelerationResultsTolerance);

		float SimulationTimeAccelSpin = 0.0f;
		float DrivingDistanceWheelspin = 0.f;
		Wheel.AccessSetup().Radius = 0.3f;
		DriveTorque = 5000; // definitely cause wheel spin
		SimulateAccelerating(Wheel, Gravity, DriveTorque, 30.0f, DeltaTime, DrivingDistanceWheelspin, SimulationTimeAccelSpin);

		// drives further to reach the same speed
		EXPECT_GT(DrivingDistanceWheelspin, DrivingDistanceA);

		// takes longer to reach the same speed
		EXPECT_GT(SimulationTimeAccelSpin, SimulationTimeAccel);

		// Enable traction control should be better than both of the above
		float SimulationTimeAccelTC = 0.0f;
		float DrivingDistanceTC = 0.f;
		Wheel.AccessSetup().TractionControlEnabled = true;
		DriveTorque = 5000; // definitely cause wheel spin
		SimulateAccelerating(Wheel, Gravity, DriveTorque, MPHToMS(30.0f), DeltaTime, DrivingDistanceTC, SimulationTimeAccelTC);

		// reaches target speed in a shorter distance
		EXPECT_LT(DrivingDistanceTC, DrivingDistanceWheelspin);

		// reaches speed quicker with TC on when wheel would be slipping from drive torque
		EXPECT_LT(SimulationTimeAccelTC, SimulationTimeAccelSpin);

	}

	GTEST_TEST(AllTraits, ModularVehicleTest_WheelRolling)
	{
		FWheelSettings Setup;
		Setup.Radius = 0.3f;
		FWheelSimModule Wheel(Setup);

		FAllInputs Inputs;
		FInputsContainer IC(Inputs);
		Inputs.GetControls().SetValue(TEXT("Throttle"), 0.0f);
		Inputs.GetControls().SetValue(TEXT("Brake"), 0.0f);
		FSimModuleTree Tree;

		float DeltaTime = 1.f / 30.f;
		float MaxSimTime = 10.0f;
		float Tolerance = 0.1f; // wheel friction losses slow wheel speed

		//------------------------------------------------------------------
		// Car is moving FORWARDS - with AMPLE friction we would expect an initially 
		// static rolling wheel to speed up and match the vehicle speed
		FVector VehicleGroundSpeed(10.0f, 0.0f, 0.0f);
		Wheel.SetSurfaceFriction(1.0f);	// Some wheel/ground friction
		Wheel.SetForceIntoSurface(250.f); // wheel pressed into the ground, to give it grip
		Wheel.SetAngularVelocity(0.f);
		Wheel.SetLocalLinearVelocity(VehicleGroundSpeed);

		// initially wheel is static
		EXPECT_LT(Wheel.GetAngularVelocity(), SMALL_NUMBER);

		// after some time, the wheel picks up speed to match the vehicle speed
		float SimulatedTime = 0.f;
		while (SimulatedTime < MaxSimTime)
		{
			Wheel.Simulate(DeltaTime, Inputs, Tree);
			SimulatedTime += DeltaTime;
		}

		// there's enough grip to cause the wheel to spin and match the vehivle speed
		float WheelGroundSpeed = Wheel.GetAngularVelocity() * Wheel.GetEffectiveRadius();
		EXPECT_LT(VehicleGroundSpeed.X - WheelGroundSpeed, Tolerance);
		EXPECT_LT(VehicleGroundSpeed.X - WheelGroundSpeed, Tolerance);
		EXPECT_GT(Wheel.GetAngularVelocity(), 0.f); // +ve spin on it

		//------------------------------------------------------------------
		// Car is moving BACKWARDS - with AMPLE friction we would expect an initially 
		// static rolling wheel to speed up and match the vehicle speed
		VehicleGroundSpeed.Set(-10.0f, 0.0f, 0.0f);
		Wheel.SetSurfaceFriction(1.0f);	// Some wheel/ground friction
		Wheel.SetForceIntoSurface(250.f); // wheel pressed into the ground, to give it grip
		Wheel.SetAngularVelocity(0.f);
		Wheel.SetLocalLinearVelocity(VehicleGroundSpeed);

		// initially wheel is static
		EXPECT_LT(Wheel.GetAngularVelocity(), SMALL_NUMBER);

		// after some time, the wheel picks up speed to match the vehicle speed
		SimulatedTime = 0.f;
		while (SimulatedTime < MaxSimTime)
		{
			Wheel.Simulate(DeltaTime, Inputs, Tree);
			SimulatedTime += DeltaTime;
		}

		// there's enough grip to cause the wheel to spin and match the vehicle speed
		WheelGroundSpeed = Wheel.GetAngularVelocity() * Wheel.GetEffectiveRadius();
		EXPECT_LT(VehicleGroundSpeed.X - WheelGroundSpeed, Tolerance);
		EXPECT_LT(VehicleGroundSpeed.X - Wheel.GetLinearSpeed(), Tolerance);
		EXPECT_LT(Wheel.GetAngularVelocity(), 0.f); // -ve spin on it

		//------------------------------------------------------------------
		// Car is moving FORWARDS - with NO friction we would expect an initially 
		// static wheel to NOT speed up to match the vehicle speed
		Wheel.SetSurfaceFriction(0.0f);	// No wheel/ground friction
		Wheel.SetForceIntoSurface(250.f); // wheel pressed into the ground, to give it grip
		Wheel.SetAngularVelocity(0.f);
		Wheel.SetLocalLinearVelocity(VehicleGroundSpeed);

		// initially wheel is static
		EXPECT_LT(Wheel.GetAngularVelocity(), SMALL_NUMBER);

		// after some time, the wheel picks up speed to match the vehicle speed
		SimulatedTime = 0.f;
		while (SimulatedTime < MaxSimTime)
		{
			Wheel.Simulate(DeltaTime, Inputs, Tree);
			SimulatedTime += DeltaTime;
		}

		WheelGroundSpeed = Wheel.GetAngularVelocity() * Wheel.GetEffectiveRadius();

		// wheel is just sliding there's no friction to make it spin
		EXPECT_LT(WheelGroundSpeed, SMALL_NUMBER);

	}

	// Engine
	GTEST_TEST(AllTraits, ModularVehicleTest_EngineRPM)
	{
		FAllInputs Inputs;
		FInputsContainer IC(Inputs);
		FSimModuleTree Tree;

		FEngineSettings Setup;
		{
			Setup.MaxRPM = 5000;
			Setup.IdleRPM = 1000;
			Setup.MaxTorque = 400.f;
			Setup.EngineBrakeEffect = 200.0f;
			Setup.EngineInertia = 100.0f;

			Setup.TorqueCurve.Empty();

			Setup.TorqueCurve.AddNormalized(0.5f);
			Setup.TorqueCurve.AddNormalized(0.5f);
			Setup.TorqueCurve.AddNormalized(0.5f);
			Setup.TorqueCurve.AddNormalized(0.5f);
			Setup.TorqueCurve.AddNormalized(0.6f);
			Setup.TorqueCurve.AddNormalized(0.7f);
			Setup.TorqueCurve.AddNormalized(0.8f);
			Setup.TorqueCurve.AddNormalized(0.9f);
			Setup.TorqueCurve.AddNormalized(1.0f);
			Setup.TorqueCurve.AddNormalized(0.9f);
			Setup.TorqueCurve.AddNormalized(0.7f);
			Setup.TorqueCurve.AddNormalized(0.5f);
		}

		FEngineSimModule Engine(Setup);

		Inputs.GetControls().SetValue(TEXT("Throttle"), 0.0f);

		float DeltaTime = 1.0f / 30.0f;
		float TOLERANCE = 0.1f;

		// engine idle - no throttle
		for (int i = 0; i < 200; i++)
		{
			Engine.Simulate(DeltaTime, Inputs, Tree);
		}

		EXPECT_LT(Engine.GetRPM() - Engine.Setup().IdleRPM, TOLERANCE);

		// apply half throttle
		Inputs.GetControls().SetValue(TEXT("Throttle"), 0.5f);

		for (int i = 0; i < 100; i++)
		{
			Engine.Simulate(DeltaTime, Inputs, Tree);
			//UE_LOGF(LogChaos, Warning, "EngineSpeed %.2f rad/sec (%.1f RPM)", Engine.GetAngularVelocity(), Engine.GetRPM());
		}

		EXPECT_GT(Engine.GetRPM(), Engine.Setup().IdleRPM);

		Inputs.GetControls().SetValue(TEXT("Throttle"), 0.0f);

		// engine idle - no throttle
		for (int i = 0; i < 200; i++)
		{
			Engine.Simulate(DeltaTime, Inputs, Tree);
			//UE_LOGF(LogChaos, Warning, "EngineSpeed %.2f rad/sec (%.1f RPM)", Engine.GetAngularVelocity(), Engine.GetRPM());
		}

		EXPECT_LT(Engine.GetRPM() - Engine.Setup().IdleRPM, TOLERANCE);

	}


	GTEST_TEST(AllTraits, ModularVehicleTest_SimModuleTree_EngineDrivingWheelsThroughClutch)
	{
		FAllInputs Inputs;
		FInputsContainer IC(Inputs);
		FSimModuleTree Tree;
		float TOLERANCE = 0.1f;

		FEngineSettings Setup;
		{
			Setup.MaxRPM = 5000;
			Setup.IdleRPM = 1000;
			Setup.MaxTorque = 400.f;
			Setup.EngineBrakeEffect = 20.0f;
			Setup.EngineInertia = 100.0f;

			Setup.TorqueCurve.Empty();

			Setup.TorqueCurve.AddNormalized(0.5f);
			Setup.TorqueCurve.AddNormalized(0.5f);
			Setup.TorqueCurve.AddNormalized(0.5f);
			Setup.TorqueCurve.AddNormalized(0.5f);
			Setup.TorqueCurve.AddNormalized(0.6f);
			Setup.TorqueCurve.AddNormalized(0.7f);
			Setup.TorqueCurve.AddNormalized(0.8f);
			Setup.TorqueCurve.AddNormalized(0.9f);
			Setup.TorqueCurve.AddNormalized(1.0f);
			Setup.TorqueCurve.AddNormalized(0.9f);
			Setup.TorqueCurve.AddNormalized(0.7f);
			Setup.TorqueCurve.AddNormalized(0.5f);
		}

		FChassisSettings ChassisSettings;
		int RootNodeIndex = Tree.AddRoot(new FChassisSimModule(ChassisSettings));

		FEngineSettings EngineSettings;
		int NodeIndex = Tree.AddNodeBelow(RootNodeIndex, new FEngineSimModule(EngineSettings));

		FClutchSettings ClutchSettings;
		NodeIndex = Tree.AddNodeBelow(NodeIndex, new FClutchSimModule(ClutchSettings));

		FTransmissionSettings TransmissionSettings;
		int TransmissionNodeIndex = Tree.AddNodeBelow(NodeIndex, new FTransmissionSimModule(TransmissionSettings));

		FWheelSettings WheelSettingsDriven;
		FWheelSettings WheelSettingsRolling;

		int Wheel0Idx = Tree.AddNodeBelow(TransmissionNodeIndex, new FWheelSimModule(WheelSettingsDriven));
		int Wheel1Idx = Tree.AddNodeBelow(TransmissionNodeIndex, new FWheelSimModule(WheelSettingsDriven));
		int Wheel2Idx = Tree.AddNodeBelow(RootNodeIndex, new FWheelSimModule(WheelSettingsRolling));
		int Wheel3Idx = Tree.AddNodeBelow(RootNodeIndex, new FWheelSimModule(WheelSettingsRolling));
		
		const FWheelSimModule& Wheel0 = *static_cast<const FWheelSimModule*>(Tree.GetSimModule(Wheel0Idx));
		const FWheelSimModule& Wheel1 = *static_cast<const FWheelSimModule*>(Tree.GetSimModule(Wheel1Idx));
		const FWheelSimModule& Wheel2 = *static_cast<const FWheelSimModule*>(Tree.GetSimModule(Wheel2Idx));
		const FWheelSimModule& Wheel3 = *static_cast<const FWheelSimModule*>(Tree.GetSimModule(Wheel3Idx));

		IPhysicsProxyBase* Proxy = nullptr;
		FPBDRigidParticleHandle* Particle = nullptr;

		float DeltaTime = 1.0f / 30.0f;

		// We need to setup reverse pointer to tree
		for (int I = 0; I < Tree.GetNumNodes(); I++)
		{
			Tree.AccessSimModule(I)->SetSimModuleTree(&Tree);
		}

		// Throttle ON
		Inputs.GetControls().SetValue(TEXT("Throttle"), 1.0f);
		Inputs.GetControls().SetValue(TEXT("Brake"), 0.0f);
		Inputs.GetControls().SetValue(TEXT("Clutch"), 0.0f);

		// wheels not moving initially
		EXPECT_NEAR(Wheel0.GetRPM(), 0, TOLERANCE);
		EXPECT_NEAR(Wheel1.GetRPM(), 0, TOLERANCE);
		EXPECT_NEAR(Wheel2.GetRPM(), 0, TOLERANCE);
		EXPECT_NEAR(Wheel3.GetRPM(), 0, TOLERANCE);
		
		// Simulate
		for (int I=0; I<50; I++)
		{
			Tree.Simulate(DeltaTime, Inputs, Proxy, Particle);
		}

		// driven wheels are turning
		EXPECT_GT(Wheel0.GetRPM(), TOLERANCE);
		EXPECT_GT(Wheel1.GetRPM(), TOLERANCE);

		// non-driven wheels are still static
		EXPECT_NEAR(Wheel2.GetRPM(), 0, TOLERANCE);
		EXPECT_NEAR(Wheel3.GetRPM(), 0, TOLERANCE);

		// Brake ON 
		Inputs.GetControls().SetValue(TEXT("Throttle"), 0.0f);
		Inputs.GetControls().SetValue(TEXT("Brake"), 1.0f);
		Inputs.GetControls().SetValue(TEXT("Clutch"), 0.0f);

		// Simulate
		for (int I = 0; I < 50; I++)
		{
			Tree.Simulate(DeltaTime, Inputs, Proxy, Particle);
		}

		// all wheels stop spinning
		EXPECT_NEAR(Wheel0.GetRPM(), 0, TOLERANCE);
		EXPECT_NEAR(Wheel1.GetRPM(), 0, TOLERANCE);
		EXPECT_NEAR(Wheel2.GetRPM(), 0, TOLERANCE);
		EXPECT_NEAR(Wheel3.GetRPM(), 0, TOLERANCE);


		// Throttle ON, Clutch depressed
		Inputs.GetControls().SetValue(TEXT("Throttle"), 1.0f);
		Inputs.GetControls().SetValue(TEXT("Brake"), 0.0f);
		Inputs.GetControls().SetValue(TEXT("Clutch"), 1.0f);

		// Simulate
		for (int I = 0; I < 50; I++)
		{
			Tree.Simulate(DeltaTime, Inputs, Proxy, Particle);
		}

		// driven wheels are not turning due to disengaged clutch
		EXPECT_NEAR(Wheel0.GetRPM(), 0, TOLERANCE);
		EXPECT_NEAR(Wheel1.GetRPM(), 0, TOLERANCE);

	}

	GTEST_TEST(AllTraits, ModularVehicleTest_SimModuleTree_WheelsSpinEngineShouldSpin)
	{
		FAllInputs Inputs;
		FInputsContainer IC(Inputs);
		FSimModuleTree Tree;
		float TOLERANCE = 0.1f;

		FChassisSettings ChassisSettings;
		int RootNodeIndex = Tree.AddRoot(new FChassisSimModule(ChassisSettings));

		Inputs.GetControls().SetValue(TEXT("Throttle"), 0.0f);
		Inputs.GetControls().SetValue(TEXT("Brake"), 0.0f);

		FEngineSettings EngineSettings;
		{
			EngineSettings.MaxRPM = 5000;
			EngineSettings.IdleRPM = 0;
			EngineSettings.MaxTorque = 400.f;
			EngineSettings.EngineBrakeEffect = 0.0f;
			EngineSettings.EngineInertia = 100.0f;
			EngineSettings.TorqueCurve.Empty();
			EngineSettings.TorqueCurve.AddNormalized(0.5f);
			EngineSettings.TorqueCurve.AddNormalized(0.5f);
			EngineSettings.TorqueCurve.AddNormalized(0.5f);
			EngineSettings.TorqueCurve.AddNormalized(0.5f);
			EngineSettings.TorqueCurve.AddNormalized(0.6f);
			EngineSettings.TorqueCurve.AddNormalized(0.7f);
			EngineSettings.TorqueCurve.AddNormalized(0.8f);
			EngineSettings.TorqueCurve.AddNormalized(0.9f);
			EngineSettings.TorqueCurve.AddNormalized(1.0f);
			EngineSettings.TorqueCurve.AddNormalized(0.9f);
			EngineSettings.TorqueCurve.AddNormalized(0.7f);
			EngineSettings.TorqueCurve.AddNormalized(0.5f);
		}
		int EngineNodeIndex = Tree.AddNodeBelow(RootNodeIndex, new FEngineSimModule(EngineSettings));

		FWheelSettings WheelSettingsDriven;
		FWheelSettings WheelSettingsRolling;
		WheelSettingsDriven.AutoHandbrakeEnabled = false;
		WheelSettingsRolling.AutoHandbrakeEnabled = false;

		int Wheel0Idx = Tree.AddNodeBelow(EngineNodeIndex, new FWheelSimModule(WheelSettingsDriven));
		int Wheel1Idx = Tree.AddNodeBelow(EngineNodeIndex, new FWheelSimModule(WheelSettingsDriven));

		FEngineSimModule& Engine = *static_cast<FEngineSimModule*>(Tree.AccessSimModule(EngineNodeIndex));
		FWheelSimModule& Wheel0 = *static_cast<FWheelSimModule*>(Tree.AccessSimModule(Wheel0Idx));
		FWheelSimModule& Wheel1 = *static_cast<FWheelSimModule*>(Tree.AccessSimModule(Wheel1Idx));

		IPhysicsProxyBase* Proxy = nullptr;
		FPBDRigidParticleHandle* Particle = nullptr;
		float DeltaTime = 1.0f / 30.0f;

		// We need to setup reverse pointer to tree
		for (int I = 0; I < Tree.GetNumNodes(); I++)
		{
			Tree.AccessSimModule(I)->SetSimModuleTree(&Tree);
		}


		// Simulate - engine takes speed from wheels spin
		for (int I = 0; I < 5; I++)
		{
			Wheel0.SetLinearSpeed(100);
			Wheel1.SetLinearSpeed(100);
			Tree.Simulate(DeltaTime, Inputs, Proxy, Particle);
		}

		EXPECT_NEAR(Engine.GetRPM(), Wheel0.GetRPM(), TOLERANCE);


		// Simulate - engine will just need to take average of connected wheels speeds
		for (int I = 0; I < 5; I++)
		{
			Wheel0.SetLinearSpeed(300);
			Wheel1.SetLinearSpeed(100);
			Tree.Simulate(DeltaTime, Inputs, Proxy, Particle);
		}

		EXPECT_NEAR(Engine.GetRPM(), (Wheel0.GetRPM() + Wheel1.GetRPM())*0.5f, TOLERANCE);

	}


}


