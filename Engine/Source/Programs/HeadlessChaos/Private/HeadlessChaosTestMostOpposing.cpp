// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestMostOpposing.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Convex.h"

namespace ChaosTest
{
	using namespace Chaos;

	/*We  want to test the following:
	- Correct face index simple case
	- Correct face on shared edge
	*/

	void TrimeshMostOpposing()
	{
		{
			FReal Time;
			FVec3 Position;
			FVec3 Normal;
			TArray<uint16> DummyMaterials;
			int32 FaceIndex;

			FTriangleMeshImplicitObject::ParticlesType Particles;
			Particles.AddParticles(6);
			Particles.SetX(0, FVec3(1, 1, 1));
			Particles.SetX(1, FVec3(5, 1, 1));
			Particles.SetX(2, FVec3(1, 5, 1));
			Particles.SetX(3, FVec3(1, 1, 1));
			Particles.SetX(4, FVec3(1, 5, 1));
			Particles.SetX(5, FVec3(1, 1, -5));

			TArray<TVec3<int32>> Indices;
			Indices.Emplace(0, 1, 2);
			Indices.Emplace(3, 4, 5);
			FTriangleMeshImplicitObject Tri(MoveTemp(Particles), MoveTemp(Indices), MoveTemp(DummyMaterials));

			//simple into the triangle
			bool bHit = Tri.Raycast(FVec3(3, 2, 2), FVec3(0, 0, -1), 2, 0, Time, Position, Normal, FaceIndex);
			EXPECT_TRUE(bHit);
			EXPECT_EQ(Tri.FindMostOpposingFace(Position, FVec3(0, 0, -1), FaceIndex, 0.01), 0);
			EXPECT_EQ(Tri.GetFaceNormal(0).X, Normal.X);
			EXPECT_EQ(Tri.GetFaceNormal(0).Y, Normal.Y);
			EXPECT_EQ(Tri.GetFaceNormal(0).Z, Normal.Z);

			//simple into second triangle
			bHit = Tri.Raycast(FVec3(0, 2, 0), FVec3(1, 0, 0), 2, 0, Time, Position, Normal, FaceIndex);
			EXPECT_TRUE(bHit);
			EXPECT_EQ(FaceIndex, 1);
			EXPECT_EQ(Tri.FindMostOpposingFace(Position, FVec3(1, 0, 0), FaceIndex, 0.01), 1);
			const FVec3 FaceNormal = Tri.GetFaceNormal(1);
			EXPECT_EQ(FaceNormal.X, Normal.X);
			EXPECT_EQ(FaceNormal.Y, Normal.Y);
			EXPECT_EQ(FaceNormal.Z, Normal.Z);

			//very close to edge, for now just return face hit regardless of direction because that's the implementation we currently rely on.
			//todo: inconsistent with hulls, should make them the same, but may have significant impact on existing content

			bHit = Tri.Raycast(FVec3(0, 2, 0.9), FVec3(1, 0, 0), 2, 0, Time, Position, Normal, FaceIndex);
			EXPECT_TRUE(bHit);
			EXPECT_EQ(FaceIndex, 1);
			EXPECT_EQ(Tri.FindMostOpposingFace(Position, FVec3(1, 0, 0), FaceIndex, 0.01), 1);
			EXPECT_EQ(Tri.FindMostOpposingFace(Position, FVec3(0, 0, -1), FaceIndex, 0.01), 1);	//ignores direction completely as per current implementation
		}

		// Non-uniform scale (Actual bug regression test)
		{
			// 4 triangles
			FTriangleMeshImplicitObject::ParticlesType Particles;
			Particles.AddParticles(12);
			// in z-y plane
			Particles.SetX(0, FVec3(0, 0, 0));
			Particles.SetX(1, FVec3(0, 1, 0));
			Particles.SetX(2, FVec3(0, 0, 1));

			// In x-z plane
			Particles.SetX(3, FVec3(0, 0, 0));
			Particles.SetX(4, FVec3(1, 0, 0));
			Particles.SetX(5, FVec3(0, 0, 1));

			// In x-y plane
			Particles.SetX(6, FVec3(0, 0, 0));
			Particles.SetX(7, FVec3(1, 0, 0));
			Particles.SetX(8, FVec3(0, 1, 0));

			// One 45 degree slanted triangle
			Particles.SetX(9, FVec3(1, 0, 0));
			Particles.SetX(10,  FVec3(1, 1, 0));
			Particles.SetX(11,  FVec3(0, 0, 1));

			TArray<TVec3<int32>> Indices;
			Indices.Emplace(0, 1, 2);
			Indices.Emplace(3, 4, 5);
			Indices.Emplace(6, 7, 8);
			Indices.Emplace(9, 10, 11);
			TArray<uint16> DummyMaterials;
			FImplicitObjectPtr Tri( new FTriangleMeshImplicitObject(MoveTemp(Particles), MoveTemp(Indices), MoveTemp(DummyMaterials)));

			// Using typical non uniform scale values
			//const FVec3 Scale(10, 1, 0.1);
			const FVec3 Scale(10, 1, 0.1);
			TImplicitObjectScaledGeneric<FReal, 3> ScaledTri(Tri, Scale);

			FVec3 OpposeSlantedFace = (FVec3(-1, 0, -1)/Scale).GetSafeNormal(); // Note we are transforming a normal here.
			EXPECT_NEAR(FVec3::DotProduct(OpposeSlantedFace, FVec3(Scale.Z, 0, Scale.X).GetSafeNormal()), -1.0f, KINDA_SMALL_NUMBER); // Check if OpposeSlantedFace opposes the slanted plane

			// We should hit the slanted face, since we are directly opposing it
			EXPECT_EQ(ScaledTri.FindMostOpposingFace(FVec3(0, 0, 0), OpposeSlantedFace, INDEX_NONE, 100.0), 3);

			// Now modify the test vector just a bit 
			FVec3 OpposeSlantedFaceNotExactlyOrthogonal = FVec3(OpposeSlantedFace.X*3.0f, OpposeSlantedFace.Y, OpposeSlantedFace.Z).GetSafeNormal();

			// We should not get the triangle in the y-z plane (should still be the slanted plane)
			// Check that the slanted triangle (3) is more apposing than the y-z triangle (0)
			EXPECT_GT(FVec3::DotProduct(OpposeSlantedFaceNotExactlyOrthogonal, FVec3(-1, 0, 0)), FVec3::DotProduct(OpposeSlantedFaceNotExactlyOrthogonal, -OpposeSlantedFace));

			EXPECT_NE(ScaledTri.FindMostOpposingFace(FVec3(0, 0, 0), OpposeSlantedFaceNotExactlyOrthogonal, INDEX_NONE, 100.0), 0);

			// Now check mirroring cases:
			FVec3 MirrorScale(1, -1, 1);
			TImplicitObjectScaledGeneric<FReal, 3> MirTri(Tri, MirrorScale);
			EXPECT_EQ(MirTri.FindMostOpposingFace(FVec3(0, 0, 0), FVec3(0,0,-1), INDEX_NONE, 100.0), 2);

			// Note: Switching to FindMostOpposingFaceScaled version instead of using wrapper class
			MirrorScale = FVec3(-1, 1, 1);
			EXPECT_EQ(Tri->FindMostOpposingFaceScaled(FVec3(0, 0, 0), FVec3(0, 0, -1), INDEX_NONE, 100.0, MirrorScale), 2);

			MirrorScale = FVec3(1, 1, -1);
			EXPECT_EQ(Tri->FindMostOpposingFaceScaled(FVec3(0, 0, 0), FVec3(0, 0, 1), INDEX_NONE, 100.0, MirrorScale), 2);

			MirrorScale = FVec3(-1, -1, -1);
			EXPECT_EQ(Tri->FindMostOpposingFaceScaled(FVec3(0, 0, 0), FVec3(0, 0, 1), INDEX_NONE, 100.0, MirrorScale), 2);

			MirrorScale = FVec3(-1, -1, 1);
			EXPECT_EQ(Tri->FindMostOpposingFaceScaled(FVec3(0, 0, 0), FVec3(0, 0, -1), INDEX_NONE, 100.0, MirrorScale), 2);		

		}

		// Simple Non-uniform scale (Actual bug regression test)
		{
			// 2 triangles
			FTriangleMeshImplicitObject::ParticlesType Particles;
			Particles.AddParticles(6);
			// in z-y plane
			Particles.SetX(0, FVec3(0, 0, 0));
			Particles.SetX(1, FVec3(0, 100, 0));
			Particles.SetX(2, FVec3(0, 0, 100));

			// In x-z plane
			Particles.SetX(3, FVec3(0, 0, 0));
			Particles.SetX(4, FVec3(100, 0, 0));
			Particles.SetX(5, FVec3(0, 0, 100));

			TArray<TVec3<int32>> Indices;
			Indices.Emplace(0, 1, 2);
			Indices.Emplace(3, 4, 5);
			TArray<uint16> DummyMaterials;
			FImplicitObjectPtr Tri( new FTriangleMeshImplicitObject(MoveTemp(Particles), MoveTemp(Indices), MoveTemp(DummyMaterials)));

			const FVec3 Scale(10, 1, 1);
			TImplicitObjectScaledGeneric<FReal, 3> ScaledTri(Tri, Scale);

			// Make sure we find the correct face when at 200 units away from origin on x-axes. The scaling will ensure that the point is on the face. using 1 unit search distance
			EXPECT_EQ(ScaledTri.FindMostOpposingFace(FVec3(200, 0, 0), FVec3(0,1,0), INDEX_NONE, 1.0f), 1);
			// Change direction and increase search distance, to hit other face
			EXPECT_EQ(ScaledTri.FindMostOpposingFace(FVec3(200, 0, 0), FVec3(-1, 0, 0), INDEX_NONE, 201.0f), 0);
			// Reduce search distance so that we don't hit the best face
			EXPECT_EQ(ScaledTri.FindMostOpposingFace(FVec3(200, 0, 0), FVec3(-1, 0, 0), INDEX_NONE, 199.0f), 1);
		}
	}


	void ConvexMostOpposing()
	{
		FReal Time;
		FVec3 Position;
		FVec3 Normal;
		int32 FaceIndex;

		TArray<FConvex::FVec3Type> Particles;
		Particles.SetNum(6);
		Particles[0] = { 1, 1, 1 };
		Particles[1] = { 5, 1, 1 };
		Particles[2] = { 1, 5, 1 };

		Particles[3] = { 1, 1, 1 };
		Particles[4] = { 1, 5, 1 };
		Particles[5] = { 1, 1, -5};

		FConvex Convex(MoveTemp(Particles), 0.0f);

		//simple into the triangle
		bool bHit = Convex.Raycast(FVec3(3, 2, 2), FVec3(0, 0, -1), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_FLOAT_EQ(Position.X, 3);
		EXPECT_FLOAT_EQ(Position.Y, 2);
		EXPECT_FLOAT_EQ(Position.Z, 1);
		EXPECT_EQ(FaceIndex, INDEX_NONE);	//convex should not compute its own face index as this is too expensive
		EXPECT_EQ(Convex.FindMostOpposingFace(Position, FVec3(0, 0, -1), FaceIndex, 0.01 + SMALL_NUMBER), 0);	//front face, just so happens that convex hull generates the planes in this order

		//simple into second triangle
		bHit = Convex.Raycast(FVec3(0, 2, 0), FVec3(1, 0, 0), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 2);
		EXPECT_FLOAT_EQ(Position.Z, 0);
		EXPECT_EQ(FaceIndex, INDEX_NONE);	//convex should not compute its own face index as this is too expensive
		EXPECT_EQ(Convex.FindMostOpposingFace(Position, FVec3(1, 0, 0), FaceIndex, 0.01), 3);	//side face, just so happens that convex hull generates the planes in this order

		bHit = Convex.Raycast(FVec3(0, 2, 0.99), FVec3(1, 0, 0), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(FaceIndex, INDEX_NONE);
		EXPECT_EQ(Convex.FindMostOpposingFace(Position, FVec3(1, 0, 0), FaceIndex, 0.01 + SMALL_NUMBER), 3);
		EXPECT_EQ(Convex.FindMostOpposingFace(Position, FVec3(0, 0, -1), FaceIndex, 0.01 + SMALL_NUMBER), 0);

		//again but far enough away from edge
		bHit = Convex.Raycast(FVec3(0, 2, 0.9), FVec3(1, 0, 0), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(FaceIndex, INDEX_NONE);
		EXPECT_EQ(Convex.FindMostOpposingFace(Position, FVec3(1, 0, 0), FaceIndex, 0.01 + SMALL_NUMBER), 3);
		EXPECT_EQ(Convex.FindMostOpposingFace(Position, FVec3(0, 0, -1), FaceIndex, 0.01 + SMALL_NUMBER), 3);	//too far to care about other face
	}


	void ScaledMostOpposing()
	{
		FReal Time;
		FVec3 Position;
		FVec3 Normal;
		int32 FaceIndex;

		TArray<FConvex::FVec3Type> Particles;
		Particles.SetNum(6);
		Particles[0] = { 0, -1, 1 };
		Particles[1] = { 1, -1, -1 };
		Particles[2] = { 0, 1, 1 };

		Particles[3] = { 0, -1, 1 };
		Particles[4] = { 0, 1, 1 };
		Particles[5] = { -1, -1, -1 };

		Chaos::FImplicitObjectPtr Convex(new FConvex(MoveTemp(Particles), 0.0f));

		//identity scale
		{
			TImplicitObjectScaledGeneric<FReal, 3> Scaled(Convex, FVec3(1, 1, 1));

			//simple into the triangle
			bool bHit = Scaled.Raycast(FVec3(0.5, 0, 2), FVec3(0, 0, -1), 3, 0, Time, Position, Normal, FaceIndex);
			EXPECT_TRUE(bHit);
			EXPECT_EQ(Position.X, 0.5);
			EXPECT_EQ(Position.Y, 0);
			EXPECT_EQ(Position.Z, 2 - Time);
			EXPECT_EQ(FaceIndex, INDEX_NONE);	//convex should not compute its own face index as this is too expensive
			EXPECT_EQ(Scaled.FindMostOpposingFace(Position, FVec3(0, 0, -1), FaceIndex, 0.01), 0);	//x+ face, just so happens that convex hull generates the planes in this order

			//simple into second triangle
			bHit = Scaled.Raycast(FVec3(-2, 0, 0.5), FVec3(1, 0, 0), 3, 0, Time, Position, Normal, FaceIndex);
			EXPECT_TRUE(bHit);
			EXPECT_EQ(Position.X, -2+Time);
			EXPECT_EQ(Position.Y, 0);
			EXPECT_EQ(Position.Z, 0.5);
			EXPECT_EQ(FaceIndex, INDEX_NONE);	//convex should not compute its own face index as this is too expensive
			EXPECT_EQ(Scaled.FindMostOpposingFace(Position, FVec3(1, 0, 0), FaceIndex, 0.01), 1);	//x- face, just so happens that convex hull generates the planes in this order

			bHit = Scaled.Raycast(FVec3(-0.001, 0, 2), FVec3(0, 0, -1), 3, 0, Time, Position, Normal, FaceIndex);
			EXPECT_TRUE(bHit);
			EXPECT_EQ(FaceIndex, INDEX_NONE);
			EXPECT_EQ(Scaled.FindMostOpposingFace(Position, FVec3(1, 0, 0), FaceIndex, 0.01), 1);
			EXPECT_EQ(Scaled.FindMostOpposingFace(Position, FVec3(-1, 0, 0), FaceIndex, 0.01), 0);

			//again but far enough away from edge
			bHit = Scaled.Raycast(FVec3(-0.1, 0, 2), FVec3(0, 0, -1), 3, 0, Time, Position, Normal, FaceIndex);
			EXPECT_TRUE(bHit);
			EXPECT_EQ(FaceIndex, INDEX_NONE);
			EXPECT_EQ(Scaled.FindMostOpposingFace(Position, FVec3(1, 0, 0), FaceIndex, 0.01), 1);
			EXPECT_EQ(Scaled.FindMostOpposingFace(Position, FVec3(-1, 0, 0), FaceIndex, 0.01), 1);	//too far to care about other face
		}

		//non-uniform scale
		{
			TImplicitObjectScaledGeneric<FReal, 3> Scaled(Convex, FVec3(2, 1, 1));

			//simple into the triangle
			bool bHit = Scaled.Raycast(FVec3(0.5, 0, 2), FVec3(0, 0, -1), 3, 0, Time, Position, Normal, FaceIndex);
			EXPECT_TRUE(bHit);
			EXPECT_EQ(Position.X, 0.5);
			EXPECT_EQ(Position.Y, 0);
			EXPECT_EQ(Position.Z, 2 - Time);
			EXPECT_EQ(FaceIndex, INDEX_NONE);	//convex should not compute its own face index as this is too expensive
			EXPECT_EQ(Scaled.FindMostOpposingFace(Position, FVec3(0, 0, -1), FaceIndex, 0.01), 0);	//x+ face, just so happens that convex hull generates the planes in this order

			//simple into second triangle
			bHit = Scaled.Raycast(FVec3(-2, 0, 0.5), FVec3(1, 0, 0), 3, 0, Time, Position, Normal, FaceIndex);
			EXPECT_TRUE(bHit);
			EXPECT_EQ(Position.X, -2 + Time);
			EXPECT_EQ(Position.Y, 0);
			EXPECT_EQ(Position.Z, 0.5);
			EXPECT_EQ(FaceIndex, INDEX_NONE);	//convex should not compute its own face index as this is too expensive
			EXPECT_EQ(Scaled.FindMostOpposingFace(Position, FVec3(1, 0, 0), FaceIndex, 0.01), 1);	//x- face, just so happens that convex hull generates the planes in this order

			bHit = Scaled.Raycast(FVec3(-0.001, 0, 2), FVec3(0, 0, -1), 3, 0, Time, Position, Normal, FaceIndex);
			EXPECT_TRUE(bHit);
			EXPECT_EQ(FaceIndex, INDEX_NONE);
			EXPECT_EQ(Scaled.FindMostOpposingFace(Position, FVec3(1, 0, 0), FaceIndex, 0.01), 1);
			EXPECT_EQ(Scaled.FindMostOpposingFace(Position, FVec3(-1, 0, 0), FaceIndex, 0.01), 0);

			//again but far enough away from edge
			bHit = Scaled.Raycast(FVec3(-0.1, 0, 2), FVec3(0, 0, -1), 3, 0, Time, Position, Normal, FaceIndex);
			EXPECT_TRUE(bHit);
			EXPECT_EQ(FaceIndex, INDEX_NONE);
			EXPECT_EQ(Scaled.FindMostOpposingFace(Position, FVec3(1, 0, 0), FaceIndex, 0.01), 1);
			EXPECT_EQ(Scaled.FindMostOpposingFace(Position, FVec3(-1, 0, 0), FaceIndex, 0.01), 1);	//too far to care about other face
		}

		// Reflection
		{
			// Reflecting in x-z plane should return the same face as not reflecting (in this particular case)
			const FVec3 ReflectionScale = FVec3(1, -1, 1);
			const int32 ReflectionFace = Convex->FindMostOpposingFaceScaled(FVec3(0, 0, 0), FVec3(-2, 0, -1).GetSafeNormal(), INDEX_NONE, 1000.0f, ReflectionScale);
			const int32 NoReflectionFace = Convex->FindMostOpposingFace(FVec3(0, 0, 0), FVec3(-2, 0, -1).GetSafeNormal(), INDEX_NONE, 1000.0f);
			EXPECT_EQ(NoReflectionFace, ReflectionFace);
		}
	}


	// Unscaled test to accompany TestConvexBox_Scaled. See comments on that function
	// This test has always passed whereas TestConvexBox_Scaled used to fail.
	GTEST_TEST(FindMostOpposingFaceTests, TestConvexBox_Unsccaled)
	{
		const FVec3 BoxScale = FVec3(10, 5, 1);
		const FVec3 BoxSize = FVec3(100, 100, 100);
		const FReal Margin = 0;
		FImplicitConvex3 Box = CreateConvexBox(BoxSize * BoxScale, Margin);

		// Position is not important in this test as all faces are within TestDistance of TestPosition
		const FVec3 TestPos = FVec3(0, 0, 0);
		const FReal TestDistance = 10000.0f;

		// Pointing mostly left to select the right-facing face. If we transform this normal into unscaled space and
		// call FindMostOpposingFace on the core convex, it would incorrectly select the up-facing face.
		const FVec3 TestDir = FVec3(-1.0, 0, -0.5).GetSafeNormal();
		int32 FaceIndex = Box.FindMostOpposingFace(TestPos, TestDir, INDEX_NONE, TestDistance);
		const FVec3 FaceNormal = Box.GetPlane(FaceIndex).Normal();

		EXPECT_NEAR(FaceNormal.X, 1.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(FaceNormal.Z, 0.0f, KINDA_SMALL_NUMBER);
	}

	// FindMostOpposingFace on Scaled Convex Box
	// NOTE: Box faces normals are not affected by scale though the face positions are
	// This tests for a specific bug: TImplicitObjectScaled::FindMostOpposingFace was calling
	// onto FConvex::FindMostOpposingFace with a corrected normal, but this does not work - we
	// actually need to perform the most-opposing test in scaled space.
	GTEST_TEST(FindMostOpposingFaceTests, TestConvexBox_Scaled)
	{
		const FVec3 BoxScale = FVec3(10, 5, 1);
		const FVec3 BoxSize = FVec3(100, 100, 100);
		const FReal Margin = 0;
		TImplicitObjectScaled<FImplicitConvex3> Box = CreateScaledConvexBox(BoxSize, BoxScale, Margin);

		// Position is not important in this test as all faces are within TestDistance of TestPosition
		const FVec3 TestPos = FVec3(0, 0, 0);
		const FReal TestDistance = 10000.0f;

		// Pointing mostly left to select the right-facing face. If we transform this normal into unscaled space and
		// call FindMostOpposingFace on the core convex, it would incorrectly select the up-facing face.
		const FVec3 TestDir = FVec3(-1.0, 0, -0.5).GetSafeNormal();
		int32 FaceIndex = Box.FindMostOpposingFace(TestPos, TestDir, INDEX_NONE, TestDistance);
		const FVec3 FaceNormal = Box.GetPlane(FaceIndex).Normal();

		EXPECT_NEAR(FaceNormal.X, 1.0f, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(FaceNormal.Z, 0.0f, KINDA_SMALL_NUMBER);
	}
}