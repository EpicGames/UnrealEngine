// Copyright Epic Games, Inc. All Rights Reserved.
#include "HeadlessChaos.h"
#include "HeadlessChaosTestEvolution.h"

namespace ChaosTest
{
	using namespace Chaos;

	GTEST_TEST(MoveTests, TestMoveNoSweep)
	{
		const int32 NumTicks = 100;
		const FReal Dt = 1.0/30.0;
		const FReal DistanceTolerance = 0.1f;

		const FVec3 FloorSize = FVec3(10000, 10000, 100);
		const FVec3 BoxPos = FVec3(0,0,200);
		const FRotation3 BoxRot = FRotation3::FromIdentity();
		const FVec3 BoxSize = FVec3(100);
		const FReal BoxMass = 100;

		FEvolutionTest Test;

		FGeometryParticleHandle* Floor = Test.AddParticleFloor(FloorSize);
		FPBDRigidParticleHandle* Box = Test.AddParticleBox(BoxPos, BoxRot, BoxSize, BoxMass);

		// Drop the box on the ground and wait until it sleeps
		Test.TickUntilSleep();
		EXPECT_TRUE(Box->IsSleeping());
		EXPECT_NEAR(Box->GetX().Z, 0.5 * BoxSize.Z, DistanceTolerance);

		// Move the box horizontally and up into the air
		const FVec3 NewBoxPos = Box->GetX() + FVec3(100, 0, 100);
		Test.GetEvolution().SetParticleTransformSwept(Box, NewBoxPos, BoxRot, false);

		// The box should be at its new location and awake
		EXPECT_FALSE(Box->IsSleeping());
		EXPECT_NEAR(Box->GetX().X, NewBoxPos.X, DistanceTolerance);
		EXPECT_NEAR(Box->GetX().Y, NewBoxPos.Y, DistanceTolerance);
		EXPECT_NEAR(Box->GetX().Z, NewBoxPos.Z, DistanceTolerance);

		// The box should fall to the ground again
		Test.TickUntilSleep();
		EXPECT_TRUE(Box->IsSleeping());
		EXPECT_NEAR(Box->GetX().X, NewBoxPos.X, DistanceTolerance);
		EXPECT_NEAR(Box->GetX().Z, 0.5 * BoxSize.Z, DistanceTolerance);
	}

	GTEST_TEST(MoveTests, TestMoveWithSweepNoHit)
	{
		const int32 NumTicks = 100;
		const FReal Dt = 1.0 / 30.0;
		const FReal DistanceTolerance = 0.1f;

		const FVec3 FloorSize = FVec3(10000, 10000, 100);
		const FVec3 BoxPos = FVec3(0, 0, 200);
		const FRotation3 BoxRot = FRotation3::FromIdentity();
		const FVec3 BoxSize = FVec3(100);
		const FReal BoxMass = 100;

		FEvolutionTest Test;

		FGeometryParticleHandle* Floor = Test.AddParticleFloor(FloorSize);
		FPBDRigidParticleHandle* Box = Test.AddParticleBox(BoxPos, BoxRot, BoxSize, BoxMass);

		// Drop the box on the ground and wait until it sleeps
		Test.TickUntilSleep();
		EXPECT_TRUE(Box->IsSleeping());
		EXPECT_NEAR(Box->GetX().Z, 0.5 * BoxSize.Z, DistanceTolerance);

		// Move the box horizontally and up into the air
		const FVec3 NewBoxPos = Box->GetX() + FVec3(100, 0, 100);
		Test.GetEvolution().SetParticleTransformSwept(Box, NewBoxPos, BoxRot, false);

		// The box should be at its new location and awake
		EXPECT_FALSE(Box->IsSleeping());
		EXPECT_NEAR(Box->GetX().X, NewBoxPos.X, DistanceTolerance);
		EXPECT_NEAR(Box->GetX().Y, NewBoxPos.Y, DistanceTolerance);
		EXPECT_NEAR(Box->GetX().Z, NewBoxPos.Z, DistanceTolerance);

		// The box should fall to the ground again
		Test.TickUntilSleep();
		EXPECT_TRUE(Box->IsSleeping());
		EXPECT_NEAR(Box->GetX().X, NewBoxPos.X, DistanceTolerance);
		EXPECT_NEAR(Box->GetX().Z, 0.5 * BoxSize.Z, DistanceTolerance);
	}

	GTEST_TEST(MoveTests, TestMoveWithSweepBlockingHit)
	{
		const int32 NumTicks = 100;
		const FReal Dt = 1.0 / 30.0;
		const FReal DistanceTolerance = 0.1f;

		const FVec3 FloorSize = FVec3(10000, 10000, 100);
		const FVec3 BoxPos = FVec3(0, 0, 200);
		const FRotation3 BoxRot = FRotation3::FromIdentity();
		const FVec3 BoxSize = FVec3(100);
		const FReal BoxMass = 100;

		const FVec3 BoxTargetPos = FVec3(100, 0, 100 + 0.5 * BoxSize.Z);

		FEvolutionTest Test;

		FGeometryParticleHandle* Floor = Test.AddParticleFloor(FloorSize);
		FPBDRigidParticleHandle* Box = Test.AddParticleBox(BoxPos, BoxRot, BoxSize, BoxMass);
		FPBDRigidParticleHandle* BlockingBox = Test.AddParticleBox(BoxTargetPos + 0.5 * BoxSize, BoxRot, BoxSize, 0);

		// Drop the box on the ground and wait until it sleeps
		Test.TickUntilSleep();
		EXPECT_TRUE(Box->IsSleeping());
		EXPECT_NEAR(Box->GetX().Z, 0.5 * BoxSize.Z, DistanceTolerance);

		// Move the box horizontally and up into the air
		Test.GetEvolution().SetParticleTransformSwept(Box, BoxTargetPos, BoxRot, false);

		// The box should have been stopped on its way to its new location
		EXPECT_FALSE(Box->IsSleeping());
		EXPECT_NEAR(Box->GetX().X, 50, DistanceTolerance);
		EXPECT_NEAR(Box->GetX().Y, 0, DistanceTolerance);
		EXPECT_NEAR(Box->GetX().Z, 50 + 0.5 * BoxSize.Z, DistanceTolerance);

		// The box should fall to the ground again
		Test.TickUntilSleep();
		EXPECT_TRUE(Box->IsSleeping());
		EXPECT_NEAR(Box->GetX().X, 50, DistanceTolerance);
		EXPECT_NEAR(Box->GetX().Z, 0.5 * BoxSize.Z, DistanceTolerance);
	}

}