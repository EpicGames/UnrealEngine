// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Core.h"
#include "Chaos/Matrix.h"
#include "Chaos/Utilities.h"
#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

namespace ChaosTest {

	using namespace Chaos;

#define EXPECT_TRANSFORM_EQ(A, B) EXPECT_TRUE(A.Equals(B)) << "Result: \n" << *A.ToHumanReadableString() << "\nExpected: \n" << *B.ToHumanReadableString();

	/* Reference implementation of Multiply, copied here for base comparison when Multiply is optimized. */
	inline FMatrix33 MultiplyReference(const FMatrix33& LIn, const FMatrix33& RIn)
	{
		FMatrix33 L = LIn.GetTransposed();
		FMatrix33 R = RIn.GetTransposed();

		// We want L.R (FMatrix operator* actually calculates R.(L)T; i.e., Right is on the left, and the Left is transposed on the right.)
		// NOTE: PMatrix constructor takes values in column order
		return FMatrix33(
			L.M[0][0] * R.M[0][0] + L.M[0][1] * R.M[1][0] + L.M[0][2] * R.M[2][0],	// x00
			L.M[1][0] * R.M[0][0] + L.M[1][1] * R.M[1][0] + L.M[1][2] * R.M[2][0],	// x10
			L.M[2][0] * R.M[0][0] + L.M[2][1] * R.M[1][0] + L.M[2][2] * R.M[2][0],	// x20

			L.M[0][0] * R.M[0][1] + L.M[0][1] * R.M[1][1] + L.M[0][2] * R.M[2][1],	// x01
			L.M[1][0] * R.M[0][1] + L.M[1][1] * R.M[1][1] + L.M[1][2] * R.M[2][1],	// x11
			L.M[2][0] * R.M[0][1] + L.M[2][1] * R.M[1][1] + L.M[2][2] * R.M[2][1],	// x21

			L.M[0][0] * R.M[0][2] + L.M[0][1] * R.M[1][2] + L.M[0][2] * R.M[2][2],	// x02
			L.M[1][0] * R.M[0][2] + L.M[1][1] * R.M[1][2] + L.M[1][2] * R.M[2][2],	// x12
			L.M[2][0] * R.M[0][2] + L.M[2][1] * R.M[1][2] + L.M[2][2] * R.M[2][2]	// x22
		).GetTransposed();
	}

	/* Generate a matrix string in column order. */
	FString MatrixToString(const FMatrix33& M)
	{
		FString Output("");
		for (int32 i = 0; i < 3; ++i) {
			for (int32 j = 0; j < 3; ++j) {
				Output += FString::Printf(TEXT("%f "), M.M[i][j]);
			}
		}
		return Output;
	}

	void CheckMatrix(const FMatrix33& A, const FMatrix33& B, const FReal Tolerance = SMALL_NUMBER)
	{
		for (int II = 0; II < 3; ++II)
		{
			for (int JJ = 0; JJ < 3; ++JJ)
			{
				EXPECT_NEAR(A.M[II][JJ], B.M[II][JJ], Tolerance);
			}
		}
	}

	GTEST_TEST(MatrixTests, Multiply)
	{
		const FMatrix33 Test(7.f, 0.f, -3.f, 2.f, 3.f, 4.f, 1.f, -1.f, -2.f);
		const FMatrix33 TestInverse(-2.f, 3.f, 9.f, 8.f, -11.f, -34.f, -5.f, 7.f, 21.f);

		const FMatrix33 Identity(1.0f, 1.0f, 1.0f);
		const FMatrix33 ScaleDouble(Identity * 2);
		const FMatrix33 ScaleHalf(Identity * 0.5f);
		const FMatrix33 Zero(0);
		const FMatrix33 Rotate(0.f, -1.f, 0.f, 1.f, 0.f, 0.f, 0.f, 0.f, 1.f);

		// Basic integrity tests of Multiply(3x3 Matrix, 3x3 Matrix)
		EXPECT_TRUE(Chaos::Utilities::Multiply(Identity, Test).Equals(Test));
		EXPECT_TRUE(Chaos::Utilities::Multiply(Test, Identity).Equals(Test));
		EXPECT_TRUE(Chaos::Utilities::Multiply(ScaleDouble, Test).Equals(Test * 2));
		EXPECT_TRUE(Chaos::Utilities::Multiply(Test, ScaleDouble).Equals(Test * 2));
		EXPECT_TRUE(Chaos::Utilities::Multiply(ScaleHalf, Test).Equals(Test * 0.5f));
		EXPECT_TRUE(Chaos::Utilities::Multiply(Test, ScaleHalf).Equals(Test * 0.5f));
		EXPECT_TRUE(Chaos::Utilities::Multiply(Zero, Test).Equals(Zero));
		EXPECT_TRUE(Chaos::Utilities::Multiply(Test, Zero).Equals(Zero));
		EXPECT_TRUE(Chaos::Utilities::Multiply(TestInverse, Test).Equals(Identity));
		EXPECT_TRUE(Chaos::Utilities::Multiply(Test, TestInverse).Equals(Identity));

		// Verify that random matrices multiply correctly.
		for (int i = 0; i < 100; i++)
		{
			FMatrix33 A = RandomMatrix(-100, 100);
			FMatrix33 B = RandomMatrix(-100, 100);
			FMatrix33 Result = Chaos::Utilities::Multiply(A, B);
			FMatrix33 Expected = MultiplyReference(A, B);
			EXPECT_TRUE(Result.Equals(Expected)) << "Result:   " << *MatrixToString(Result) << "\nExpected: " << *MatrixToString(Expected);
		}

		const FVec3 TestVector(1.f, 2.f, 3.f);

		// Basic integrity tests of Multiply(3x3 Matrix, Vector3)
		EXPECT_TRUE(Chaos::Utilities::Multiply(Identity, TestVector).Equals(TestVector));
		EXPECT_TRUE(Chaos::Utilities::Multiply(ScaleDouble, TestVector).Equals(TestVector * 2));
		EXPECT_TRUE(Chaos::Utilities::Multiply(ScaleHalf, TestVector).Equals(TestVector * 0.5f));
		EXPECT_TRUE(Chaos::Utilities::Multiply(Zero, TestVector).Equals(TestVector * 0));
		EXPECT_TRUE(Chaos::Utilities::Multiply(Rotate, FVec3(1, 0, 0)).Equals(FVec3(0, 1, 0)));
		EXPECT_TRUE(Chaos::Utilities::Multiply(Rotate, FVec3(-1, 0, 0)).Equals(FVec3(0, -1, 0)));
		EXPECT_TRUE(Chaos::Utilities::Multiply(Rotate, FVec3(0, 1, 0)).Equals(FVec3(-1, 0, 0)));
		EXPECT_TRUE(Chaos::Utilities::Multiply(Rotate, FVec3(0, -1, 0)).Equals(FVec3(1, 0, 0)));

		// Verify combining very simple translations. 
		for (int i = 0; i < 3; ++i)
		{
			for (int j = i; j < 3; j++) {
				FVec3 Translate1(0);
				Translate1[i] = 1;
				FVec3 Translate2(0);
				Translate2[j] = 1;
				FVec3 ExpectedResult(0);
				ExpectedResult[i] += 1;
				ExpectedResult[j] += 1;

				FRigidTransform3 Multiplied = Chaos::Utilities::Multiply(
					FRigidTransform3(Translate1, FRotation3::FromIdentity()),
					FRigidTransform3(Translate2, FRotation3::FromIdentity())
				);
				FRigidTransform3 Expected(ExpectedResult, FRotation3::FromIdentity());

				EXPECT_TRUE(Multiplied.Equals(Expected)) << "Failed combining translations at i = " << i << ", j = " << j << "\nResult:   " << *Multiplied.ToHumanReadableString() << "\nExpected: " << *Expected.ToHumanReadableString();
			}
		}

		// Verify multiplying simple 90 degree rotations.
		FRigidTransform3 IdentityTransform(FVec3(0), FRotation3::FromIdentity());
		FRigidTransform3 RotationY(FVec3(0), FRotation3::FromRotatedVector(FVec3(1, 0, 0), FVec3(0, 0, -1)));
		FRigidTransform3 RotationZ(FVec3(0), FRotation3::FromRotatedVector(FVec3(1, 0, 0), FVec3(0, 1, 0)));
		FRigidTransform3 RotationX(FVec3(0), FRotation3::FromRotatedVector(FVec3(0, -1, 0), FVec3(0, 0, 1)));

		EXPECT_TRUE(Chaos::Utilities::Multiply(IdentityTransform, RotationY).Equals(RotationY));
		EXPECT_TRUE(Chaos::Utilities::Multiply(RotationY, IdentityTransform).Equals(RotationY));

		FRigidTransform3 ResultYZ(FVec3(0), FMatrix33(0, 0, 1, 1, 0, 0, 0, 1, 0));
		FRigidTransform3 ResultZY(FVec3(0), FMatrix33(0, -1, 0, 0, 0, 1, -1, 0, 0));
		FRigidTransform3 ResultXZ(FVec3(0), FMatrix33(0, -1, 0, 0, 0, 1, -1, 0, 0));
		FRigidTransform3 ResultZX(FVec3(0), FMatrix33(0, 0, -1, 1, 0, 0, 0, -1, 0));
		FRigidTransform3 ResultYX(FVec3(0), FMatrix33(0, -1, 0, 0, 0, 1, -1, 0, 0));
		FRigidTransform3 ResultXY(FVec3(0), FMatrix33(0, 0, 1, -1, 0, 0, 0, -1, 0));

		EXPECT_TRANSFORM_EQ(Chaos::Utilities::Multiply(RotationY, RotationZ), ResultYZ);
		EXPECT_TRANSFORM_EQ(Chaos::Utilities::Multiply(RotationZ, RotationY), ResultZY);
		EXPECT_TRANSFORM_EQ(Chaos::Utilities::Multiply(RotationX, RotationZ), ResultXZ);
		EXPECT_TRANSFORM_EQ(Chaos::Utilities::Multiply(RotationZ, RotationX), ResultZX);
		EXPECT_TRANSFORM_EQ(Chaos::Utilities::Multiply(RotationY, RotationX), ResultYX);
		EXPECT_TRANSFORM_EQ(Chaos::Utilities::Multiply(RotationX, RotationY), ResultXY);
	}

	GTEST_TEST(MatrixTests, MultiplyAB)
	{
		FMatrix33 A = RandomMatrix(10, 10);
		FMatrix33 B = RandomMatrix(10, 10);
		FMatrix33 C = MultiplyReference(A, B);
		FMatrix33 C2 = Utilities::MultiplyAB(A, B);

		CheckMatrix(C, C2);
	}

	GTEST_TEST(MatrixTests, MultiplyABt)
	{
		FMatrix33 A = RandomMatrix(10, 10);
		FMatrix33 B = RandomMatrix(10, 10);
		FMatrix33 C = MultiplyReference(A, B.GetTransposed());
		FMatrix33 C2 = Utilities::MultiplyABt(A, B);

		CheckMatrix(C, C2);
	}

	GTEST_TEST(MatrixTests, MultiplyAtB)
	{
		FMatrix33 A = RandomMatrix(10, 10);
		FMatrix33 B = RandomMatrix(10, 10);
		FMatrix33 C = MultiplyReference(A.GetTransposed(), B);
		FMatrix33 C2 = Utilities::MultiplyAtB(A, B);

		CheckMatrix(C, C2);
	}
}