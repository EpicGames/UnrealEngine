// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/OBBVectorized.h"
#include "Chaos/TriangleMeshImplicitObject.h"

namespace ChaosTest
{
	using namespace Chaos;


	GTEST_TEST(ObbTest, IntersectAABB)
	{
		{
			// Original points to create OBB // TArray<FVec3f> PointsA{ FVec3f(0, 0, 0), FVec3f(5, 0, 0), FVec3f(0, 5, 0) };
			FRigidTransform3 Transform = FRigidTransform3(FVec3(1.25, 1.25, 0.0), FQuat(FVec3(0.0, 0.0, 1.0), -3.1415926 / 4.0));
			Private::FOBBVectorized ObbA(Transform, FVec3f(UE_INV_SQRT_2*5.0f, UE_INV_SQRT_2 * 5.0f /2.0f, 0.0f), FVec3f(1.0f, 1.0f, 1.0f));
			{
				TAABB<FRealSingle, 3> Bounds(FVec3f(-2.0f, 1.5f, 0.0f), FVec3f(-1.0f, 2.5f, 0.0f));
				EXPECT_TRUE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				TAABB<FRealSingle, 3> Bounds(FVec3f(-2.0f, 4.5f, 0.0f), FVec3f(-1.0f, 5.5f, 0.0f));
				EXPECT_FALSE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				TAABB<FRealSingle, 3> Bounds(FVec3f(2.0f, 2.0f, -10.0f), FVec3f(3.0f, 3.0f, 10.0f));
				EXPECT_TRUE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				TAABB<FRealSingle, 3> Bounds(FVec3f(-10.0f, -10.0f, -10.0f), FVec3f(10.0f, 10.0f, 10.0f));
				EXPECT_TRUE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				TAABB<FRealSingle, 3> Bounds(FVec3f(4.0f, -1.0f, 0.0f), FVec3f(6.0f, 1.0f, 0.0f));
				EXPECT_TRUE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				TAABB<FRealSingle, 3> Bounds(FVec3f(6.0f, -1.0f, 0.0f), FVec3f(7.0f, 1.0f, 0.0f));
				EXPECT_FALSE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				TAABB<FRealSingle, 3> Bounds(FVec3f(-1.0f, 6.0f, 0.0f), FVec3f(1.0f, 7.0f, 0.0f));
				EXPECT_FALSE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				TAABB<FRealSingle, 3> Bounds(FVec3f(4.0f, 4.0f, 0.0f), FVec3f(10.0f, 10.0f, 0.0f));
				EXPECT_FALSE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				TAABB<FRealSingle, 3> Bounds(FVec3f(-1.5f, -1.0f, 0.0f), FVec3f(-0.5f, -0.5f, 0.0f));
				EXPECT_FALSE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
		}
		{
			TAABB<FRealSingle, 3> Bounds(FVec3f(-10.0f, -10.0f, 0.0f), FVec3f(10.0f, 10.0f, 10.0f));
			{
				FRigidTransform3 Transform = FRigidTransform3(FVec3(0.0, 0.0, 12.0), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
				Private::FOBBVectorized ObbA(Transform.ToMatrixNoScale(), FVec3f(2.0f, 2.0f, 2.0f), FVec3f(1.0f, 1.0f, 1.0f));
				EXPECT_TRUE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				FRigidTransform3 Transform = FRigidTransform3(FVec3(0.0, 11.0, 0.0), FQuat::Identity);
				Private::FOBBVectorized ObbA(Transform.ToMatrixNoScale(), FVec3f(2.0f, 2.0f, 2.0f), FVec3f(1.0f, 1.0f, 1.0f));
				EXPECT_TRUE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				FRigidTransform3 Transform = FRigidTransform3(FVec3(0.0, 11.0, 0.0), FQuat::Identity);
				Private::FOBBVectorized ObbA(Transform.ToMatrixNoScale(), FVec3f(2.0f, 2.0f, 2.0f), FVec3f(1.0f, 3.0f, 1.0f));
				EXPECT_FALSE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				FRigidTransform3 Transform = FRigidTransform3(FVec3(0.0, 11.0, 0.0), FQuat::Identity);
				Private::FOBBVectorized ObbA(Transform.ToMatrixNoScale(), FVec3f(2.0f, 2.0f, 2.0f), FVec3f(1.0f, 1.0f, 3.0f));
				EXPECT_TRUE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				FRigidTransform3 Transform = FRigidTransform3(FVec3(0.0, 11.0, 0.0), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 2.0));
				Private::FOBBVectorized ObbA(Transform.ToMatrixNoScale(), FVec3f(2.0f, 2.0f, 2.0f), FVec3f(3.0f, 1.0f, 1.0f));
				EXPECT_TRUE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
			{
				FRigidTransform3 Transform = FRigidTransform3(FVec3(0.0, 11.0, 0.0), FQuat(FVec3(1.0, 0.0, 0.0), -3.1415926 / 2.0));
				Private::FOBBVectorized ObbA(Transform.ToMatrixNoScale(), FVec3f(2.0f, 2.0f, 2.0f), FVec3f(3.0f, 1.0f, 1.0f));
				EXPECT_TRUE(ObbA.IntersectAABB(FAABBVectorized(Bounds)));
			}
		}
	}

}