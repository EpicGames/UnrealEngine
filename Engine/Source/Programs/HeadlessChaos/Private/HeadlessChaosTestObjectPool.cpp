// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "Chaos/ObjectPool.h"

namespace ChaosTest
{
	using namespace Chaos;

	GTEST_TEST(ObjectPoolTests, TestPool_SimpleType)
	{
		// Tests Alloc/Free works with basic types

		using FPool = TObjectPool<int>;
		FPool Pool(3);

		// Fill fist page
		Pool.Alloc(1);
		Pool.Alloc(2);
		FPool::FPtr Ptr = Pool.Alloc(3);

		const int32 PoolSize = Pool.GetAllocatedSize();

		for(int32 i = 0; i < 32; ++i)
		{
			Pool.Free(Ptr);
			Ptr = Pool.Alloc(4);
		}

		// Make sure the freelist is working - should only ever have 3 elements allocated
		EXPECT_EQ(PoolSize, Pool.GetAllocatedSize());
		EXPECT_EQ(Pool.GetNumAllocatedBlocks(), 1);
	}

	GTEST_TEST(ObjectPoolTests, TestPool_CompoundType)
	{
		// Tests Alloc/Free works with class types

		struct FCompoundPodType
		{
			FCompoundPodType(int A_, float B_)
				: A(A_)
				, B(B_)
			{}

			int A = 1;
			float B = 2.0f;
		};

		using FPool = TObjectPool<FCompoundPodType>;
		FPool Pool(3);

		// Fill fist page
		Pool.Alloc(1, 1.0f);
		Pool.Alloc(2, 2.0f);
		FPool::FPtr Ptr = Pool.Alloc(3, 3.0f);

		const int32 PoolSize = Pool.GetAllocatedSize();

		for(int32 i = 0; i < 32; ++i)
		{
			Pool.Free(Ptr);
			Ptr = Pool.Alloc(4, 4.0f);
		}

		// Make sure the freelist is working - should only ever have 3 elements allocated
		EXPECT_EQ(PoolSize, Pool.GetAllocatedSize());
		EXPECT_EQ(Pool.GetNumAllocatedBlocks(), 1);
	}

	GTEST_TEST(ObjectPoolTests, TestPool_ComplexType)
	{
		// Tests Alloc/Free works with non-trivial types

		struct FRefIncrementOnConstructDestroy
		{
			FRefIncrementOnConstructDestroy(int32& InConstructRef, int32& InDestructRef)
				: ConstructRef(InConstructRef)
				, DestructRef(InDestructRef)
			{
				++ConstructRef;
			}

			~FRefIncrementOnConstructDestroy()
			{
				++DestructRef;
			}

			int32& ConstructRef;
			int32& DestructRef;
		};

		int32 ConstructorCount = 0;
		int32 DestructorCount = 0;

		using FPool = TObjectPool<FRefIncrementOnConstructDestroy>;
		FPool Pool(3);

		// Fill fist page
		Pool.Alloc(ConstructorCount, DestructorCount);
		Pool.Alloc(ConstructorCount, DestructorCount);
		FPool::FPtr Ptr = Pool.Alloc(ConstructorCount, DestructorCount);

		const int32 PoolSize = Pool.GetAllocatedSize();

		for(int32 i = 0; i < 32; ++i)
		{
			Pool.Free(Ptr);
			Ptr = Pool.Alloc(ConstructorCount, DestructorCount);
		}

		// Make sure the freelist is working - should only ever have 3 elements allocated
		EXPECT_EQ(PoolSize, Pool.GetAllocatedSize());
		EXPECT_EQ(Pool.GetNumAllocatedBlocks(), 1);

		// Make sure when types require destructors they area actually called
		EXPECT_EQ(ConstructorCount, 35);
		EXPECT_EQ(DestructorCount, 32);
	}

	GTEST_TEST(ObjectPoolTests, TestPool_SimpleType_Reset)
	{
		// Tests calling Reset works correctly with trivially destructible types

		using FPool = TObjectPool<int>;
		FPool Pool(32);

		TArray<FPool::FPtr> PooledInts;

		for(int32 i = 0; i < 32; ++i)
		{
			Pool.Alloc(i);
		}

		const int32 PoolSize = Pool.GetAllocatedSize();

		Pool.Reset();

		for(int32 i = 0; i < 32; ++i)
		{
			Pool.Alloc(i);
		}

		// Make sure reset works, the allocated size should never change as we only need one block
		EXPECT_EQ(PoolSize, Pool.GetAllocatedSize());
		EXPECT_EQ(Pool.GetNumAllocatedBlocks(), 1);
	}

	GTEST_TEST(ObjectPoolTests, TestPool_ComplexType_Reset)
	{
		// Tests calling Reset on the pool with items that require destruction actually destructs the items

		struct FRefIncrementOnConstructDestroy
		{
			FRefIncrementOnConstructDestroy(int32& InConstructRef, int32& InDestructRef)
				: ConstructRef(InConstructRef)
				, DestructRef(InDestructRef)
			{
				++ConstructRef;
			}

			~FRefIncrementOnConstructDestroy()
			{
				++DestructRef;
			}

			int32& ConstructRef;
			int32& DestructRef;
		};

		int32 ConstructorCount = 0;
		int32 DestructorCount = 0;

		using FPool = TObjectPool<FRefIncrementOnConstructDestroy>;
		FPool Pool(3);

		TArray<FPool::FPtr> PooledInts;

		// Fill fist page
		TArray<FPool::FPtr> Ptrs;

		// Add 32 items
		for(int32 i = 0; i < 32; ++i)
		{
			Ptrs.Add(Pool.Alloc(ConstructorCount, DestructorCount));
		}

		// Free half of them
		for(int32 i = 0; i < 16; ++i)
		{
			Pool.Free(Ptrs[i * 2]);
		}

		// We should have constructed 32 items, destroyed 16
		EXPECT_EQ(ConstructorCount, 32);
		EXPECT_EQ(DestructorCount, 16);

		// This call should reset the pool, destroying the remaining 16 elements
		Pool.Reset();

		EXPECT_EQ(DestructorCount, 32);
	}

	GTEST_TEST(ObjectPoolTests, TestPool_Shrink)
	{
		// Tests calling ShrinkTo correctly removes blocks

		using FPool = TObjectPool<int>;
		FPool Pool(3);

		FPool::FPtr Objects[9];

		// Fill 3 pages
		for(int32 i = 0; i < 9; ++i)
		{
			Objects[i] = Pool.Alloc(i);
		}

		// Empty middle block
		Pool.Free(Objects[3]);
		Pool.Free(Objects[4]);
		Pool.Free(Objects[5]);

		// This should do nothing, we have an empty page so it should stay
		Pool.ShrinkTo(1);

		EXPECT_EQ(Pool.GetNumAllocatedBlocks(), 3);

		// Empty last block
		Pool.Free(Objects[6]);
		Pool.Free(Objects[7]);
		Pool.Free(Objects[8]);

		// Now this should free a block
		Pool.ShrinkTo(1);

		EXPECT_EQ(Pool.GetNumAllocatedBlocks(), 2);

		// And this should free another
		Pool.ShrinkTo(0);

		EXPECT_EQ(Pool.GetNumAllocatedBlocks(), 1);
	}

	GTEST_TEST(ObjectPoolTests, TestPool_ReserveBlocks)
	{
		// Tests calling ReserveBlocks reserves the correct number of blocks

		TObjectPool<int32> Pool(4);

		Pool.ReserveBlocks(10);

		// Make sure it has 10 blocks
		EXPECT_EQ(Pool.GetNumAllocatedBlocks(), 10);

		// Make sure the size matches up (Items will be T + int32 for freelist per block)
		EXPECT_EQ(Pool.GetAllocatedSize(), (sizeof(int32) * 2) * 10 * 4);
	}

	GTEST_TEST(ObjectPoolTests, TestPool_ReserveItems)
	{
		// Tests calling ReserveItems reserves the correct number of blocks

		TObjectPool<int32> Pool(4);

		Pool.ReserveItems(13);

		// Make sure it has 10 blocks
		EXPECT_EQ(Pool.GetNumAllocatedBlocks(), 4);

		// Make sure the size matches up (Items will be T + int32 for freelist per block, * 4 for 4 blocks)
		EXPECT_EQ(Pool.GetAllocatedSize(), (sizeof(int32) * 2) * 4 * 4);
	}

	GTEST_TEST(ObjectPoolTests, TestPool_LargeTypeLargePage_Trivial)
	{
		// Tests larger pools with trivial items

		struct FType
		{
			int32 StaticArrayOfInts[64];
		};

		TObjectPool<FType> Pool(128);

		for(int i = 0; i < 129; ++i)
		{
			Pool.Alloc();
		}

		Pool.Reset();

		EXPECT_EQ(Pool.GetNumAllocatedBlocks(), 2);
	}

	GTEST_TEST(ObjectPoolTests, TestPool_LargeTypeLargePage_NonTrivial)
	{
		//Tests larger objects and larger pools, with destructors

		struct FType
		{
			FType(int32& InCounter)
				: Counter(InCounter)
			{}

			~FType()
			{
				++Counter;
			}

			int32 StaticArrayOfInts[64];

			int32& Counter;
		};

		TObjectPool<FType> Pool(128);

		int32 DestructCount = 0;

		for(int i = 0; i < 129; ++i)
		{
			Pool.Alloc(DestructCount);
		}

		// 129 will push into second block
		EXPECT_EQ(Pool.GetNumAllocatedBlocks(), 2);

		Pool.Reset();

		// Make sure we only hit 129 destructors, and all items are free
		EXPECT_EQ(DestructCount, 129);
		EXPECT_EQ(Pool.GetNumFree(), Pool.GetCapacity());
	}

	GTEST_TEST(ObjectPoolTests, TestPool_Ratios)
	{
		// Tests a few ratios - designed to catch changes to the pool that make the efficiency of the pool worse.
		// Note: That might be fine - just trying to catch it happening accidentally here

		constexpr float Epsilon = 1e-3f;

		{
			TObjectPool<int32> Pool(1, 0);
			EXPECT_NEAR(Pool.GetRatio(), 0.5f, Epsilon);
		}

		{
			struct FLocal
			{
				int32 Ints[2];
			};

			TObjectPool<FLocal> Pool(1, 0);
			EXPECT_NEAR(Pool.GetRatio(), 0.66666f, Epsilon);
		}

		{
			struct FLocal
			{
				int32 Ints[64];
			};

			TObjectPool<FLocal> Pool(1, 0);
			EXPECT_NEAR(Pool.GetRatio(), 0.98461f, Epsilon);
		}
	}

	GTEST_TEST(ObjectPoolTests, TestPool_Shuffle)
	{
		TObjectPool<int32> Pool(10);
		int32* Ptrs[100];

		for(int32 i = 0; i < 100; ++i)
		{
			Ptrs[i] = Pool.Alloc(i);
		}

		// Back index internally should walk down to 7 then shuffle once pulling the block with a gap to the front.
		Pool.Free(Ptrs[65]);

		EXPECT_EQ(Pool.GetNumFree(), 1);

		int32* NewPtr = Pool.Alloc(1234);

		// Check we're now full and the pointer is stable
		EXPECT_EQ(Pool.GetNumFree(), 0);
		EXPECT_EQ(NewPtr, Ptrs[65]);
	}
}