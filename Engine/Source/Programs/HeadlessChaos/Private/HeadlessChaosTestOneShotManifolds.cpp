// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"

//PRAGMA_DISABLE_OPTIMIZATION


namespace Chaos
{
	namespace Collisions
	{
		// Forward declaration of functions that we need to test but is not part of the public interface
		void ConstructBoxBoxOneShotManifold(
			const Chaos::FImplicitBox3& Box1,
			const Chaos::FRigidTransform3& Box1Transform, //world
			const Chaos::FImplicitBox3& Box2,
			const Chaos::FRigidTransform3& Box2Transform, //world
			const Chaos::FReal Dt,
			Chaos::FPBDCollisionConstraint& Constraint);

		template <typename ConvexImplicitType1, typename ConvexImplicitType2>
		void ConstructConvexConvexOneShotManifold(
			const ConvexImplicitType1& Implicit1,
			const FRigidTransform3& Convex1Transform, //world
			const ConvexImplicitType2& Implicit2,
			const FRigidTransform3& Convex2Transform, //world
			const FReal Dt,
			FPBDCollisionConstraint& Constraint);
	}
}


namespace ChaosTest
{
	using namespace Chaos;

	

	TEST(OneShotManifoldTests, OneShotBoxBox)
	{
		FReal Dt = 1 / 30.0f;
		// Test 1 is a degenerate case where 2 boxes are on top of each other. Make sure that it does not crash
		{
			TBox<FReal, 3> Box1(FVec3(-100.0f, -100, -100.0f), FVec3(100.0f, 100.0f, 100.0f));
			TBox<FReal, 3> Box2(FVec3(-100.0f, -100, -100.0f), FVec3(100.0f, 100.0f, 100.0f));
			FRigidTransform3 Box1Transform = FRigidTransform3(FVec3(0.0f, 0.0f, 0.0f), FRotation3::FromElements(0.0f, 0.0f, 0.0f, 1.0f));
			FRigidTransform3 Box2Transform = FRigidTransform3(FVec3(0.0f, 0.0f, 0.0f), FRotation3::FromElements(0.0f, 0.0f, 0.0f, 1.0f));

			FPBDCollisionConstraint Constraint;
			Collisions::ConstructBoxBoxOneShotManifold(Box1, Box1Transform, Box2, Box2Transform, Dt, Constraint);

			// Result should give a negative phi on all contacts
			// Phi direction may be in a random face direction
			int ContactCount = Constraint.GetManifoldPoints().Num();
			for (int ConstraintIndex = 0; ConstraintIndex < ContactCount; ConstraintIndex++)
			{
				EXPECT_NEAR(-200.0f, Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.Phi, 0.01);
			}
		}

		// Test 2 Very simple case of one box on top of another (slightly separated)
		{
			TBox<FReal, 3> Box1(FVec3(-100.0f, -100, -100.0f), FVec3(100.0f, 100.0f, 100.0f));
			TBox<FReal, 3> Box2(FVec3(-100.0f, -100, -100.0f), FVec3(100.0f, 100.0f, 100.0f));
			FRigidTransform3 Box1Transform = FRigidTransform3(FVec3(0.0f, 0.0f, 210.0f), FRotation3::FromElements(0.0f, 0.0f, 0.0f, 1.0f));
			FRigidTransform3 Box2Transform = FRigidTransform3(FVec3(0.0f, 0.0f, 0.0f), FRotation3::FromElements(0.0f, 0.0f, 0.0f, 1.0f));

			FPBDCollisionConstraint Constraint;
			Collisions::ConstructBoxBoxOneShotManifold(Box1, Box1Transform, Box2, Box2Transform, Dt, Constraint);
			int ContactCount = Constraint.GetManifoldPoints().Num();
			EXPECT_EQ(ContactCount, 4);
			for (int ConstraintIndex = 0; ConstraintIndex < ContactCount; ConstraintIndex++)
			{
				EXPECT_NEAR(10.0f, Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.Phi, 0.01);
				EXPECT_NEAR(FMath::Abs(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[0].X), 100.0f, 0.01);
				EXPECT_NEAR(FMath::Abs(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[0].Y), 100.0f, 0.01);
				EXPECT_NEAR(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[0].Z, -100.0f, 0.01);
				EXPECT_NEAR(FMath::Abs(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[1].X), 100.0f, 0.01);
				EXPECT_NEAR(FMath::Abs(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[1].Y), 100.0f, 0.01);
				EXPECT_NEAR(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[1].Z, 100.0f, 0.01);
			}
		}

		// Test 2b Same as test 1b, but rotate box 2 a bit so that box1 is the reference cube
		{
			TBox<FReal, 3> Box1(FVec3(-100.0f, -100, -100.0f), FVec3(100.0f, 100.0f, 100.0f));
			TBox<FReal, 3> Box2(FVec3(-100.0f, -100, -100.0f), FVec3(100.0f, 100.0f, 100.0f));
			FRigidTransform3 Box1Transform = FRigidTransform3(FVec3(0.0f, 0.0f, 210.0f), FRotation3::FromElements(0.0f, 0.0f, 0.0f, 1.0f));
			FRigidTransform3 Box2Transform = FRigidTransform3(FVec3(0.0f, 0.0f, 0.0f), FRotation3::FromAxisAngle(FVec3(0.0f, 1.0f, 0.0f), 0.1f));

			FPBDCollisionConstraint Constraint;
			Collisions::ConstructBoxBoxOneShotManifold(Box1, Box1Transform, Box2, Box2Transform, Dt, Constraint);
			int ContactCount = Constraint.GetManifoldPoints().Num();
			EXPECT_EQ(ContactCount, 4);
			for (int ConstraintIndex = 0; ConstraintIndex < ContactCount; ConstraintIndex++)
			{
				EXPECT_NEAR(10.0f, Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.Phi, 15.0f); // The cube is at an angle now
			}
		}

		// Test 3 one box on top of another (slightly separated)
		// The box vertices are offset
		{
			FVec3 OffsetBox1(300.0f, 140.0f, -210.0f);
			FVec3 OffsetBox2(-300.0f, 20.0f, 30.0f);

			TBox<FReal, 3> Box1(FVec3(-100.0f, -100, -100.0f) + OffsetBox1, FVec3(100.0f, 100.0f, 100.0f) + OffsetBox1);
			TBox<FReal, 3> Box2(FVec3(-100.0f, -100, -100.0f) + OffsetBox2, FVec3(100.0f, 100.0f, 100.0f) + OffsetBox2);
			FRigidTransform3 Box1Transform = FRigidTransform3(FVec3(0.0f, 0.0f, 210.0f) - OffsetBox1, FRotation3::FromElements(0.0f, 0.0f, 0.0f, 1.0f));
			FRigidTransform3 Box2Transform = FRigidTransform3(FVec3(0.0f, 0.0f, 0.0f) - OffsetBox2, FRotation3::FromElements(0.0f, 0.0f, 0.0f, 1.0f));

			FPBDCollisionConstraint Constraint;
			Collisions::ConstructBoxBoxOneShotManifold(Box1, Box1Transform, Box2, Box2Transform, Dt, Constraint);
			int ContactCount = Constraint.GetManifoldPoints().Num();
			EXPECT_EQ(ContactCount, 4);
			for (int ConstraintIndex = 0; ConstraintIndex < ContactCount; ConstraintIndex++)
			{
				EXPECT_NEAR(10.0f, Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.Phi, 0.01);
				EXPECT_NEAR(FMath::Abs(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[0].X - OffsetBox1.X), 100.0f, 0.01);
				EXPECT_NEAR(FMath::Abs(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[0].Y - OffsetBox1.Y), 100.0f, 0.01);
				EXPECT_NEAR(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[0].Z - OffsetBox1.Z, -100.0f, 0.01);
				EXPECT_NEAR(FMath::Abs(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[1].X - OffsetBox2.X), 100.0f, 0.01);
				EXPECT_NEAR(FMath::Abs(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[1].Y - OffsetBox2.Y), 100.0f, 0.01);
				EXPECT_NEAR(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[1].Z - OffsetBox2.Z, 100.0f, 0.01);
			}
		}

		// Test 4 one box on top of another (slightly separated)
		// With transforms
		{
			// Vertex offsets
			FVec3 OffsetBox1(300.0f, 140.0f, -210.0f);
			FVec3 OffsetBox2(-300.0f, 20.0f, 30.0f);

			FVec3 Axis(1.0f, 1.0f, 1.0f) ;
			Axis.Normalize();
			ensure(Axis.IsNormalized());

			FRigidTransform3 RotationTransform(FVec3(0.0f, 0.0f, 0.0f), FRotation3::FromAxisAngle(Axis, PI/2));

			FRigidTransform3 TranslationTransform1(FVec3(-100.0f, 50.0f, 1000.0f + 210.0f) - OffsetBox1, FRotation3::FromAxisAngle(FVec3(1.0f, 0.0f, 0.0f), 0));
			FRigidTransform3 TranslationTransform2(FVec3(-100.0f, 50.0f, 1000.0f + 0.0) - OffsetBox2, FRotation3::FromAxisAngle(FVec3(0.0f, 1.0f, 0.0f), 0));

			TBox<FReal, 3> Box1(FVec3(-100.0f, -100, -100.0f) + OffsetBox1, FVec3(100.0f, 100.0f, 100.0f) + OffsetBox1);
			TBox<FReal, 3> Box2(FVec3(-100.0f, -100, -100.0f) + OffsetBox2, FVec3(100.0f, 100.0f, 100.0f) + OffsetBox2);

			FRigidTransform3 Box1Transform = TranslationTransform1 * RotationTransform;
			FRigidTransform3 Box2Transform = TranslationTransform2 * RotationTransform;

			FPBDCollisionConstraint Constraint;
			Collisions::ConstructBoxBoxOneShotManifold(Box1, Box1Transform, Box2, Box2Transform, Dt, Constraint);
			int ContactCount = Constraint.GetManifoldPoints().Num();
			EXPECT_EQ(ContactCount, 4);
			for (int ConstraintIndex = 0; ConstraintIndex < ContactCount; ConstraintIndex++)
			{
				EXPECT_NEAR(10.0f, Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.Phi, 0.01);
				EXPECT_NEAR(FMath::Abs(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[0].X - OffsetBox1.X), 100.0f, 0.01);
				EXPECT_NEAR(FMath::Abs(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[0].Y - OffsetBox1.Y), 100.0f, 0.01);
				EXPECT_NEAR(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[0].Z - OffsetBox1.Z, -100.0f, 0.01);
				EXPECT_NEAR(FMath::Abs(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[1].X - OffsetBox2.X), 100.0f, 0.01);
				EXPECT_NEAR(FMath::Abs(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[1].Y - OffsetBox2.Y), 100.0f, 0.01);
				EXPECT_NEAR(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[1].Z - OffsetBox2.Z, 100.0f, 0.01);
			}
		}

		// Test 5 one box on top of another (slightly separated)
		// With 90 degree box rotations
		{
			// Vertex offsets
			FVec3 OffsetBox1(0.0f, 210.0f, 0.0f); // Will rotate into z
			FVec3 OffsetBox2(0.0f, 0.0f, 0.0f);

			FVec3 Axis(1.0f, 1.0f, 1.0f);
			Axis.Normalize();
			ensure(Axis.IsNormalized());

			//FRigidTransform3 RotationTransform(FVec3(0.0f, 0.0f, 0.0f), FRotation3::FromAxisAngle(Axis, PI / 2));

			FRigidTransform3 Box1Transform(FVec3(0.0f, 0.0f, 0.0f /* 210.0f*/), FRotation3::FromAxisAngle(FVec3(1.0f, 0.0f, 0.0f), PI / 2));
			FRigidTransform3 Box2Transform(FVec3(0.0, 0.0f, 0.0f + 0.0), FRotation3::FromAxisAngle(FVec3(0.0f, 1.0f, 0.0f), PI / 2));

			TBox<FReal, 3> Box1(FVec3(-100.0f, -100, -100.0f) + OffsetBox1, FVec3(100.0f, 100.0f, 100.0f) + OffsetBox1);
			TBox<FReal, 3> Box2(FVec3(-100.0f, -100, -100.0f) + OffsetBox2, FVec3(100.0f, 100.0f, 100.0f) + OffsetBox2);

			FPBDCollisionConstraint Constraint;
			Collisions::ConstructBoxBoxOneShotManifold(Box1, Box1Transform, Box2, Box2Transform, Dt, Constraint);
			int ContactCount = Constraint.GetManifoldPoints().Num();
			for (int ConstraintIndex = 0; ConstraintIndex < ContactCount; ConstraintIndex++)
			{
				EXPECT_NEAR(10.0f, Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.Phi, 0.01);
				FVec3 Location1 = Box1Transform.TransformPosition(FVec3(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[0]));
				FVec3 Location2 = Box2Transform.TransformPosition(FVec3(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[1]));
				EXPECT_NEAR(Location1.Z, 110.0f, 0.01);
				EXPECT_NEAR(Location2.Z, 100.0f, 0.01);
			}
		}

		// Test 6 Rotate top box by 45 degrees
		{
			TBox<FReal, 3> Box1(FVec3(-100.0f, -100, -100.0f), FVec3(100.0f, 100.0f, 100.0f));
			TBox<FReal, 3> Box2(FVec3(-100.0f, -100, -100.0f), FVec3(100.0f, 100.0f, 100.0f));
			FRigidTransform3 Box1Transform = FRigidTransform3(FVec3(0.0f, 0.0f, 210.0f), FRotation3::FromAxisAngle(FVec3(0.0f, 0.0f, 1.0f), PI / 2));
			FRigidTransform3 Box2Transform = FRigidTransform3(FVec3(0.0f, 0.0f, 0.0f), FRotation3::FromElements(0.0f, 0.0f, 0.0f, 1.0f));

			FPBDCollisionConstraint Constraint;
			Collisions::ConstructBoxBoxOneShotManifold(Box1, Box1Transform, Box2, Box2Transform, Dt, Constraint);
			int ContactCount = Constraint.GetManifoldPoints().Num();
			EXPECT_EQ(ContactCount, 4);  
			for (int ConstraintIndex = 0; ConstraintIndex < ContactCount; ConstraintIndex++)
			{
				EXPECT_NEAR(10.0f, Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.Phi, 0.01);
				EXPECT_NEAR(Box2Transform.TransformPosition(FVec3(Constraint.GetManifoldPoints()[ConstraintIndex].ContactPoint.ShapeContactPoints[1])).Z, 100.0f, 0.01);
			}
		}

	}

	// Test that we correctly identify edge-edge contacts and calculate the correct separation
	// even when we have large margins.
	GTEST_TEST(OneShotManifoldTests, TestConvexMarginEdgeEdge)
	{
		FReal Dt = 1 / 30.0f;
		FConvex::FRealType HalfSize = 100.0f;
		FReal Margin = 0.2f * HalfSize;
		float ExpectedPhi = 0.0f;
		float Offset = 2.0f * HalfSize * FMath::Sqrt(2.0f);

		TArray<FConvex::FVec3Type> BoxVerts =
		{
			{-HalfSize, -HalfSize, -HalfSize},
			{-HalfSize,  HalfSize, -HalfSize},
			{ HalfSize,  HalfSize, -HalfSize},
			{ HalfSize, -HalfSize, -HalfSize},
			{-HalfSize, -HalfSize,  HalfSize},
			{-HalfSize,  HalfSize,  HalfSize},
			{ HalfSize,  HalfSize,  HalfSize},
			{ HalfSize, -HalfSize,  HalfSize},
		};

		// First box rotated 45 degrees about Z and placed at the oirigin
		// Second box rotated 45 degrees about Z and placed along X axis so that we have edge-edge contact with specified Phi
		FConvex Convex(BoxVerts, Margin);
		FRigidTransform3 Convex1Transform = FRigidTransform3(FVec3(0.0f, 0.0f, 0.0f), FRotation3::FromAxisAngle(FVec3(0.0f, 0.0f, 1.0f), FMath::DegreesToRadians(45)));
		FRigidTransform3 Convex2Transform = FRigidTransform3(FVec3(Offset + ExpectedPhi, 0.0f, 0.0f), FRotation3::FromAxisAngle(FVec3(0.0f, 1.0f, 0.0f), FMath::DegreesToRadians(45)));

		FPBDCollisionConstraint Constraint;

		Collisions::ConstructConvexConvexOneShotManifold(Convex, Convex1Transform, Convex, Convex2Transform, Dt, Constraint);
		int ContactCount = Constraint.GetManifoldPoints().Num();
		EXPECT_EQ(ContactCount, 1);
		if (ContactCount > 0)
		{
			EXPECT_NEAR(Constraint.GetManifoldPoints()[0].ContactPoint.Phi, 0.0f, 0.01f);
			EXPECT_NEAR(Convex1Transform.TransformPosition(FVec3(Constraint.GetManifoldPoints()[0].ContactPoint.ShapeContactPoints[0])).X, 0.5f * Offset, 0.01f);
		}
	}

	//GTEST_TEST(OneShotManifoldTests, TestNonUniformScaledConvexMargin)
	//{
	//}

}