// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestOverlap.h"

#include "HeadlessChaos.h"
#include "Chaos/Box.h"
#include "Chaos/Capsule.h"
#include "Chaos/Sphere.h"
#include "Chaos/GeometryQueries.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"

namespace ChaosTest
{
	using namespace Chaos;

	void OverlapTriMesh()
	{
		{
			// Trimesh is simple pyramid
			using namespace Chaos;
			FTriangleMeshImplicitObject::ParticlesType TrimeshParticles(
				{
					{-10.0, 0.0, 0.0},
					{10.0, 0.0, 0.0},
					{0.0, 10.0, 0.0},
					{0.0, -10.0, 0.0},
					{0.0,  0.0, 10.0},
				});

			TArray<TVec3<int32>> Indices(
				{
					{0, 1, 2},
					{0, 3, 1},
					{0, 1, 2},
					{0, 2, 4},
					{1, 2, 4},
					{0, 3, 4},
					{1, 3, 4}
				});

			TArray<uint16> Materials;
			for (int32 i = 0; i < Indices.Num(); ++i)
			{
				Materials.Emplace(0);
			}

			TUniquePtr<FTriangleMeshImplicitObject> TriangleMesh = MakeUnique<FTriangleMeshImplicitObject>(MoveTemp(TrimeshParticles), MoveTemp(Indices), MoveTemp(Materials));
			FMTDInfo MTDInfo;
			{
				// Capsule test
				const FVec3 X1 = { 0.0, 0.0, -2.0 };
				const FVec3 X2 = { 0.0, 0.0, 2.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);

				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(12.5, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(10.0, 10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-10.0, -10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(10.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, -10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-10.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 12.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 12.0), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					// Inside Mesh
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 4.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(6.5, 0.0, 6.5), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(6.5, 0.0, 6.5), FQuat(FVec3(0.0, 1.0, 0.0), -3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_FALSE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(2.5, 2.5, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-2.5, -2.5, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
				}
			}
			{
				const FVec3 X1 = { -11.0, 2.0, 0.0 };
				const FVec3 X2 = { -9.0, -8.0, 0.0 };
				const FReal Radius = 0.01;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_FALSE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
			{
				const FVec3 X1 = { 10.0, -1.0, 0.0 };
				const FVec3 X2 = { 11.0, 1.0, 0.0 };
				const FReal Radius = 0.1;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
				bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
				EXPECT_FALSE(bResult);
			}
			{
				// Sphere test
				const FVec3 X = { 0.0, 0.0, 0.0 };
				const FReal Radius = 1.0;
				const Chaos::FSphere Sphere = Chaos::FSphere(X, Radius);

				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(12.5, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(7.0, 7.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-7.0, -7.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(10.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 10.2, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, -10.2, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-10.3, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 10.9), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 11.5), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					// Inside Mesh
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 4.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(5.5, 0.0, 5.5), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(5.8, 0.0, 5.8), FQuat(FVec3(0.0, 1.0, 0.0), -3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_FALSE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-3.0, 0.0, 9.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_FALSE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
			{
				const FVec3 X1 = { 0.0, 0.0, -200.0 };
				const FVec3 X2 = { 0.0, 0.0, 200.0 };
				const FReal Radius = 100.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				TRefCountPtr<FCapsule> CapsuleShared( new FCapsule(X1, X2, Radius));
				FVec3 TriMeshScale = { 0.01, 0.01, 0.01 };
				TImplicitObjectScaled<FCapsule> ScaledCapsule = TImplicitObjectScaled<FCapsule>(CapsuleShared, TriMeshScale);
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(10.0, 10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-10.0, -10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(10.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, -10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-10.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 12.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 12.0), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					// Inside Mesh
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 4.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(6.5, 0.0, 6.5), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(6.5, 0.0, 6.5), FQuat(FVec3(0.0, 1.0, 0.0), -3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_FALSE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
			{
				const FVec3 X1 = { 0.0, 0.0, -20.0 };
				const FVec3 X2 = { 0.0, 0.0, 20.0 };
				const FReal Radius = 10.0;
				TRefCountPtr<FCapsule> CapsuleShared( new FCapsule(X1, X2, Radius));
				FVec3 TriMeshScale = { 10.0, 10.0, 10.0 };
				FVec3 InvScale = 1.0 / TriMeshScale;
				TImplicitObjectScaled<FCapsule> ScaledCapsule = TImplicitObjectScaled<FCapsule>(CapsuleShared, InvScale);
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(100.0, 100.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(125.0, 0.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(-100.0, -100.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(100.0, 0.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 100.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, -100.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(-100.0, 0.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 120.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 120.0), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					// Inside Mesh
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 40.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(65.0, 0.0, 65.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(65.0, 0.0, 65.0), FQuat(FVec3(0.0, 1.0, 0.0), -3.1415926 / 4.0));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_FALSE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
			// Non uniform scale
			{
				const FVec3 X1 = { 0.0, 0.0, -2.0 };
				const FVec3 X2 = { 0.0, 0.0, 2.0 };
				const FReal Radius = 1.0;
				TRefCountPtr<FCapsule> CapsuleShared( new FCapsule(X1, X2, Radius));
				FVec3 TriMeshScale = { 1.0, 1.0, 2.0 };
				FVec3 InvScale = 1.0 / TriMeshScale;
				TImplicitObjectScaled<FCapsule> ScaledCapsule = TImplicitObjectScaled<FCapsule>(CapsuleShared, InvScale);
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(10.0, 10.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(12.5, 0.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(-10.0, -10.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(10.0, 0.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 10.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, -10.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(-10.0, 0.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 22.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 22.0), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 22.0), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 2.0));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					// Inside Mesh
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 8.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(6.0, 0.0, 11.5), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					// Triangles parallel to the capsule axis with non uniform scale
					FRigidTransform3 QueryTM(FVec3(6.0, 0.0, 11.5), FQuat(FVec3(0.0, 1.0, 0.0), -3.1415926 / 4.0 + 3.1415926 / 8.0));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_FALSE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledCapsule, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
			// Non uniform scale sphere
			{
				const FVec3 X = { 0.0, 0.0, 0.0 };
				const FReal Radius = 1.0;
				Chaos::FSpherePtr SphereShared( new Chaos::FSphere(X, Radius));
				FVec3 TriMeshScale = { 1.0, 1.0, 2.0 };
				FVec3 InvScale = 1.0 / TriMeshScale;
				TImplicitObjectScaled<Chaos::FSphere> ScaledSphere = TImplicitObjectScaled<Chaos::FSphere>(SphereShared, InvScale);
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(10.0, 10.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(12.5, 0.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(-10.0, -10.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(10.0, 0.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 10.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, -10.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(-10.0, 0.0, 0.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 20.5), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 22.0), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					// Inside Mesh
					FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 8.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(5.5, 0.0, 10.5), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					// Triangles parallel to the capsule axis with non uniform scale
					FRigidTransform3 QueryTM(FVec3(6.0, 0.0, 11.5), FQuat(FVec3(0.0, 1.0, 0.0), -3.1415926 / 4.0 + 3.1415926 / 8.0));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_FALSE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledSphere, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}

			{
				// Box test
				const TBox<FReal, 3> Box = TBox<FReal, 3>({ -1.0, -1.0, -2.0 }, { 1.0, 1.0, 2.0 });
				{
					// Consistency test
					for (FReal X = 0.0; X < 10.0; X += 1.0)
					{
						for (FReal Y = 0.0; Y < 10.0; Y += 1.0)
						{
							for (FReal Z = 0.0; Z < 10.0; Z += 1.0)
							{
								const FRigidTransform3 QueryTM(FVec3(X, Y, Z), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 3.0));
								bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
								bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
								EXPECT_EQ(bResult, bResultMTD);

								const FRigidTransform3 QueryIdTM(FVec3(X, Y, Z), FQuat::Identity);
								bResult = TriangleMesh->OverlapGeom(Box, QueryIdTM, 0.0);
								bResultMTD = TriangleMesh->OverlapGeom(Box, QueryIdTM, 0.0, &MTDInfo);
								EXPECT_EQ(bResult, bResultMTD);
							}
						}
					}
				}
				{
					const FRigidTransform3 QueryTM(FVec3(7.0, 0.0, 10.0), FQuat(FVec3(0.0, 1.0, 0.0), 3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, nullptr);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(10.0, 10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-10.0, -10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(10.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, -10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-10.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 12.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 12.0), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					// Inside Mesh
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 4.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, -2.5), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(5.0, 5.0, -2.5), FQuat(FVec3(0.0, 1.0, 0.0), 3.1415926 / 6.0));
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-5.0, 5.0, -2.5), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-5.0, -5.0, -2.5), FQuat(FVec3(1.0, 0.0, 0.0), -3.1415926 / 6.0));
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, -2.5), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(7.0, 0.0, 10.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
			{
				// Box test
				const TBox<FReal, 3> Box = TBox<FReal, 3>({ -1.0, -1.0, -1.0 }, { 1.0, 1.0, 1.0 });
				{
					const FRigidTransform3 QueryTM(FVec3(7.0, 0.0, 10.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
				}
			}

			// Box Scaled
			{
				FBoxPtr BigBoxSafe( new TBox<FReal, 3>(FVec3(-100.0, -100.0, -100.0), FVec3(100.0, 100.0, 100.0)));
				FVec3 TriMeshScale = { 0.01, 0.01, 0.02 };
				TImplicitObjectScaled<TBox<FReal, 3>> ScaledBox = TImplicitObjectScaled<TBox<FReal, 3>>(BigBoxSafe, TriMeshScale);
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(10.0, 10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-10.0, -10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(10.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, -10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-10.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 12.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 12.0), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					// Inside Mesh
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 4.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
			// Box Scaled test non uniform transform with rotation
			{
				FBoxPtr BigBoxSafe( new TBox<FReal, 3>(FVec3(-100.0, -100.0, -100.0), FVec3(100.0, 100.0, 100.0)));
				FVec3 TriMeshScale = { 0.01, 0.01, 0.05 };
				TImplicitObjectScaled<TBox<FReal, 3>> ScaledBox = TImplicitObjectScaled<TBox<FReal, 3>>(BigBoxSafe, TriMeshScale);
				{
					const FRigidTransform3 QueryTM(FVec3(5.5, 0.0, 10.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(7.0, 0.0, 10.001), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(7.0, 0.0, 10.0), FQuat(FVec3(0.0, 1.0, 0.0), -3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(7.0, 0.0, 10.0), FQuat(FVec3(0.0, 1.0, 0.0), 3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
			{
				FBoxPtr BigBoxSafe( new TBox<FReal, 3>(FVec3(-1.0, -1.0, -1.0), FVec3(1.0, 1.0, 1.0)));
				FVec3 TriMeshScale = { 10.0, 10.0, 2.0 };
				FVec3 InvScale = 1.0 / TriMeshScale;
				TImplicitObjectScaled<TBox<FReal, 3>> ScaledBox = TImplicitObjectScaled<TBox<FReal, 3>>(BigBoxSafe, InvScale);
				{
					FRigidTransform3 QueryTM(FVec3(50, 0.0, 10.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					// Inside mesh
					FRigidTransform3 QueryTM(FVec3(40.0, 0.0, 10.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(52.0, 0.0, 11.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(52.0, 0.0, 11.0), FQuat(FVec3(0.0, 1.0, 0.0), -3.1415926 / 4.0));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(51.0, 0.0, 11.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(51.5, 0.0, 11.0), FQuat(FVec3(0.0, 1.0, 0.0), 4.0 / 5.0 * 3.1415926 / 2.0));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(51.5, 0.0, 11.0), FQuat(FVec3(0.0, 1.0, 0.0), -4.0 / 5.0 * 3.1415926 / 2.0));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}

			{
				// Regression test: Box with centre not at the local origin
								
				// Rotated UnScaled box
				const TBox<FReal, 3> BoxUnscaled = TBox<FReal, 3>({ -5.0f, -6.0f, 0.0f }, { 5.0f, 6.0f, 10.0f });
				{
					FRigidTransform3 QueryTM(FVec3(0, 0.0, 11.0), FQuat{ 1, 0, 0, 0 }); // Pi rotation around x axis
					bool bResult = TriangleMesh->OverlapGeom(BoxUnscaled, QueryTM, 0.0, nullptr);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(BoxUnscaled, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}

				// Rotated Scaled box
				FBoxPtr BoxSafe(new TBox<FReal, 3>(FVec3(-10.0, -10.0, 0.0), FVec3(10.0, 10.0, 10.0)));
				FVec3 TriMeshScale = { 5.0f, 6.0f, 10.0f };
				FVec3 BoxScale = { 5.0f, 6.0f, 10.0f };
				TImplicitObjectScaled<TBox<FReal, 3>> ScaledBox = TImplicitObjectScaled<TBox<FReal, 3>>(BoxSafe, BoxScale);
				{
					FRigidTransform3 QueryTM(FVec3(0, 0.0, 110.0), FQuat{1, 0, 0, 0}); // Pi rotation around x axis
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);					
				}

				// Mirrored and Rotated box, mirrored trimesh // regression test
				{
					FBoxPtr BoxSafe2(new TBox<FReal, 3>(FVec3(-100.0, -100.0, -100.0), FVec3(100.0, 100.0, 100.0)));
					FVec3 TriMeshScale2 = { -10.0f, 10.0f, 10.0f };
					FVec3 InvTriMeshScale2 = 1.0 / TriMeshScale2;
					FVec3 BoxScale2 = { 1, 1, 1 };
					BoxScale2 = BoxScale2 * InvTriMeshScale2; // Box needs to be in mesh space // TODO make sure other tests are correct
					TImplicitObjectScaled<TBox<FReal, 3>> ScaledBox2 = TImplicitObjectScaled<TBox<FReal, 3>>(BoxSafe2, BoxScale2);

					FRigidTransform3 QueryTM(FVec3(0, 0.0, 105.0) * InvTriMeshScale2, FQuat{ 1, 0, 0, 0 }); // Pi rotation around x axis
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox2, QueryTM, 0.0, nullptr, TriMeshScale2);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox2, QueryTM, 0.0, &MTDInfo, TriMeshScale2);
					EXPECT_EQ(bResult, bResultMTD);
				}

				// Unrotated unscaled box
				{
					FRigidTransform3 QueryTM(FVec3(0, 0.0, 11.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(BoxUnscaled, QueryTM, 0.0, nullptr);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(BoxUnscaled, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}

				// Unrotated scaled box
				{
					FRigidTransform3 QueryTM(FVec3(0, 0.0, 110.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}

			{
				// Non uniform test with box not being a cube
				FBoxPtr BigBoxSafe( new TBox<FReal, 3>(FVec3(-1.0, -5.0, -1.0), FVec3(1.0, 5.0, 1.0)));
				FVec3 TriMeshScale = { 10.0, 10.0, 2.0 };
				FVec3 InvScale = 1.0 / TriMeshScale;
				TImplicitObjectScaled<TBox<FReal, 3>> ScaledBox = TImplicitObjectScaled<TBox<FReal, 3>>(BigBoxSafe, InvScale);
				{
					FRigidTransform3 QueryTM(FVec3(50, 0.0, 10.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
					FReal ErrorMargin = 0.01; // Good enough for engineering 
					EXPECT_VECTOR_NEAR(MTDInfo.Normal, FVec3(0.1961f , 0.0f, 0.9806f), ErrorMargin);
					EXPECT_VECTOR_NEAR(MTDInfo.Position, FVec3(49.2308f, 0.0f, 10.1538f), ErrorMargin);
					EXPECT_NEAR(MTDInfo.Penetration, 1.1767, ErrorMargin);
				}
				{
					// Inside mesh
					FRigidTransform3 QueryTM(FVec3(40.0, 0.0, 10.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(52.0, 0.0, 11.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 52.0, 11.0), FQuat::Identity);
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(54.0, 0.0, 11.0), FQuat(FVec3(0.0, 1.0, 0.0), -3.1415926 / 2.0 * (1.5 / 5.0)));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 54.0, 11.0), FQuat(FVec3(1.0, 0.0, 0.0), -3.1415926 / 2.0 * (1.5 / 5.0)));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(52.0, 0.0, 11.0), FQuat(FVec3(0.0, 1.0, 0.0), (1.5 / 5.0) * 3.1415926 / 2.0));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					FRigidTransform3 QueryTM(FVec3(0.0, 52.0, 11.0), FQuat(FVec3(1.0, 0.0, 0.0), (1.5 / 5.0) * 3.1415926 / 2.0));
					QueryTM = FRigidTransform3(QueryTM.GetLocation() * InvScale, QueryTM.GetRotation());
					bool bResult = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, nullptr, TriMeshScale);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(ScaledBox, QueryTM, 0.0, &MTDInfo, TriMeshScale);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
			{
				// Sphere test
				const Chaos::FSphere Sphere = Chaos::FSphere({ 0.0, 0.0, 0.0 }, 2.0);
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(10.0, 10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-10.0, -10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(10.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, -10.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(-10.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 12.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 12.0), FQuat(FVec3(1.0, 0.0, 0.0), 3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					// Inside Mesh
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 4.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
		}
		{
			FTriangleMeshImplicitObject::ParticlesType TrimeshParticles(
			{
				{-10.0, -10.0, 0.0},
				{10.0, -10.0, 0.0},
				{-10.0, 10.0, 0.0},
				{10.0, 10.0, 0.0},
				{-10.0, -10.0, 10.0},
				{10.0, -10.0, 10.0},
				{ -10.0, 10.0, 10.0 },
				{ 10.0, 10.0, 10.0 },

			});

			TArray<TVec3<int32>> Indices(
				{
					{0, 1, 2},
					{0, 3, 1},
					{0, 1, 2},
					{4, 5, 6},
					{4, 7, 5},
					{4, 5, 6},
				});

			TArray<uint16> Materials;
			for (int32 i = 0; i < Indices.Num(); ++i)
			{
				Materials.Emplace(0);
			}

			TUniquePtr<FTriangleMeshImplicitObject> TriangleMesh = MakeUnique<FTriangleMeshImplicitObject>(MoveTemp(TrimeshParticles), MoveTemp(Indices), MoveTemp(Materials));
			FMTDInfo MTDInfo;
			{
				// Capsule test
				const FVec3 X1 = { 0.0, 0.0, -2.0 };
				const FVec3 X2 = { 0.0, 0.0, 2.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);

				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 5.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(10.0, 0.0, 11.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(11.5, 0.0, 11.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(11.5, 0.0, 11.0), FQuat(FVec3(0.0, 1.0, 0.0), 3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, true);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(11.5, 0.0, 11.0), FQuat(FVec3(0.0, 1.0, 0.0), -3.1415926 / 4.0));
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_EQ(bResult, false);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
			{
				const FVec3 X1 = { 0.0, 0.0, -250.0 };
				const FVec3 X2 = { 0.0, 0.0, 250.0 };
				const FReal Radius = 0.5;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);

				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 5.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
			{
				const FVec3 X1 = { 0.0, 0.0, 0.0 };
				const FVec3 X2 = { 0.0, 0.0, 0.0 };
				const FReal Radius = 0.5;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 5.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_FALSE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 10.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(5.0, -10.25, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(5.0, -10.6, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_FALSE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
		}
		{
			FTriangleMeshImplicitObject::ParticlesType TrimeshParticles(
				{
					{-5.0, 0.0, 0.0},
					{5.0, 0.0, 0.0},
					{3.0, 0.0, 100.0},
				});

			TArray<TVec3<int32>> Indices(
				{
					{0, 1, 2},
				});

			TArray<uint16> Materials;
			for (int32 i = 0; i < Indices.Num(); ++i)
			{
				Materials.Emplace(0);
			}

			TUniquePtr<FTriangleMeshImplicitObject> TriangleMesh = MakeUnique<FTriangleMeshImplicitObject>(MoveTemp(TrimeshParticles), MoveTemp(Indices), MoveTemp(Materials));
			FMTDInfo MTDInfo;
			{
				// Sphere test
				const FVec3 X = { 0.0, 0.0, 0.0 };
				const FReal Radius = 1.0;
				const Chaos::FSphere Sphere = Chaos::FSphere(X, Radius);
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(3.9, 0.0, 100.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(4.0, 0.0, 90.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(4.5, 0.0, 90.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_FALSE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
			}
			{
				// Capsule test
				const FVec3 X1 = { 0.0, 0.0, -2.0 };
				const FVec3 X2 = { 0.0, 0.0, 2.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(3.9, 0.0, 100.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(4.0, 0.0, 90.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}
				{
					const FRigidTransform3 QueryTM(FVec3(4.5, 0.0, 90.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_FALSE(bResult);
					bool bResultMTD = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0, &MTDInfo);
					EXPECT_EQ(bResult, bResultMTD);
				}

			}
		}
		{
			FTriangleMeshImplicitObject::ParticlesType TrimeshParticles(
				{
					{100.0, 0.0, 0.0},
					{86.0, -75.0, 0.0},
					{50.0, -130.0, 0.0},
				});

			TArray<TVec3<int32>> Indices(
				{
					{0, 1, 2},
				});

			TArray<uint16> Materials;
			for (int32 i = 0; i < Indices.Num(); ++i)
			{
				Materials.Emplace(0);
			}

			TUniquePtr<FTriangleMeshImplicitObject> TriangleMesh = MakeUnique<FTriangleMeshImplicitObject>(MoveTemp(TrimeshParticles), MoveTemp(Indices), MoveTemp(Materials));
			
				// Capsule test
			{
				const FVec3 X1 = { 66.0, -99.0, 0.0 };
				const FVec3 X2 = { 67.0, -98.0, 0.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
				bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
				EXPECT_TRUE(bResult);
			}
			{
				const FVec3 X1 = { 80.0, -92.0, 0.0 };
				const FVec3 X2 = { 85.0, -88.0, 0.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
				bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
				EXPECT_FALSE(bResult);
			}
			{
				const FVec3 X1 = { 40.0, -127.0, 0.0 };
				const FVec3 X2 = { 55.0, -137.0, 0.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
				bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
				EXPECT_FALSE(bResult);
			}
			{
				const FVec3 X1 = { 0.0, -110.0, 0.0 };
				const FVec3 X2 = { 100.0, -160.0, 0.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
				bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
				EXPECT_FALSE(bResult);
			}
			{
				const FVec3 X1 = { 56.0, -138.0, 0.0 };
				const FVec3 X2 = { 55.0, -137.0, 0.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
				bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
				EXPECT_FALSE(bResult);
			}
			{
				const FVec3 X1 = { 48.0, -130.0, 0.0 };
				const FVec3 X2 = { 47.0, -131.0, 0.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
				bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
				EXPECT_FALSE(bResult);
			}
			{
				const FVec3 X1 = { 101.5, 0.0, 0.0 };
				const FVec3 X2 = { 106.0, 0.0, 0.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
				bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
				EXPECT_FALSE(bResult);
			}
			{
				const FVec3 X1 = { 95.0, 0.0, 0.0 };
				const FVec3 X2 = { 98.0, 0.0, 0.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
				bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
				EXPECT_FALSE(bResult);
			}
			{
				const FVec3 X1 = { 95.0, 0.0, 0.0 };
				const FVec3 X2 = { 94, 0.0, 0.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
				bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
				EXPECT_FALSE(bResult);
			}
			{
				// Sphere test
				const FVec3 X = { 66.0, -99.0, 0.0 };
				const FReal Radius = 1.0;
				const Chaos::FSphere Sphere = Chaos::FSphere(X, Radius);
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
				}
			}
			{
				const TBox<FReal, 3> Box = TBox<FReal, 3>({ -0.5, -0.5, -0.5 }, { 0.5, 0.5, 0.5 });
				{
					const FRigidTransform3 QueryTM(FVec3(66.0, -99.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
				}
			}
		}
		{
			FTriangleMeshImplicitObject::ParticlesType TrimeshParticles(
				{
					{100.0, 0.0, 0.0},
					{86.0, -75.0, 0.0},
					{60.0, -110.0, 0.0},
				});

			TArray<TVec3<int32>> Indices(
				{
					{0, 1, 2},
				});

			TArray<uint16> Materials;
			for (int32 i = 0; i < Indices.Num(); ++i)
			{
				Materials.Emplace(0);
			}

			TUniquePtr<FTriangleMeshImplicitObject> TriangleMesh = MakeUnique<FTriangleMeshImplicitObject>(MoveTemp(TrimeshParticles), MoveTemp(Indices), MoveTemp(Materials));
			{
				// Capsule test
				const FVec3 X1 = { 66.0, -99.0, 0.0 };
				const FVec3 X2 = { 67.0, -98.0, 0.0 };
				const FReal Radius = 1.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
				}
			}
			{
				// Sphere test
				const FVec3 X = { 66.0, -99.0, 0.0 };
				const FReal Radius = 1.0;
				const Chaos::FSphere Sphere = Chaos::FSphere(X, Radius);
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
				}
			}
			{
				const TBox<FReal, 3> Box = TBox<FReal, 3>({ -0.5, -0.5, -0.5 }, { 0.5, 0.5, 0.5 });
				{
					const FRigidTransform3 QueryTM(FVec3(66.0, -99.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Box, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
				}
			}
		}
		{
			FTriangleMeshImplicitObject::ParticlesType TrimeshParticles(
				{
					{0.0, 0.0, 0.0},
					{-200.0, 100.0, 0.0},
					{-200.0, -100.0, 0.0},
				});

			TArray<TVec3<int32>> Indices(
				{
					{0, 1, 2},
				});

			TArray<uint16> Materials;
			for (int32 i = 0; i < Indices.Num(); ++i)
			{
				Materials.Emplace(0);
			}

			TUniquePtr<FTriangleMeshImplicitObject> TriangleMesh = MakeUnique<FTriangleMeshImplicitObject>(MoveTemp(TrimeshParticles), MoveTemp(Indices), MoveTemp(Materials));
			{
				// Capsule test
				const FVec3 X1 = { 0.0, -100.0, 100.0 };
				FVec3 X2 = { 0.0, 100.0, 100.0 };
				const FReal Radius = 101.0;
				const FCapsule Capsule = FCapsule(X1, X2, Radius);
				{
					const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
					bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
					EXPECT_TRUE(bResult);
				}
			}
		}
		{
			FTriangleMeshImplicitObject::ParticlesType TrimeshParticles(
				{
					{0.0, 0.0, 0.0},
					{100, -100, 0.0},
					{-200.0, -400.0, 0.0},
				});

			TArray<TVec3<int32>> Indices(
				{
					{0, 1, 2},
				});

			TArray<uint16> Materials;
			for (int32 i = 0; i < Indices.Num(); ++i)
			{
				Materials.Emplace(0);
			}

			TUniquePtr<FTriangleMeshImplicitObject> TriangleMesh = MakeUnique<FTriangleMeshImplicitObject>(MoveTemp(TrimeshParticles), MoveTemp(Indices), MoveTemp(Materials));
			{
				{
					// Capsule test
					const FVec3 X1 = { -100, 0, 0 };
					const FVec3 X2 = { -200, 0, 0 };
					const FReal Radius = 99.0f;
					// This capsule will not touch a vertex, but will intersect an edge
					const FCapsule Capsule = FCapsule(X1, X2, Radius); 
					{
						const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
						bool bResult = TriangleMesh->OverlapGeom(Capsule, QueryTM, 0.0);
						EXPECT_TRUE(bResult);
					}
				}
				{
					const FVec3 X = { -100, 0, 0 };
					const FReal Radius = 99.0f;
					const Chaos::FSphere Sphere = Chaos::FSphere(X, Radius);
					{
						const FRigidTransform3 QueryTM(FVec3(0.0, 0.0, 0.0), FQuat::Identity);
						bool bResult = TriangleMesh->OverlapGeom(Sphere, QueryTM, 0.0);
						EXPECT_TRUE(bResult);
					}
				}
			}
		}
	}
}
