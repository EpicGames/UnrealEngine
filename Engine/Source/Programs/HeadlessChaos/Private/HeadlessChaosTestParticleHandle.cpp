// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestParticleHandle.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDRigidsSOAs.h"

namespace ChaosTest {

	using namespace Chaos;

	void ParticleIteratorTest()
	{
		auto Empty = MakeUnique<FGeometryParticles>();
		auto Five = MakeUnique<FGeometryParticles>();
		Five->AddParticles(5);
		auto Two = MakeUnique<FGeometryParticles>();
		Two->AddParticles(2);

		TArray<TUniquePtr<FGeometryParticleHandle>> HandleStorage;
		auto CreateHandlesHelper = [&HandleStorage](auto& SOA)
		{
			//Create a handle with just the bare minimum that we need so that handle iterator uses real handles that are linked with SOA
			for(int32 ParticleIdx = 0; (unsigned)ParticleIdx < SOA->Size(); ++ParticleIdx)
			{
				TUniquePtr<FGeometryParticleHandle> NewParticleHandle = FGeometryParticleHandle::CreateParticleHandle(MakeSerializable(SOA), ParticleIdx, HandleStorage.Num());
				SOA->SetHandle(ParticleIdx, NewParticleHandle.Get());
				HandleStorage.Add(MoveTemp(NewParticleHandle));
			}
		};

		CreateHandlesHelper(Five);
		CreateHandlesHelper(Two);

		//empty soa in the start
		{
			TArray<FGeometryParticleHandle*> Handles;
			TArray<TSOAView<FGeometryParticles>> TmpArray = { Empty.Get(), Five.Get(), Two.Get() };
			TParticleView<FGeometryParticles> View = MakeParticleView(MoveTemp(TmpArray));
			for (auto& Particle : View)
			{
				Handles.Add(Particle.Handle());
			}
			EXPECT_EQ(Handles.Num(), 7);

			//disable first middle and last
			Five->LightWeightDisabled(0) = true;
			Five->LightWeightDisabled(2) = true;
			Five->LightWeightDisabled(4) = true;
			Two->LightWeightDisabled(1) = true;

			int32 NonDisabledCount = 0;
			for (auto& Particle : View)
			{
				++NonDisabledCount;
			}
			EXPECT_EQ(NonDisabledCount, 3);

			THandleView<FGeometryParticles> HandleView = MakeHandleView(Handles);
			int32 Count = 0;
			for (auto& Handle : HandleView)
			{
				++Count;
			}
			EXPECT_EQ(Count, 3);	//3 because 4 are disabled

			Five->LightWeightDisabled(0) = false;
			Five->LightWeightDisabled(2) = false;
			Five->LightWeightDisabled(4) = false;
			Two->LightWeightDisabled(1) = false;
		}

		//empty soa in the middle
		{
			TArray<FGeometryParticleHandle*> Handles;
			TArray<TSOAView<FGeometryParticles>> TmpArray = { Five.Get(), Empty.Get(), Two.Get()};
			TParticleView<FGeometryParticles> View = MakeParticleView(MoveTemp(TmpArray));
			for (auto& Particle : View)
			{
				Handles.Add(Particle.Handle());
			}
			EXPECT_EQ(Handles.Num(), 7);

			THandleView<FGeometryParticles> HandleView = MakeHandleView(Handles);
			int32 Count = 0;
			for (auto& Handle : HandleView)
			{
				++Count;
			}
			EXPECT_EQ(Count, 7);
		}

		//empty soa in the end
		{
			TArray<FGeometryParticleHandle*> Handles;
			TArray<TSOAView<FGeometryParticles>> TmpArray = { Five.Get(), Two.Get(), Empty.Get() };
			TParticleView<FGeometryParticles> View = MakeParticleView(MoveTemp(TmpArray));
			for (auto& Particle : View)
			{
				Handles.Add(Particle.Handle());
			}
			EXPECT_EQ(Handles.Num(), 7);

			//disable first middle and last
			Five->LightWeightDisabled(0) = true;
			Five->LightWeightDisabled(2) = true;
			Five->LightWeightDisabled(4) = true;
			Two->LightWeightDisabled(1) = true;

			int32 NonDisabledCount = 0;
			for (auto& Particle : View)
			{
				++NonDisabledCount;
			}
			EXPECT_EQ(NonDisabledCount, 3);

			THandleView<FGeometryParticles> HandleView = MakeHandleView(Handles);
			int32 Count = 0;
			for (auto& Handle : HandleView)
			{
				++Count;
			}
			EXPECT_EQ(Count, 3);	//3 because 4 are disabled

			Five->LightWeightDisabled(0) = false;
			Five->LightWeightDisabled(2) = false;
			Five->LightWeightDisabled(4) = false;
			Two->LightWeightDisabled(1) = false;
		}

		//parallel for
		{
			TArray<TSOAView<FGeometryParticles>> TmpArray = { Empty.Get(), Five.Get(), Two.Get() };
			TParticleView<FGeometryParticles> View = MakeParticleView(MoveTemp(TmpArray));
			{
				TArray<bool> AuxArray;
				AuxArray.SetNumZeroed(View.Num());
				bool DoubleWrite = false;
				View.ParallelFor([&AuxArray, &DoubleWrite](const auto& Particle, int32 Idx)
				{
					if (AuxArray[Idx])
					{
						DoubleWrite = true;
					}
					AuxArray[Idx] = true;
				});

				EXPECT_FALSE(DoubleWrite);

				for (bool Val : AuxArray)
				{
					EXPECT_TRUE(Val);
				}
			}

			TArray<FGeometryParticleHandle*> Handles;
			for (auto& Particle : View)
			{
				Handles.Add(Particle.Handle());
			}
			THandleView<FGeometryParticles> HandleView = MakeHandleView(Handles);
			
			{
				TArray<bool> AuxArray;
				AuxArray.SetNumZeroed(HandleView.Num());
				bool DoubleWrite = false;
				HandleView.ParallelFor([&AuxArray, &DoubleWrite](const auto& Particle, int32 Idx)
				{
					if (AuxArray[Idx])
					{
						DoubleWrite = true;
					}
					AuxArray[Idx] = true;
				});

				EXPECT_FALSE(DoubleWrite);

				for (bool Val : AuxArray)
				{
					EXPECT_TRUE(Val);
				}
			}


			//disable first middle and last
			Five->LightWeightDisabled(0) = true;
			Five->LightWeightDisabled(2) = true;
			Five->LightWeightDisabled(4) = true;
			Two->LightWeightDisabled(1) = true;

			{
				TArray<bool> AuxArray;
				AuxArray.SetNumZeroed(View.Num());
				bool DoubleWrite = false;
				View.ParallelFor([&AuxArray, &DoubleWrite](const auto& Particle, int32 Idx)
					{
						if (AuxArray[Idx])
						{
							DoubleWrite = true;
						}
						AuxArray[Idx] = true;
					});

				EXPECT_FALSE(DoubleWrite);

				for (int32 Idx = 0; Idx < AuxArray.Num(); ++Idx)
				{
					if(Idx == 0 || Idx == 2 || Idx == 4 || Idx == 6)
					{
						//didn't write because disabled
						EXPECT_FALSE(AuxArray[Idx]);
					}
					else
					{
						EXPECT_TRUE(AuxArray[Idx]);
					}
					
				}
			}

			{
				TArray<bool> AuxArray;
				AuxArray.SetNumZeroed(View.Num());
				bool DoubleWrite = false;
				HandleView.ParallelFor([&AuxArray, &DoubleWrite](const auto& Particle, int32 Idx)
					{
						if (AuxArray[Idx])
						{
							DoubleWrite = true;
						}
						AuxArray[Idx] = true;
					});

				EXPECT_FALSE(DoubleWrite);

				for (int32 Idx = 0; Idx < AuxArray.Num(); ++Idx)
				{
					if (Idx == 0 || Idx == 2 || Idx == 4 || Idx == 6)
					{
						//didn't write because disabled
						EXPECT_FALSE(AuxArray[Idx]);
					}
					else
					{
						EXPECT_TRUE(AuxArray[Idx]);
					}

				}
			}
		}
	}

	template <typename TPBDRigid>
	void ParticleHandleTestHelperObjectState(TPBDRigid* PBDRigid);

	template <>
	void ParticleHandleTestHelperObjectState<FPBDRigidParticle>(FPBDRigidParticle* PBDRigid)
	{
		PBDRigid->SetObjectState(EObjectStateType::Dynamic);
		EXPECT_EQ(PBDRigid->ObjectState(), EObjectStateType::Dynamic);
	}

	template <>
	void ParticleHandleTestHelperObjectState<FPBDRigidParticleHandle>(FPBDRigidParticleHandle* PBDRigid)
	{
		PBDRigid->SetObjectStateLowLevel(EObjectStateType::Dynamic);
		EXPECT_EQ(PBDRigid->ObjectState(), EObjectStateType::Dynamic);
	}

	template <typename TGeometry, typename TKinematicGeometry, typename TPBDRigid>
	void ParticleHandleTestHelper(TGeometry* Geometry, TKinematicGeometry* KinematicGeometry, TPBDRigid* PBDRigid)
	{
		EXPECT_EQ(Geometry->GetX()[0], 0);	//default constructor
		EXPECT_EQ(Geometry->GetX()[1], 0);
		EXPECT_EQ(Geometry->GetX()[2], 0);

		EXPECT_EQ(KinematicGeometry->GetV()[0], 0);	//default constructor
		EXPECT_EQ(KinematicGeometry->GetV()[1], 0);
		EXPECT_EQ(KinematicGeometry->GetV()[2], 0);

		EXPECT_EQ(PBDRigid->GetX()[0], 0);	//default constructor of base
		EXPECT_EQ(PBDRigid->GetX()[1], 0);
		EXPECT_EQ(PBDRigid->GetX()[2], 0);
		EXPECT_EQ(PBDRigid->GetV()[0], 0);
		EXPECT_EQ(PBDRigid->GetV()[1], 0);
		EXPECT_EQ(PBDRigid->GetV()[2], 0);
		EXPECT_EQ(PBDRigid->M(), 1);

		PBDRigid->SetX(FVec3(1, 2, 3));
		EXPECT_EQ(PBDRigid->GetX()[0], 1);
		KinematicGeometry->SetV(FVec3(3, 3, 3));
		EXPECT_EQ(KinematicGeometry->GetV()[0], 3);

		EXPECT_EQ(Geometry->ObjectState(), EObjectStateType::Static);
		EXPECT_EQ(KinematicGeometry->ObjectState(), EObjectStateType::Kinematic);
		TGeometry* KinematicAsStatic = KinematicGeometry;	//shows polymorphism works
		EXPECT_EQ(KinematicAsStatic->ObjectState(), EObjectStateType::Kinematic);

		TGeometry* DynamicAsStatic = PBDRigid;
		EXPECT_EQ(DynamicAsStatic->ObjectState(), EObjectStateType::Dynamic);
		EXPECT_EQ(DynamicAsStatic->GetX()[0], 1);

		//more polymorphism
		ParticleHandleTestHelperObjectState(PBDRigid);
	}

	void ParticleLifetimeAndThreading()
	{
		{
			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs SOAs(UniqueIndices);

			TArray<TUniquePtr<FPBDRigidParticle>> GTRawParticles;
			for (int i = 0; i < 3; ++i)
			{
				GTRawParticles.Emplace(FPBDRigidParticle::CreateParticle());
			}
			{
				//for each GT particle, create a physics thread side
				SOAs.CreateDynamicParticles(3);

				//Solver sets the game thread particle on the physics thread handle
				int32 Idx = 0;
				for (auto& Particle : SOAs.GetAllParticlesView())
				{
					Particle.GTGeometryParticle() = GTRawParticles[Idx++].Get();
				}
				
				FReal Count = 0;
				//fake step and write to physics side
				for (auto& Particle : SOAs.GetAllParticlesView())
				{
					Particle.SetX(FVec3(Count));
					Count += 1;
				}
			}

			//copy step to GT data
			{
				for (const auto& Particle : SOAs.GetAllParticlesView())
				{
					Particle.GTGeometryParticle()->SetX(Particle.GetX());
				}
			}

			//consume on GT using raw pointers
			for (int i = 0; i < 3; ++i)
			{
				EXPECT_EQ(GTRawParticles[i]->X()[0], i);
			}

			//GT destroys a particle by enqueing a command and nulling out its own raw pointer
			auto RawParticleToDelete = GTRawParticles[1].Get();
			GTRawParticles[1] = nullptr;
			//PT does the actual delete
			//GT would hold a private pointer that the solver can access
			//SOAs.DestroyParticle(RawParticleToDelete->GetPhysicsTreadHandle());
			//For now we just search
			for (auto& Particle : SOAs.GetAllParticlesView())
			{
				if (Particle.GTGeometryParticle() == RawParticleToDelete)
				{
					SOAs.DestroyParticle(Particle.Handle());	//GT data will be removed
					break;
				}
			}

			//make sure we deleted the right particle
			EXPECT_EQ(SOAs.GetAllParticlesView().Num(), 2);

			for (auto& Particle : SOAs.GetAllParticlesView())
			{
				EXPECT_TRUE(Particle.GetX()[0] != 1);
			}
		}
	}

	void ParticleDestroyOrdering()
	{
		{
			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs SOAs(UniqueIndices);
			SOAs.CreateDynamicParticles(10);
			FReal Count = 0;
			FGeometryParticleHandle* ThirdParticle = nullptr;
			for (auto& Particle : SOAs.GetAllParticlesView())
			{
				Particle.SetX(FVec3(Count));
				if (Count == 2)
				{
					ThirdParticle = Particle.Handle();
				}

				Count += 1;
			}
			EXPECT_EQ(ThirdParticle->GetX()[0], 2);

			SOAs.DestroyParticle(ThirdParticle);
			//default behavior is swap dynamics at end
			Count = 0;
			for (auto& Particle : SOAs.GetAllParticlesView())
			{
				if (Count == 2)
				{
					EXPECT_EQ(Particle.GetX()[0], 9);
				}
				else
				{
					EXPECT_EQ(Particle.GetX()[0], Count);
				}

				Count += 1;
			}
		}

		//now test non swapping remove
		{
			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs SOAs(UniqueIndices);
			SOAs.CreateClusteredParticles(10);
			FReal Count = 0;
			FGeometryParticleHandle* ThirdParticle = nullptr;
			for (auto& Particle : SOAs.GetAllParticlesView())
			{
				Particle.SetX(FVec3(Count));
				if (Count == 2)
				{
					ThirdParticle = Particle.Handle();
				}

				Count += 1;
			}
			EXPECT_EQ(ThirdParticle->GetX()[0], 2);

			/*
			//For now we're just disabling removing clustered all together
			SOAs.DestroyParticle(ThirdParticle);
			//default behavior is swap dynamics at end
			Count = 0;
			for (auto& Particle : SOAs.GetAllParticlesView())
			{
				if (Count < 2)
				{
					EXPECT_EQ(Particle.X()[0], Count);
				}
				else
				{
					EXPECT_EQ(Particle.X()[0], Count+1);
				}

				Count += 1;
			}
			*/
		}
	}

	GTEST_TEST(WeakParticleHandle,BasicTest)
	{
		FWeakParticleHandle WeakHandle;
		{
			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs SOAs(UniqueIndices);
			SOAs.CreateStaticParticles(1);
			for(auto& Particle : SOAs.GetAllParticlesView())
			{
				WeakHandle = Particle.WeakParticleHandle();
				EXPECT_EQ(WeakHandle.GetHandleUnsafe(),Particle.Handle());
			}
		}

		//weak handle properly updated
		EXPECT_EQ(WeakHandle.GetHandleUnsafe(),nullptr);
	}

	void ParticleHandleTest()
	{
		{
			auto GeometryParticles = MakeUnique<FGeometryParticles>();
			GeometryParticles->AddParticles(1);

			auto KinematicGeometryParticles = MakeUnique<FKinematicGeometryParticles>();
			KinematicGeometryParticles->AddParticles(1);

			auto PBDRigidParticles = MakeUnique<FPBDRigidParticles>();
			PBDRigidParticles->AddParticles(1);
			
			auto PartialPBDRigids = MakeUnique<FPBDRigidParticles>();
			PartialPBDRigids->AddParticles(10);
			
			auto Geometry = FGeometryParticleHandle::CreateParticleHandle(MakeSerializable(GeometryParticles), 0, INDEX_NONE);

			auto KinematicGeometry = FKinematicGeometryParticleHandle::CreateParticleHandle(MakeSerializable(KinematicGeometryParticles), 0, INDEX_NONE);

			auto PBDRigid = FPBDRigidParticleHandle::CreateParticleHandle(MakeSerializable(PBDRigidParticles), 0, INDEX_NONE);

			ParticleHandleTestHelper(Geometry.Get(), static_cast<FKinematicGeometryParticleHandle*>(KinematicGeometry.Get()), static_cast<FPBDRigidParticleHandle*>(PBDRigid.Get()));

			//Test particle iterator
			{
				FGeometryParticleHandle* GeomHandles[] = { Geometry.Get(), KinematicGeometry.Get(), PBDRigid.Get() };
				TArray<TSOAView<FGeometryParticles>> SOAViews = { GeometryParticles.Get(), KinematicGeometryParticles.Get(), PBDRigidParticles.Get() };
				int32 Count = 0;
				for (auto Itr = MakeParticleIterator(SOAViews); Itr; ++Itr)
				{
					//set X back to 0 for all particles
					Itr->SetX(FVec3(0));
					EXPECT_EQ(Itr->Handle(), GeomHandles[Count]);
					//implicit const
					TConstParticleIterator<FGeometryParticles>& ConstItr = Itr;
					EXPECT_EQ(ConstItr->Handle(), GeomHandles[Count]);
					++Count;
				}

				for (auto Itr = MakeConstParticleIterator(SOAViews); Itr; ++Itr)
				{
					//check Xs are back to 0
					EXPECT_EQ(Itr->GetX()[0], 0);
				}

				Count = 0;
				for (auto Itr = MakeConstParticleIterator(SOAViews); Itr; ++Itr)
				{
					//check InvM for dynamics
					const FTransientPBDRigidParticleHandle* PBDRigid2 = Itr->CastToRigidParticle();
					if (PBDRigid2 && PBDRigid2->ObjectState() == EObjectStateType::Dynamic)
					{
						++Count;
						EXPECT_EQ(PBDRigid2->InvM(), 1);
						EXPECT_EQ(PBDRigid2->Handle(), PBDRigid.Get());
					}
				}
				EXPECT_EQ(Count, 1);
			}

			{
				TArray<TSOAView<FPBDRigidParticles>> SOAViews = { PBDRigidParticles.Get() };
				FPBDRigidParticleHandle* PBDRigidHandles[] = { static_cast<FPBDRigidParticleHandle*>(PBDRigid.Get()) };
				int32 Count = 0;
				for (auto Itr = MakeParticleIterator(MoveTemp(SOAViews)); Itr; ++Itr)
				{
					//set P to 1,1,1
					Itr->SetP(FVec3(1));
					EXPECT_EQ(Itr->Handle(), PBDRigidHandles[Count++]);
					EXPECT_EQ(Itr->Handle()->GetP()[0], Itr->GetP()[0]);	//handle type is deduced from iterator type
				}
				EXPECT_EQ(Count, 1);
			}

			//Use an SOA with an active list
			{
				FParticleUniqueIndicesMultithreaded UniqueIndices;
				FPBDRigidsSOAs SOAsWithHandles(UniqueIndices);	//todo: create a mock object so we can more easily create handles
				auto PartialDynamics = SOAsWithHandles.CreateDynamicParticles(10);

				TArray<FPBDRigidParticleHandle*> ActiveParticles = { PartialDynamics[3], PartialDynamics[5] };
				PartialDynamics[3]->SetX(FVec3(3));
				PartialDynamics[5]->SetX(FVec3(5));
				
				TArray<TSOAView<FPBDRigidParticles>> SOAViews = { PBDRigidParticles.Get(), &ActiveParticles, PBDRigidParticles.Get() };
				int32 Count = 0;
				for (auto Itr = MakeParticleIterator(MoveTemp(SOAViews)); Itr; ++Itr)
				{
					if (Count == 1)
					{
						EXPECT_EQ(Itr->GetX()[0], 3);
					}

					if (Count == 2)
					{
						EXPECT_EQ(Itr->GetX()[0], 5);
					}
					++Count;
				}
				EXPECT_EQ(Count, 4);
			}
		}

		{
			// try game thread representation
			auto Geometry = FGeometryParticle::CreateParticle();
			auto KinematicGeometry = FKinematicGeometryParticle::CreateParticle();
			auto PBDRigid = FPBDRigidParticle::CreateParticle();
			ParticleHandleTestHelper(Geometry.Get(), KinematicGeometry.Get(), PBDRigid.Get());
		}

		{
			// try using SOA manager
			FParticleUniqueIndicesMultithreaded UniqueIndices;
			FPBDRigidsSOAs SOAs(UniqueIndices);
			SOAs.CreateStaticParticles(3);
			auto KinematicParticles = SOAs.CreateKinematicParticles(3);
			SOAs.CreateDynamicParticles(3);

			EXPECT_EQ(SOAs.GetNonDisabledView().Num(), 9);

			//move to disabled
			FReal Count = 0;
			for (auto& Kinematic : KinematicParticles)
			{
				Kinematic->SetX(FVec3(Count));
				SOAs.DisableParticle(Kinematic);
				Count += 1;
			}

			EXPECT_EQ(SOAs.GetNonDisabledView().Num(), 6);

			//values are still set
			EXPECT_EQ(KinematicParticles[0]->GetX()[0], 0);
			EXPECT_EQ(KinematicParticles[1]->GetX()[0], 1);
			EXPECT_EQ(KinematicParticles[2]->GetX()[0], 2);

			//move to enabled
			for (auto& Kinematic : KinematicParticles)
			{
				SOAs.EnableParticle(Kinematic);
			}

			EXPECT_EQ(SOAs.GetNonDisabledView().Num(), 9);

			//destroy particle
			SOAs.DestroyParticle(KinematicParticles[0]);
			KinematicParticles[0] = nullptr;

			EXPECT_EQ(SOAs.GetNonDisabledView().Num(), 8);

			SOAs.DestroyParticle(KinematicParticles[2]);
			KinematicParticles[2] = nullptr;

			EXPECT_EQ(SOAs.GetNonDisabledView().Num(), 7);

			//disable some and then delete all
			SOAs.DisableParticle(KinematicParticles[1]);

			TArray<FGeometryParticleHandle*> ToDelete;
			for (auto& Particle : SOAs.GetAllParticlesView())	//todo: add check that iterator invalidates during delete
			{
				ToDelete.Add(Particle.Handle());
			}

			for (auto& Handle : ToDelete)
			{
				SOAs.DestroyParticle(Handle);
			}
			EXPECT_EQ(SOAs.GetNonDisabledView().Num(), 0);
		}

		ParticleLifetimeAndThreading();
		ParticleDestroyOrdering();
	}

	void AccelerationStructureHandleComparison()
	{
#if 0
		// When an external particle is created, the handle is retrieved via proxy.
		// Proxy gets handle initialized async on PT later, so this means a handle 
		// will always have external particle pointer, and eventually an internal pointer.
		// because of this, we must be able to compare (external, null) == (external, null) 
		// and also (external, null) == (external, internal)

		FPBDRigidsSOAs SOAs;

		auto GTParticle = FGeometryParticle::CreateParticle();
		//fake unique assignment like we would for solver
		GTParticle->SetUniqueIdx(SOAs.GetUniqueIndices().GenerateUniqueIdx());

		FAccelerationStructureHandle ExternalOnlyHandle(GTParticle.Get());

		FUniqueIdx Idx = GTParticle->UniqueIdx();
		auto Particles = SOAs.CreateStaticParticles(1, &Idx);
		FAccelerationStructureHandle ExternalInternalHandle(Particles[0], GTParticle.Get());

		FAccelerationStructureHandle InternalOnlyHandle(Particles[0], nullptr);

		FAccelerationStructureHandle NullHandle;

		EXPECT_EQ(ExternalOnlyHandle, ExternalInternalHandle);
		EXPECT_EQ(ExternalOnlyHandle, ExternalOnlyHandle);

		// Disabled because operator== ensures on null handle.
		/*EXPECT_EQ(NullHandle, NullHandle);
		EXPECT_NE(NullHandle, ExternalOnlyHandle);
		EXPECT_NE(NullHandle, ExternalInternalHandle);
		EXPECT_NE(NullHandle, InternalOnlyHandle);*/

		EXPECT_NE(InternalOnlyHandle, ExternalOnlyHandle);
		EXPECT_EQ(InternalOnlyHandle, ExternalInternalHandle);
#endif
	}

	void HandleObjectStateChangeTest()
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs SOAs(UniqueIndices);

		// Lambda for adding a particle to the dynamic-backed kinematic SOA
		const auto CreateDynamicKinematic = [&]()
		{
			TPBDRigidParticleHandle<FReal, 3>* Particle = SOAs.CreateDynamicParticles(1)[0];
			Particle->SetObjectStateLowLevel(EObjectStateType::Kinematic);
			SOAs.SetDynamicParticleSOA(Particle);
			return Particle;
		};

		// Create two dynamic kinematics, move one of them back to the dynamic SOA
		auto* Particle0 = CreateDynamicKinematic();
		auto* Particle1 = CreateDynamicKinematic();
		Particle0->SetObjectStateLowLevel(EObjectStateType::Dynamic);
		SOAs.SetDynamicParticleSOA(Particle0);

		// Ensure only one dynamic is in Active Particles
		auto ActiveIt = SOAs.GetActiveParticlesView().Begin();
		EXPECT_EQ(ActiveIt->ObjectState(), EObjectStateType::Dynamic);
		EXPECT_EQ(SOAs.GetActiveParticlesView().Num(), 1);

		// Ensure setting to kinematic removes it
		Particle0->SetObjectStateLowLevel(EObjectStateType::Kinematic);
		SOAs.SetDynamicParticleSOA(Particle0);
		EXPECT_EQ(SOAs.GetActiveParticlesView().Num(), 0);
	}
}