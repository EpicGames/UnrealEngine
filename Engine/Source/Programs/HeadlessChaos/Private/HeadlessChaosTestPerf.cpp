// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/Box.h"
#include "Chaos/Sphere.h"
#include "Chaos/Plane.h"
#include "Chaos/Utilities.h"
#include "Modules/ModuleManager.h"
#include "ChaosSolversModule.h"
#include "PBDRigidsSolver.h"
#include "GeometryCollection/GeometryCollectionTestFramework.h"

#include <chrono>

#define USE_CONSTRAINTS 1

namespace ChaosTest {

	using namespace Chaos;

	template <typename TEvolution>
	Chaos::FGeometryParticleHandle* AddFloor(TEvolution& Evolution)
	{
		auto Static = Evolution.CreateStaticParticles(1)[0];
		Static->SetX(FVec3(0, 0, 0));
		Static->SetGeometry(MakeImplicitObjectPtr<TPlane<FReal, 3>>(FVec3(0, 0, 0), FVec3(0, 0, 1)));
		return Static;
	}

	GTEST_TEST(AllEvolutions, DISABLED_PerfTests_Sim)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->SleepCounterThreshold = 2;
		auto Material = MakeSerializable(PhysicsMaterial);

#if USE_CONSTRAINTS
		FPBDJointConstraints Joints;
		Evolution.AddConstraintContainer(Joints);

		FPBDJointSettings JointSettings;
		const FReal TwistLimit = 0.;
		const FReal Swing1Limit = FMath::DegreesToRadians(30.0f);
		const FReal Swing2Limit = FMath::DegreesToRadians(45.0f);
		JointSettings.bCollisionEnabled = false;
		JointSettings.AngularLimits = { TwistLimit, Swing1Limit, Swing2Limit };
		JointSettings.bSoftTwistLimitsEnabled = false; // true;
		JointSettings.bSoftSwingLimitsEnabled = false; //true;
		JointSettings.AngularMotionTypes = { EJointMotionType::Limited, EJointMotionType::Limited , EJointMotionType::Limited };
		JointSettings.LinearMotionTypes = { EJointMotionType::Locked, EJointMotionType::Locked , EJointMotionType::Locked };
		JointSettings.AngularSoftForceMode = EJointForceMode::Force;
		JointSettings.SoftSwingStiffness = 100.;
		JointSettings.SoftSwingDamping = 0.;
		JointSettings.SoftTwistStiffness = 100.;
		JointSettings.SoftTwistDamping = 0.;
		//JointSettings.ParentInvMassScale = 0.; => produces a Nan in P() for the particle aligned with origin
#endif

		TArray<Chaos::FGeometryParticleHandle*> ParticleHandles;
		ParticleHandles.Add(AddFloor(Evolution));
		const int32 GridSize = 30;
		const FReal Interval = (FReal)GridSize;		
		const FReal Height = 2.;
		auto DynamicParticles = Evolution.CreateDynamicParticles(GridSize * GridSize * 2);

		const FVec3 HalfExtents(Interval * 0.30);
		Chaos::FImplicitObjectPtr Box(new TBox<FReal, 3>(-HalfExtents, HalfExtents));

		const FReal Radius = (Interval * 0.30);
		Chaos::FImplicitObjectPtr Sphere(new Chaos::FSphere(FVec3(0, 0, 0), Radius));

		const FVec3 Offset(0.5, 0.5, 0.); // => if (0,0,0) produces a Nan in P() for the particle aligned with origin
		int32 DynamicParticleIndex = 0;
		for (int32 x = 0; x < GridSize; ++x)
		{
			for (int32 y = 0; y < GridSize; ++y)
			{
				auto BoxDynamic = DynamicParticles[DynamicParticleIndex++];
				BoxDynamic->SetGeometry(Box);
				BoxDynamic->SetX(Offset + FVec3(x * Interval, y * Interval, Height));
				BoxDynamic->I() = TVec3<FRealSingle>(100000.f);
				BoxDynamic->InvI() = TVec3<FRealSingle>(1.f / 100000.f);
				Evolution.SetPhysicsMaterial(BoxDynamic, Material);
				ParticleHandles.Add(BoxDynamic);

				auto SphereDynamic = DynamicParticles[DynamicParticleIndex++];
				SphereDynamic->SetGeometry(Sphere);
				SphereDynamic->SetX(Offset + FVec3(x * Interval, y * Interval, Height * (FReal)0.5));
				SphereDynamic->I() = TVec3<FRealSingle>(100000.f);
				SphereDynamic->InvI() = TVec3<FRealSingle>(1.f / 100000.f);
				Evolution.SetPhysicsMaterial(SphereDynamic, Material);
				ParticleHandles.Add(SphereDynamic);

				Evolution.EnableParticle(BoxDynamic);
				Evolution.EnableParticle(SphereDynamic);

#if USE_CONSTRAINTS
				FVec3 JointLocation(x * Interval, y * Interval, Height * (FReal)0.75);
				FPBDJointConstraints::FConstraintContainerHandle* NewJoint = Joints.AddConstraint({ BoxDynamic, SphereDynamic }, FRigidTransform3(Offset + JointLocation, FRotation3::FromIdentity()));
				Joints.SetConstraintSettings(NewJoint->GetConstraintIndex(), JointSettings);
#endif
			}
		}

		::ChaosTest::SetParticleSimDataToCollide(ParticleHandles);


		// let's make this serial
		//bDisablePhysicsParallelFor = true;
		//bDisableParticleParallelFor = true;
		//bDisableCollisionParallelFor = true;

		double MinTime = DBL_MAX;
		double MaxTime = DBL_MIN;
		double TotalTime = 0.;

		const int NumSamples = 1;
		for (int Sample = 0; Sample < NumSamples; ++Sample)
		{
			auto start = std::chrono::high_resolution_clock::now();
			{
				const FReal Dt = (FReal)1. / (FReal)60.;
				const int NumFrames = 1000;
				for (int i = 0; i < NumFrames; ++i)
				{
					Evolution.AdvanceOneTimeStep(Dt);
					Evolution.EndFrame(Dt);
				}
			}
			auto finish = std::chrono::high_resolution_clock::now();
			std::chrono::duration<double> elapsed = finish - start;

			//UE_LOGF(LogChaos, Log, "[%d/%d] PerfTests_Sim (RealSize = %d)- Elapsed Time = %fs", Sample, NumSamples, sizeof(FReal), elapsed.count());

			TotalTime += elapsed.count();

			MinTime = std::min(MinTime, elapsed.count());
			MaxTime = std::max(MaxTime, elapsed.count());
		}

		const double AvgTime = TotalTime / (double)NumSamples;
		UE_LOGF(LogChaos, Log, "[SUMMARY] PerfTests_Sim (RealSize = %d) - Min Time = %fs | Avg Time = %fs | Max Time = %fs", sizeof(FReal), MinTime, AvgTime, MaxTime);
	}
}


