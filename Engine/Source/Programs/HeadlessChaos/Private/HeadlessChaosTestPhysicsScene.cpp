// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestUtility.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ErrorReporter.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Chaos/Utilities.h"
#include "PBDRigidsSolver.h"
#include "ChaosSolversModule.h"

#include "Modules/ModuleManager.h"
#include "Chaos/ChaosEngineInterface.h"
#include "Chaos/ChaosScene.h"
#include "SQAccelerator.h"
#include "CollisionQueryFilterCallbackCore.h"
#include "BodyInstanceCore.h"

namespace Chaos
{
	extern CHAOS_API float AsyncInterpolationMultiplier;
}

namespace ChaosTest {

	using namespace Chaos;
	using namespace ChaosInterface;

	// Returns true on raycast if we hit payload bounds.
	struct FSimpleRaycastVisitor: ISpatialVisitor<FAccelerationStructureHandle>
	{
		using FPayload = FAccelerationStructureHandle;
		FVec3 Start = FVec3::Zero();
		bool bHit = false;
		bool bQueryGameThread = true; // Query game thread or physics thread data?

		bool bUseQueryFilter = false;
		Chaos::Filter::FQueryFilterData FilterData;

		FSimpleRaycastVisitor(const FVec3& InStart, bool bInQueryGameThread)
			: Start(InStart)
			, bHit(false)
			, bQueryGameThread(bInQueryGameThread)
		{
		}

		FSimpleRaycastVisitor(const FVec3& InStart, const Chaos::Filter::FQueryFilterData& InFilterData, bool bInQueryGameThread)
			: Start(InStart)
			, bHit(false)
			, bQueryGameThread(bInQueryGameThread)
			, bUseQueryFilter(true)
			, FilterData(InFilterData)
		{
		}

		virtual bool PrePreFilter(const FAccelerationStructureHandle& Payload) const override
		{
			if (bUseQueryFilter)
			{
				return Payload.PrePreFilter(FilterData);
			}
			return false;
		}

		enum class SQType
		{
			Raycast,
			Sweep,
			Overlap
		};

		bool VisitRaycast(const TSpatialVisitorData<FPayload>& Data, FQueryFastData& CurData)
		{
			FReal OutTime = 0;
			FVec3 OutPos;
			FVec3 OutNorm;
			int32 FaceIdx;

			if (Data.Bounds.Raycast(Start, CurData.Dir, CurData.CurrentLength, 0, OutTime, OutPos, OutNorm, FaceIdx))
			{
				if (bQueryGameThread)
				{
					FTransform ParticleTransform(Data.Payload.GetExternalGeometryParticle_ExternalThread()->R(), Data.Payload.GetExternalGeometryParticle_ExternalThread()->GetX());
					const FVec3 DirLocal = ParticleTransform.InverseTransformVectorNoScale(CurData.Dir);
					const FVec3 StartLocal = ParticleTransform.InverseTransformPositionNoScale(Start);
					bHit = Data.Payload.GetExternalGeometryParticle_ExternalThread()->GetGeometry()->Raycast(StartLocal, DirLocal, CurData.CurrentLength, 0, OutTime, OutPos, OutNorm, FaceIdx);
				}
				else
				{
					FTransform ParticleTransform(Data.Payload.GetGeometryParticleHandle_PhysicsThread()->GetR(), Data.Payload.GetGeometryParticleHandle_PhysicsThread()->GetX());
					const FVec3 DirLocal = ParticleTransform.InverseTransformVectorNoScale(CurData.Dir);
					const FVec3 StartLocal = ParticleTransform.InverseTransformPositionNoScale(Start);
					bHit = Data.Payload.GetGeometryParticleHandle_PhysicsThread()->GetGeometry()->Raycast(StartLocal, DirLocal, CurData.CurrentLength, 0, OutTime, OutPos, OutNorm, FaceIdx);
				}

				if (bHit)
				{
					return false;
				}
			}

			return true;
		}

		bool VisitSweep(const TSpatialVisitorData<FPayload>& Data, FQueryFastData& CurData)
		{
			check(false);
			return false;
		}

		bool VisitOverlap(const TSpatialVisitorData<FPayload>& Data)
		{
			check(false);
			return false;
		}

		virtual bool Overlap(const TSpatialVisitorData<FPayload>& Instance) override
		{
			check(false);
			return false;
		}

		virtual bool Raycast(const TSpatialVisitorData<FPayload>& Instance, FQueryFastData& CurData) override
		{
			return VisitRaycast(Instance, CurData);
		}

		virtual bool Sweep(const TSpatialVisitorData<FPayload>& Instance, FQueryFastData& CurData) override
		{
			check(false);
			return false;
		}
	};

	FSQHitBuffer<ChaosInterface::FOverlapHit> InSphereHelper(const FChaosScene& Scene, const FTransform& InTM, const FReal Radius)
	{
		FChaosSQAccelerator SQAccelerator(*Scene.GetSpacialAcceleration());
		FSQHitBuffer<ChaosInterface::FOverlapHit> HitBuffer;
		FOverlapAllCollisionQueryFilterCallback QueryCallback;
		ChaosInterface::FSceneQueryCommonParams CommonParams(QueryCallback);
		SQAccelerator.Overlap(Chaos::FSphere(FVec3(0), Radius), InTM, HitBuffer, CommonParams);
		return HitBuffer;
	}

	bool PrePreFilter(const Chaos::Filter::FQueryFilterData& QueryFilter, const Chaos::Filter::FShapeFilterData& ShapeFilter)
	{
		Chaos::Filter::FShapeUnionFilterData ParticleFilter;
		ParticleFilter.Combine(ShapeFilter);
		return ParticleFilter.BroadFilter(QueryFilter) == Chaos::Filter::EBroadFilterResult::Reject;
	}

	GTEST_TEST(EngineInterface, CreateAndReleaseActor)
	{
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		EXPECT_NE(Proxy, nullptr);

		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Proxy->GetGameThreadAPI().SetGeometry(Sphere);
		}

		FChaosEngineInterface::ReleaseActor(Proxy, &Scene);
		EXPECT_EQ(Proxy, nullptr);
	}

	GTEST_TEST(EngineInterface, CreateMoveAndReleaseInScene)
	{
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		EXPECT_NE(Proxy, nullptr);

		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);

		//make sure acceleration structure has new actor right away
		{
			const auto HitBuffer = InSphereHelper(Scene, FTransform::Identity, 3);
			EXPECT_EQ(HitBuffer.GetNumHits(), 1);
		}

		//make sure acceleration structure sees moved actor right away
		const FTransform MovedTM(FQuat::Identity, FVec3(100, 0, 0));
		FChaosEngineInterface::SetGlobalPose_AssumesLocked(Proxy, MovedTM);
		{
			const auto HitBuffer = InSphereHelper(Scene, FTransform::Identity, 3);
			EXPECT_EQ(HitBuffer.GetNumHits(), 0);

			const auto HitBuffer2 = InSphereHelper(Scene, MovedTM, 3);
			EXPECT_EQ(HitBuffer2.GetNumHits(), 1);
		}

		//move actor back and acceleration structure sees it right away
		FChaosEngineInterface::SetGlobalPose_AssumesLocked(Proxy, FTransform::Identity);
		{
			const auto HitBuffer = InSphereHelper(Scene, FTransform::Identity, 3);
			EXPECT_EQ(HitBuffer.GetNumHits(), 1);
		}

		FChaosEngineInterface::ReleaseActor(Proxy, &Scene);
		EXPECT_EQ(Proxy, nullptr);

		//make sure acceleration structure no longer has actor
		{
			const auto HitBuffer = InSphereHelper(Scene, FTransform::Identity, 3);
			EXPECT_EQ(HitBuffer.GetNumHits(), 0);
		}

	}

	template <typename TSolver>
	void AdvanceSolverNoPushHelper(TSolver* Solver, FReal Dt)
	{
		Solver->AdvanceSolverBy(Dt);
	}

	GTEST_TEST(EngineInterface, AccelerationStructureHasSyncTimestamp)
	{
		//make sure acceleration structure has appropriate sync time

		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		EXPECT_EQ(Scene.GetSpacialAcceleration()->GetSyncTimestamp(), 0);	//timestamp of 0 because we flush when scene is created

		FReal TotalDt = 0;
		for (int Step = 1; Step < 10; ++Step)
		{
			FVec3 Grav(0,0,-1);
			Scene.SetUpForFrame(&Grav, 1,0,99999,99999,10,false);
			Scene.StartFrame();
			Scene.GetSolver()->GetEvolution()->FlushSpatialAcceleration();	//make sure we get a new tree every step
			Scene.EndFrame();

			EXPECT_EQ(Scene.GetSpacialAcceleration()->GetSyncTimestamp(), Step - 1);
		}
	}

	GTEST_TEST(EngineInterface, AccelerationStructureHasSyncTimestamp_MultiFrameDelay)
	{
		//make sure acceleration structure has appropriate sync time when PT falls behind GT

		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
		Scene.GetSolver()->SetStealAdvanceTasks_ForTesting(true); // prevents execution on StartFrame so we can execute task manually.

		EXPECT_EQ(Scene.GetSpacialAcceleration()->GetSyncTimestamp(), 0);	//timestamp of 0 because we flush when scene is created

		FVec3 Grav(0, 0, -1);
		Scene.SetUpForFrame(&Grav, 1, 0, 99999, 99999, 10, false);

		// Game thread enqueues second solver task before first completes (we did not execute advance task)
		Scene.StartFrame();
		Scene.EndFrame();
		Scene.StartFrame();

		// Execute first enqueued advance task
		Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();
		Scene.GetSolver()->GetEvolution()->FlushSpatialAcceleration();

		Scene.EndFrame();

		// Still timestamp 0, as we have only processed first PT step..
		EXPECT_EQ(Scene.GetSpacialAcceleration()->GetSyncTimestamp(), 0);

		Scene.StartFrame();

		// PT catches up during this frame
		Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();
		Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();
		Scene.GetSolver()->GetEvolution()->FlushSpatialAcceleration();
		Scene.EndFrame();

		// New structure should be at 2, 3 steps have been processed, PT/GT are in sync.
		EXPECT_EQ(Scene.GetSpacialAcceleration()->GetSyncTimestamp(), 2);

	}

	GTEST_TEST(EngineInterface, AccelerationStructureHasSyncTimestamp_MultiFrameDelay2)
	{
		//make sure acceleration structure has appropriate sync time when PT falls behind GT

		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
		Scene.GetSolver()->SetStealAdvanceTasks_ForTesting(true); // prevents execution on StartFrame so we can execute task manually.

		EXPECT_EQ(Scene.GetSpacialAcceleration()->GetSyncTimestamp(), 0);	//timestamp of 0 because we flush when scene is created

		FVec3 Grav(0,0,-1);
		Scene.SetUpForFrame(&Grav, 1,0,99999,99999,10,false);

		// PT not finished yet (we didn't execute solver task), should still be 0.
		Scene.StartFrame();
		Scene.EndFrame();
		EXPECT_EQ(Scene.GetSpacialAcceleration()->GetSyncTimestamp(), 0);

		// PT not finished yet (we didn't execute solver task), should still be 0.
		Scene.StartFrame();
		Scene.EndFrame();
		EXPECT_EQ(Scene.GetSpacialAcceleration()->GetSyncTimestamp(), 0);

		// First PT task finished this frame, we are two behind, still at 0 as structure is from first GT input (timestamp 0).
		Scene.StartFrame();
		Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();
		Scene.GetSolver()->GetEvolution()->FlushSpatialAcceleration();
		Scene.EndFrame();
		EXPECT_EQ(Scene.GetSpacialAcceleration()->GetSyncTimestamp(), 0);

		Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();
		Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();
		Scene.GetSolver()->GetEvolution()->FlushSpatialAcceleration();
		// Remaining two PT tasks finish, we are caught up, GT is still time 0 as EndFrame has not updated our structure.
		EXPECT_EQ(Scene.GetSpacialAcceleration()->GetSyncTimestamp(), 0);

		// Popping acceleration structures from physics thread will give us timestamp of 2. (3 total GT inputs processed)
		Scene.CopySolverAccelerationStructure();
		EXPECT_EQ(Scene.GetSpacialAcceleration()->GetSyncTimestamp(), 2);

		// PT task this frame finishes before EndFrame, putting us at 3, in sync with GT.
		Scene.StartFrame();
		Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();
		Scene.GetSolver()->GetEvolution()->FlushSpatialAcceleration();
		Scene.EndFrame();
		EXPECT_EQ(Scene.GetSpacialAcceleration()->GetSyncTimestamp(), 3);
	}

	GTEST_TEST(EngineInterface, PullFromPhysicsState_MultiFrameDelay)
	{
		// This test is designed to verify pulldata is being timestamped correctly, and that we will not write to a deleted GT Proxy 
		// in this case. 

		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
		Scene.GetSolver()->SetStealAdvanceTasks_ForTesting(true); // prevents execution on StartFrame so we can execute task manually.

		FVec3 Grav(0,0,-1);
		Scene.SetUpForFrame(&Grav, 1,0,99999,99999,10,false);

		FActorCreationParams Params;
		Params.Scene = &Scene;
		Params.bSimulatePhysics = true;
		Params.bEnableGravity = true;
		Params.bStartAwake = true;


		// Create two Proxys, one to remove for test, the other to ensure we have > 0 proxies to hit the pull physics data path.
		FPhysicsActorHandle Proxy = nullptr;
		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		EXPECT_NE(Proxy, nullptr);
		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}
		FPhysicsActorHandle Proxy2 = nullptr;
		FChaosEngineInterface::CreateActor(Params, Proxy2);
		auto& Particle2 = Proxy2->GetGameThreadAPI();
		EXPECT_NE(Proxy2, nullptr);
		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle2.SetGeometry(Sphere);
		}
		TArray<FPhysicsActorHandle> Proxys = { Proxy, Proxy2 };
		Scene.AddActorsToScene_AssumesLocked(Proxys);

		// verify external timestamps are as expected.
		auto& MarshallingManager = Scene.GetSolver()->GetMarshallingManager();
		EXPECT_EQ(MarshallingManager.GetExternalTimestamp_External(), 0);

		// Execute a frame such that Proxys should be initialized in physics thread and game thread.
		Scene.StartFrame();
		EXPECT_EQ(MarshallingManager.GetExternalTimestamp_External(), 1);
		Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();
		Scene.EndFrame();

		// run GT frame, no PT task executed.
		Scene.StartFrame();
		EXPECT_EQ(MarshallingManager.GetExternalTimestamp_External(), 2);
		Scene.EndFrame();

		// enqueue another frame.
		Scene.StartFrame();
		EXPECT_EQ(MarshallingManager.GetExternalTimestamp_External(), 3);

		// Remove Proxy, is stamped with external time 3. PT needs to run 3 frames before this will be removed,
		// as we are two PT tasks behind, and this has not been enqueued yet.
		auto StaleProxy = Proxy;
		FChaosEngineInterface::ReleaseActor(Proxy, &Scene);
		EXPECT_EQ(Proxy, nullptr);
		EXPECT_EQ(StaleProxy->GetSyncTimestamp()->bDeleted, true);

		// Run PT task for internal timestamp 1.
		Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();

		// Proxy should not get touched in Pull, as timestamp from removal should be greater than pulldata timestamp.
		// (if it was touched we'd crash as it is now deleted).
		Scene.EndFrame();


		Scene.StartFrame();
		EXPECT_EQ(MarshallingManager.GetExternalTimestamp_External(), 4);
		EXPECT_EQ(StaleProxy->GetSyncTimestamp()->bDeleted, true);

		// run pt task for internal timestamp 3. Proxy still not removed on PT.
		Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();
		EXPECT_EQ(Scene.GetSolver()->GetEvolution()->GetParticles().GetAllParticlesView().Num(), 2); // none have been removed on pt, still 2 Proxys.

		// Proxy should not get touched in pull, as timestamp from removal is less than pulldata timestamp (3 < 4)
		// If this crashes in pull, that means this test has regressed. (Pulldata timestamp is likely wrong).
		Scene.EndFrame();


		Scene.StartFrame();
		EXPECT_EQ(MarshallingManager.GetExternalTimestamp_External(), 5);
		EXPECT_EQ(StaleProxy->GetSyncTimestamp()->bDeleted, true);
		EXPECT_EQ(Scene.GetSolver()->GetEvolution()->GetParticles().GetAllParticlesView().Num(), 2); // Proxys not yet removed on pt, still 2.


		// This is PT task that should remove Proxy (internal timestamp 4, matching stamp on removed Proxy's dirty data).
		Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();
		EXPECT_EQ(Scene.GetSolver()->GetEvolution()->GetParticles().GetAllParticlesView().Num(), 1); // one Proxy removed on pt, one remaining.

		// This PT task catches up to gamethread.
		Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();
		Scene.EndFrame();
	}

	GTEST_TEST(EngineInterface, UpdatingAccelerationStructurePrePreFilterOnShapeFilterChange)
	{
		const float PhysicsTimestep = 1; // 1 second
		FChaosScene Scene(nullptr, PhysicsTimestep);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		float DeltaSeconds = PhysicsTimestep;
		FVec3 Grav(0, 0, -1);
		Scene.SetUpForFrame(&Grav, DeltaSeconds, 0, 9999, 9999, 9999, false);

		// Raycast params, aimed to hit our particle at (0,0,0)
		const FVector Start(0, 0, -5);
		const FVector Dir(0, 0, 1);
		const float Length = 50;

		// Init kinematic particle, sphere radius 3
		FActorCreationParams Params;
		Params.Scene = &Scene;
		Params.bSimulatePhysics = false;
		Params.bEnableGravity = true;
		Params.bStartAwake = true;
		FPhysicsActorHandle Proxy = nullptr;
		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}
		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);

		// Execute a whole frame such that particle is initialized on physics thread
		Scene.StartFrame();
		Scene.EndFrame();

		// Make a query filter that will block against a particle that blocks all channels but fails on the default particle.
		// The channel index of 7 is arbitrary.
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		Chaos::Filter::FQueryTraceFilterBuilder Builder;
		Builder.SetCollisionChannelIndex(7);
		const Chaos::Filter::FQueryFilterData QueryFilterData = Builder.Build();
		PRAGMA_ENABLE_INTERNAL_WARNINGS

		// Get collision data off shape
		for (const TUniquePtr<Chaos::FPerShapeData>& Shape : Particle.ShapesArray())
		{
			const Chaos::Filter::FShapeFilterData& ShapeFilter = Shape->GetShapeFilterData();
			// Verify query is filtered out with default collision data on shape
			bool bFiltered = PrePreFilter(QueryFilterData, ShapeFilter);
			EXPECT_EQ(bFiltered, true);
		}

		// Query against particle on game thread, should fail to hit due to particle filter being defaulted, no touch/block set.
		{
			bool bQueryGameThread = true;
			FSimpleRaycastVisitor Visitor(Start, QueryFilterData, bQueryGameThread);
			Scene.GetSpacialAcceleration()->Raycast(Start, Dir, Length, Visitor);
			EXPECT_EQ(Visitor.bHit, false);
		}

		// Change filter data on game thread to contain touch/block on all channels.
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		Chaos::Filter::FShapeFilterBuilder ShapeFilterBuilder;
		ShapeFilterBuilder.SetBlockChannelMask(TNumericLimits<uint64>::Max());
		ShapeFilterBuilder.SetOverlapChannelMask(TNumericLimits<uint64>::Max());
		ShapeFilterBuilder.SetFilterFlags(Chaos::EFilterFlags::All);
		const Chaos::Filter::FShapeFilterData NewParticleQueryFilter = ShapeFilterBuilder.Build();
		PRAGMA_ENABLE_INTERNAL_WARNINGS

		for (const TUniquePtr<Chaos::FPerShapeData>& Shape : Particle.ShapesArray())
		{
			// Update filter
			Shape->SetShapeFilterData(NewParticleQueryFilter);

			// Filter with new data, ensuring we pass and are not filtered out.
			bool bFiltered = PrePreFilter(QueryFilterData, NewParticleQueryFilter);
			EXPECT_EQ(bFiltered, false);
		}

		// Update particle in GT accel structure so cached PrePreFilter updates
		Scene.UpdateActorInAccelerationStructure(Proxy);

		// Query against particle on game thread, should hit with new filter.
		{
			bool bQueryGameThread = true;
			FSimpleRaycastVisitor Visitor(Start, QueryFilterData, bQueryGameThread);
			Scene.GetSpacialAcceleration()->Raycast(Start, Dir, Length, Visitor);
			EXPECT_EQ(Visitor.bHit, true);
		}

		// Tick to push to physics thread
		Scene.StartFrame();
		Scene.EndFrame();

		// Query particle on physics thread, expected to hit with new filter.
		// If this fails it means we did not update cached filter data in acceleration structure entry.
		{
			bool bQueryGameThread = false;
			FSimpleRaycastVisitor Visitor(Start, QueryFilterData, bQueryGameThread);
			Scene.GetSolver()->GetEvolution()->GetSpatialAcceleration()->Raycast(Start, Dir, Length, Visitor);
			EXPECT_EQ(Visitor.bHit, true);
		}
	}
	
	// Disabled until we move fix with kineamtic bounds update on PushToPhysicsState into this branch. Might also need to remove bounds computation in ApplyKinematicTarget.
	GTEST_TEST(EngineInterface, DISABLED_KinematicTargetsPassingGTWrongAccelBoundsBeforeHittingTarget)
	{
		// This test is designed to catch an edge case with kinematic targets (or other things interpolated over multiple physics steps), and acceleration structure bounds.
		// Timestep is setup such that 1 GT frame = 10 physics steps, we have to make sure that if a non-final step gives an acceleration structure to game thread, in which
		// kinematic has not reached target yet, that the bounds in structure representing interpolated position do not make it to game thread, otherwise game thread has
		// position at target, but bounds that don't match.

		const float PhysicsTimestep = 1; // 1 second

		// Setup solver so we can manually execute each physics step.
		FChaosScene Scene(nullptr, PhysicsTimestep);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
		Scene.GetSolver()->SetStealAdvanceTasks_ForTesting(true);

		// In this test we have a 10s Dt, split into 10 physics steps of 1s.
		const int32 PhysicsStepsInFrame = 10; 
		float DeltaSeconds = PhysicsTimestep * PhysicsStepsInFrame;
		FVec3 Grav(0, 0, -1);

		Scene.SetUpForFrame(&Grav, DeltaSeconds, 0, 9999, 9999, 9999, false);
		
		// Raycast params, aimed to hit our kinematic target (10,0,0)
		const FVector Start(10, 0, -5);
		const FVector Dir(0, 0,1);
		const float Length = 50;

		// Init kinematic particle, sphere radius 3
		FActorCreationParams Params;
		Params.Scene = &Scene;
		Params.bSimulatePhysics = false;
		Params.bEnableGravity = true;
		Params.bStartAwake = true;
		FPhysicsActorHandle Proxy = nullptr;
		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}
		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);

		// Execute a whole frame such that particle is initialized on physics thread
		Scene.StartFrame();
		for (int32 PhysicsTicks = 0; PhysicsTicks < PhysicsStepsInFrame; ++PhysicsTicks)
		{
			// Tick each physics step generated from game thread input
			Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();
		}
		Scene.EndFrame();

		// Set kinematic target to (10,0,0) on game thread
		FTransform Target(FVector(10, 0, 0));
		FChaosEngineInterface::SetKinematicTarget_AssumesLocked(Proxy, Target);

		// Confirm particle is at target on game thread with raycast.
		{
			FSimpleRaycastVisitor Visitor(Start, true);
			Scene.GetSpacialAcceleration()->Raycast(Start, Dir, Length, Visitor);
			EXPECT_EQ(Visitor.bHit, true);
		}
		
	

		// Tick game thread again, this enqueues 10 physics steps, kinematic will interpolate
		// to target on physics thread over duration of these 10 steps.
		Scene.StartFrame();



		for (int32 PhysicsTick = 0; PhysicsTick < PhysicsStepsInFrame; ++PhysicsTick)
		{

			Scene.GetSolver()->PopAndExecuteStolenAdvanceTask_ForTesting();

			if (PhysicsTick == 2)
			{
				// On this arbritrary tick, copy acceleration structure to game thread,
				// at this point we have sim'd only some of the physics steps for this frame.
				// kinematic target is still interpolating, has not reached target of (10,0,0) yet.
				// When this was broken this would give game thread a structure with
				// the bounds of interpolated position (which is wrong because game thread particle is at target!)
				Scene.CopySolverAccelerationStructure();

				// Verify the game thread particle can still be queried at target (verifying bounds and particle position are still correct)
				{
					FSimpleRaycastVisitor Visitor(Start, true);
					Scene.GetSpacialAcceleration()->Raycast(Start, Dir, Length, Visitor);
					EXPECT_EQ(Visitor.bHit, true);
				}
			}

		}

		// Finish frame
		Scene.EndFrame();

		// Verify can still query game thread particle at our target.
		{
			FSimpleRaycastVisitor Visitor(Start, true);
			Scene.GetSpacialAcceleration()->Raycast(Start, Dir, Length, Visitor);
			EXPECT_EQ(Visitor.bHit, true);
		}
	}

	GTEST_TEST(EngineInterface, CreateActorPostFlush)
	{
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		EXPECT_NE(Proxy, nullptr);

		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		//tick solver but don't call EndFrame (want to flush and swap manually)
		{
			FVec3 Grav(0,0,-1);
			Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
			Scene.StartFrame();
		}

		//make sure acceleration structure is built
		Scene.GetSolver()->GetEvolution()->FlushSpatialAcceleration();

		//create actor after structure is finished, but before swap happens
		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);

		Scene.CopySolverAccelerationStructure();	//trigger swap manually and see pending changes apply
		{
			const auto HitBuffer = InSphereHelper(Scene, FTransform::Identity, 3);
			EXPECT_EQ(HitBuffer.GetNumHits(), 1);
		}
	}

	GTEST_TEST(EngineInterface, MoveActorPostFlush)
	{
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		EXPECT_NE(Proxy, nullptr);

		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		//create actor before structure is ticked
		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);

		//tick solver so that Proxy is created, but don't call EndFrame (want to flush and swap manually)
		{
			FVec3 Grav(0,0,-1);
			Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
			Scene.StartFrame();
		}

		//make sure acceleration structure is built
		Scene.GetSolver()->GetEvolution()->FlushSpatialAcceleration();

		//move object to get hit (shows pending move is applied)
		FChaosEngineInterface::SetGlobalPose_AssumesLocked(Proxy, FTransform(FRotation3::FromIdentity(), FVec3(100, 0, 0)));

		Scene.CopySolverAccelerationStructure();	//trigger swap manually and see pending changes apply
		{
			TRigidTransform<FReal, 3> OverlapTM(FVec3(100, 0, 0), FRotation3::FromIdentity());
			const auto HitBuffer = InSphereHelper(Scene, OverlapTM, 3);
			EXPECT_EQ(HitBuffer.GetNumHits(), 1);
		}
	}

	GTEST_TEST(EngineInterface, RemoveActorPostFlush)
	{
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		EXPECT_NE(Proxy, nullptr);

		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		//create actor before structure is ticked
		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);

		//tick solver so that Proxy is created, but don't call EndFrame (want to flush and swap manually)
		{
			FVec3 Grav(0,0,-1);
			Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
			Scene.StartFrame();
		}

		//make sure acceleration structure is built
		Scene.GetSolver()->GetEvolution()->FlushSpatialAcceleration();

		//delete object to get no hit
		FChaosEngineInterface::ReleaseActor(Proxy, &Scene);

		Scene.CopySolverAccelerationStructure();	//trigger swap manually and see pending changes apply
		{
			const auto HitBuffer = InSphereHelper(Scene, FTransform::Identity, 3);
			EXPECT_EQ(HitBuffer.GetNumHits(), 0);
		}
	}

	GTEST_TEST(EngineInterface, RemoveActorPostFlush0Dt)
	{
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		EXPECT_NE(Proxy, nullptr);

		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		//create actor before structure is ticked
		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);

		//tick solver so that Proxy is created, but don't call EndFrame (want to flush and swap manually)
		{
			//use 0 dt to make sure pending operations are not sensitive to 0 dt
			FVec3 Grav(0,0,-1);
			Scene.SetUpForFrame(&Grav,0,0,99999,99999,10,false);
			Scene.StartFrame();
		}

		//make sure acceleration structure is built
		Scene.GetSolver()->GetEvolution()->FlushSpatialAcceleration();

		//delete object to get no hit
		FChaosEngineInterface::ReleaseActor(Proxy, &Scene);

		Scene.CopySolverAccelerationStructure();	//trigger swap manually and see pending changes apply
		{
			const auto HitBuffer = InSphereHelper(Scene, FTransform::Identity, 3);
			EXPECT_EQ(HitBuffer.GetNumHits(), 0);
		}
	}

	GTEST_TEST(EngineInterface, CreateAndRemoveActorPostFlush)
	{
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		//tick solver, but don't call EndFrame (want to flush and swap manually)
		{
			FVec3 Grav(0,0,-1);
			Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
			Scene.StartFrame();
		}

		//make sure acceleration structure is built
		Scene.GetSolver()->GetEvolution()->FlushSpatialAcceleration();

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		EXPECT_NE(Proxy, nullptr);

		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		//create actor after flush
		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);

		//delete object right away to get no hit
		FChaosEngineInterface::ReleaseActor(Proxy, &Scene);

		Scene.CopySolverAccelerationStructure();	//trigger swap manually and see pending changes apply
		{
			const auto HitBuffer = InSphereHelper(Scene, FTransform::Identity, 3);
			EXPECT_EQ(HitBuffer.GetNumHits(), 0);
		}
	}

	GTEST_TEST(EngineInterface, CreateDelayed)
	{
		for (int Delay = 0; Delay < 4; ++Delay)
		{
			FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
			Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
			Scene.GetSolver()->GetMarshallingManager().SetTickDelay_External(Delay);

			FActorCreationParams Params;
			Params.Scene = &Scene;

			FPhysicsActorHandle Proxy = nullptr;

			FChaosEngineInterface::CreateActor(Params, Proxy);
			auto& Particle = Proxy->GetGameThreadAPI();
			EXPECT_NE(Proxy, nullptr);

			{
				auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
				Particle.SetGeometry(Sphere);
			}

			//create actor after flush
			TArray<FPhysicsActorHandle> Proxys = { Proxy };
			Scene.AddActorsToScene_AssumesLocked(Proxys);

			for (int Repeat = 0; Repeat < Delay; ++Repeat)
			{
				//tick solver
				{
					FVec3 Grav(0,0,-1);
					Scene.SetUpForFrame(&Grav,1,0,99999,99999,1,false);
					Scene.StartFrame();
					Scene.EndFrame();
				}

				//make sure sim hasn't seen it yet
				{
					FPBDRigidsEvolution* Evolution = Scene.GetSolver()->GetEvolution();
					const auto& SOA = Evolution->GetParticles();
					EXPECT_EQ(SOA.GetAllParticlesView().Num(), 0);
				}

				//make sure external thread knows about it
				{
					const auto HitBuffer = InSphereHelper(Scene, FTransform::Identity, 3);
					EXPECT_EQ(HitBuffer.GetNumHits(), 1);
				}
			}

			//tick solver one last time
			{
				FVec3 Grav(0,0,-1);
				Scene.SetUpForFrame(&Grav,1,0,99999,99999,1,false);
				Scene.StartFrame();
				Scene.EndFrame();
			}

			//now sim knows about it
			{
				FPBDRigidsEvolution* Evolution = Scene.GetSolver()->GetEvolution();
				const auto& SOA = Evolution->GetParticles();
				EXPECT_EQ(SOA.GetAllParticlesView().Num(), 1);
			}

			Particle.SetX(FVec3(5, 0, 0));

			for (int Repeat = 0; Repeat < Delay; ++Repeat)
			{
				//tick solver
				{
					FVec3 Grav(0,0,-1);
					Scene.SetUpForFrame(&Grav,1,0,99999,99999,1,false);
					Scene.StartFrame();
					Scene.EndFrame();
				}

				//make sure sim hasn't seen new X yet
				{
					FPBDRigidsEvolution* Evolution = Scene.GetSolver()->GetEvolution();
					const auto& SOA = Evolution->GetParticles();
					const auto& InternalProxy = *SOA.GetAllParticlesView().Begin();
					EXPECT_EQ(InternalProxy.GetX()[0], 0);
				}
			}

			//tick solver one last time
			{
				FVec3 Grav(0,0,-1);
				Scene.SetUpForFrame(&Grav,1,0,99999,99999,1,false);
				Scene.StartFrame();
				Scene.EndFrame();
			}

			//now sim knows about new X
			{
				FPBDRigidsEvolution* Evolution = Scene.GetSolver()->GetEvolution();
				const auto& SOA = Evolution->GetParticles();
				const auto& InternalProxy = *SOA.GetAllParticlesView().Begin();
				EXPECT_EQ(InternalProxy.GetX()[0], 5);
			}

			//make sure commands are also deferred

			int Count = 0;
			int ExternalCount = 0;
			TUniqueFunction<void()> Lambda = [&]()
			{
				++Count;
				EXPECT_EQ(Count, 1);	//only hit once on internal thread
				EXPECT_EQ(ExternalCount, Delay); //internal hits with expected delay
			};

			Scene.GetSolver()->EnqueueCommandImmediate(Lambda);

			for (int Repeat = 0; Repeat < Delay + 1; ++Repeat)
			{
				//tick solver
				FVec3 Grav(0,0,-1);
				Scene.SetUpForFrame(&Grav,1,0,99999,99999,1,false);
				Scene.StartFrame();
				Scene.EndFrame();

				++ExternalCount;
			}

		}

	}

	GTEST_TEST(EngineInterface, RemoveDelayed)
	{
		for (int Delay = 0; Delay < 4; ++Delay)
		{
			FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
			Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
			Scene.GetSolver()->GetMarshallingManager().SetTickDelay_External(Delay);

			FActorCreationParams Params;
			Params.Scene = &Scene;

			Params.bSimulatePhysics = true;	//simulate so that sync body is triggered
			Params.bStartAwake = true;

			FPhysicsActorHandle Proxy = nullptr;
			FChaosEngineInterface::CreateActor(Params, Proxy);
			auto& Particle = Proxy->GetGameThreadAPI();
			EXPECT_NE(Proxy, nullptr);

			{
				auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
				Particle.SetGeometry(Sphere);
				Particle.SetV(FVec3(0, 0, -1));
			}


			//make second simulating Proxy that we don't delete. Needed to trigger a sync
			//this is because some data is cleaned up on GT immediately
			FPhysicsActorHandle Proxy2 = nullptr;
			FChaosEngineInterface::CreateActor(Params, Proxy2);
			auto& Particle2 = Proxy2->GetGameThreadAPI();
			EXPECT_NE(Proxy2, nullptr);
			{
				auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
				Particle2.SetGeometry(Sphere);
				Particle2.SetV(FVec3(0, -1, 0));
			}

			//create actor
			TArray<FPhysicsActorHandle> Proxys = { Proxy, Proxy2 };
			Scene.AddActorsToScene_AssumesLocked(Proxys);

			//tick until it's being synced from sim
			for (int Repeat = 0; Repeat < Delay; ++Repeat)
			{
				{
					FVec3 Grav(0,0,0);
					Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
					Scene.StartFrame();
					Scene.EndFrame();
				}
			}

			//x starts at 0
			EXPECT_NEAR(Particle.X()[2], 0, 1e-4);
			EXPECT_NEAR(Particle2.X()[1], 0, 1e-4);

			//tick solver and see new position synced from sim
			{
				FVec3 Grav(0,0,0);
				Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
				Scene.StartFrame();
				Scene.EndFrame();
				EXPECT_NEAR(Particle.X()[2], -1, 1e-4);
				EXPECT_NEAR(Particle2.X()[1], -1, 1e-4);
			}

			//tick solver and delete in between solver finishing and sync
			{
				FVec3 Grav(0,0,0);
				Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
				Scene.StartFrame();

				//delete Proxy
				FChaosEngineInterface::ReleaseActor(Proxy, &Scene);

				Scene.EndFrame();
				EXPECT_NEAR(Particle2.X()[1], -2, 1e-4);	//other Proxy keeps moving
			}


			//tick again and don't crash
			for (int Repeat = 0; Repeat < Delay + 1; ++Repeat)
			{
				{
					FVec3 Grav(0,0,0);
					Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
					Scene.StartFrame();
					Scene.EndFrame();
					EXPECT_NEAR(Particle2.X()[1], -3 - Repeat, 1e-4);	//other Proxy keeps moving
				}
			}
		}
	}

	GTEST_TEST(EngineInterface, MoveDelayed)
	{
		for (int Delay = 0; Delay < 4; ++Delay)
		{
			FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
			Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
			Scene.GetSolver()->GetMarshallingManager().SetTickDelay_External(Delay);

			FActorCreationParams Params;
			Params.Scene = &Scene;

			Params.bSimulatePhysics = true;	//simulated so that gt conflicts with sim thread
			Params.bStartAwake = true;

			FPhysicsActorHandle Proxy = nullptr;
			FChaosEngineInterface::CreateActor(Params, Proxy);
			auto& Particle = Proxy->GetGameThreadAPI();
			EXPECT_NE(Proxy, nullptr);

			{
				auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
				Particle.SetGeometry(Sphere);
				Particle.SetV(FVec3(0, 0, -1));
			}

			//create actor
			TArray<FPhysicsActorHandle> Proxys = { Proxy };
			Scene.AddActorsToScene_AssumesLocked(Proxys);

			//tick until it's being synced from sim
			for (int Repeat = 0; Repeat < Delay; ++Repeat)
			{
				{
					FVec3 Grav(0,0,0);
					Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
					Scene.StartFrame();
					Scene.EndFrame();
				}
			}

			//x starts at 0
			EXPECT_NEAR(Particle.X()[2], 0, 1e-4);

			//tick solver and see new position synced from sim
			{
				FVec3 Grav(0,0,0);
				Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
				Scene.StartFrame();
				Scene.EndFrame();
				EXPECT_NEAR(Particle.X()[2], -1, 1e-4);
			}

			//set new x position and make sure we see it right away even though there's delay
			FChaosEngineInterface::SetGlobalPose_AssumesLocked(Proxy, FTransform(FQuat::Identity, FVec3(0, 0, 10)));

			for (int Repeat = 0; Repeat < Delay; ++Repeat)
			{
				{
					FVec3 Grav(0,0,0);
					Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
					Scene.StartFrame();
					Scene.EndFrame();

					EXPECT_NEAR(Particle.X()[2], 10, 1e-4);	//until we catch up, just use GT data
				}
			}

			//tick solver one last time, should see sim results from the place we teleported to
			{
				FVec3 Grav(0,0,0);
				Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
				Scene.StartFrame();
				Scene.EndFrame();
				EXPECT_NEAR(Particle.X()[2], 9, 1e-4);
			}

			//set x after sim but before EndFrame, make sure to see gt position since it was written after
			{
				FVec3 Grav(0,0,0);
				Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
				Scene.StartFrame();
				FChaosEngineInterface::SetGlobalPose_AssumesLocked(Proxy, FTransform(FQuat::Identity, FVec3(0, 0, 100)));
				Scene.EndFrame();
				EXPECT_NEAR(Particle.X()[2], 100, 1e-4);
			}

			for (int Repeat = 0; Repeat < Delay; ++Repeat)
			{
				{
					FVec3 Grav(0,0,0);
					Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
					Scene.StartFrame();
					Scene.EndFrame();

					EXPECT_NEAR(Particle.X()[2], 100, 1e-4);	//until we catch up, just use GT data
				}
			}

			//tick solver one last time, should see sim results from the place we teleported to
			{
				FVec3 Grav(0,0,0);
				Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
				Scene.StartFrame();
				Scene.EndFrame();
				EXPECT_NEAR(Particle.X()[2], 99, 1e-4);
			}
		}
	}

	GTEST_TEST(EngineInterface, SimRoundTrip)
	{
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);
		Particle.SetObjectState(EObjectStateType::Dynamic);
		Particle.AddForce(FVec3(0, 0, 10) * Particle.M());

		FVec3 Grav(0,0,0);
		Scene.SetUpForFrame(&Grav,1,0,99999,99999,10,false);
		Scene.StartFrame();
		Scene.EndFrame();

		//integration happened and we get results back
		EXPECT_EQ(Particle.X(), FVec3(0, 0, 10));
		EXPECT_EQ(Particle.V(), FVec3(0, 0, 10));

	}

	GTEST_TEST(EngineInterface, SimInterpolated)
	{
		//Need to test:
		//position interpolation
		//position interpolation from an inactive Proxy (i.e a step function)
		//position interpolation from an active to an inactive Proxy (i.e a step function but reversed)
		//interpolation to a deleted Proxy
		//state change should be a step function (sleep state)
		//wake events must be collapsed (sleep awake sleep becomes sleep)
		//collision events must be collapsed
		//forces are averaged
		const FReal FixedDT = 1;
		FChaosScene Scene(nullptr, FixedDT);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);


		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;
		FPhysicsActorHandle Proxy2 = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		Params.bSimulatePhysics = true;
		FChaosEngineInterface::CreateActor(Params, Proxy2);
		auto& Particle2 = Proxy2->GetGameThreadAPI();
		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle2.SetGeometry(Sphere);
		}

		TArray<FPhysicsActorHandle> Proxys = { Proxy, Proxy2 };
		Scene.AddActorsToScene_AssumesLocked(Proxys);
		Particle.SetObjectState(EObjectStateType::Dynamic);
		const FReal ZVel = 10;
		const FReal ZStart = 100;
		const FVec3 ConstantForce(0, 0, 1 * Particle2.M());
		Particle.SetV(FVec3(0, 0, ZVel));
		Particle.SetX(FVec3(0, 0, ZStart));
		const int32 NumGTSteps = 24;
		const int32 NumPTSteps = 24 / 4;

		struct FCallback : public TSimCallbackObject<FSimCallbackNoInput>
		{
			virtual void OnPreSimulate_Internal() override
			{
				EXPECT_EQ(GetConsumerInput_Internal(), nullptr);	//no inputs passed in
				//we expect the dt to be 1
				EXPECT_EQ(GetDeltaTime_Internal(), 1);
				EXPECT_EQ(GetSimTime_Internal(), Count);
				Count++;
			}

			int32 Count = 0;

			int32 NumPTSteps;
		};

		auto Callback = Scene.GetSolver()->CreateAndRegisterSimCallbackObject_External<FCallback>();
		Callback->NumPTSteps = NumPTSteps;
		FReal Time = 0;
		const FReal GTDt = FixedDT * 0.25f;
		for (int32 Step = 0; Step < NumGTSteps; Step++)
		{
			//set force every external frame
			Particle2.AddForce(ConstantForce);
			FVec3 Grav(0, 0, 0);
			Scene.SetUpForFrame(&Grav, GTDt, 0, 99999, 99999, 1, false);
			Scene.StartFrame();
			Scene.EndFrame();

			Time += GTDt;
			const FReal InterpolatedTime = Time - FixedDT * Chaos::AsyncInterpolationMultiplier;
			const FReal ExpectedVFromForce = Time;
			if (InterpolatedTime < 0)
			{
				//not enough time to interpolate so just take initial value
				EXPECT_NEAR(Particle.X()[2], ZStart, 1e-2);
				EXPECT_NEAR(Particle2.V()[2], 0, 1e-2);
			}
			else
			{
				//interpolated
				EXPECT_NEAR(Particle.X()[2], ZStart + ZVel * InterpolatedTime, 1e-2);
				EXPECT_NEAR(Particle2.V()[2], InterpolatedTime, 1e-2);
			}
		}

		EXPECT_EQ(Callback->Count, NumPTSteps);
		const FReal LastInterpolatedTime = NumGTSteps * GTDt - FixedDT * Chaos::AsyncInterpolationMultiplier;
		EXPECT_NEAR(Particle.X()[2], ZStart + ZVel * LastInterpolatedTime, 1e-2);
		EXPECT_NEAR(Particle.V()[2], ZVel, 1e-2);
	}

	void ExpectVectorEqual(const FVec3& V0, const FVec3& V1)
	{
		EXPECT_EQ(V0.X, V1.X);
		EXPECT_EQ(V0.Y, V1.Y);
		EXPECT_EQ(V0.Z, V1.Z);
	}

	void TestKinematicTarget(const bool bInUpdateKinematicFromSimulation)
	{
		// Need to test:
		// GT particle position is immediately updated after calling SetKinematicTarget_AssumesLocked
		// GT particle positions and velocities are correctly updated
		// PT particle positions and velocities are correctly updated
		// Velocity becomes zero if no KinematicTarget is set in the current frame
		// Particle positions and velocities are correct after SetKinematicTarget_AssumesLocked, SetKinematicTarget_AssumesLocked
		// Velocity is zero if only SetGlobalPose_AssumesLocked is called (Teleport)
		// Particle positions and velocities are correct after SetGlobalPose_AssumesLocked, SetKinematicTarget_AssumesLocked (Teleport)
		// Particle positions and velocities are correct after SetKinematicTarget_AssumesLocked, SetGlobalPose_AssumesLocked (Teleport, KinematicTarget is cleared)
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);
		Particle.SetObjectState(EObjectStateType::Kinematic);
		Particle.SetUpdateKinematicFromSimulation(bInUpdateKinematicFromSimulation);

		struct FDummyInput : FSimCallbackInput
		{
			FSingleParticlePhysicsProxy* Proxy;
			FVec3 CorrectX;
			FVec3 CorrectV;
			bool bKinematicWritebackEnabled;
			void Reset() {}
		};

		struct FCallback : public TSimCallbackObject<FDummyInput>
		{
			virtual void OnPreSimulate_Internal() override
			{
				const FVec3 ExpectedX = GetConsumerInput_Internal()->CorrectX;
				const FVec3 ExpectedV = GetConsumerInput_Internal()->CorrectV;

				auto Handle = GetConsumerInput_Internal()->Proxy->GetPhysicsThreadAPI();
				ExpectVectorEqual(Handle->X(), ExpectedX);
				ExpectVectorEqual(Handle->V(), ExpectedV);
			}
		};

		auto Callback = Scene.GetSolver()->CreateAndRegisterSimCallbackObject_External<FCallback>();

		Callback->GetProducerInputData_External()->Proxy = Proxy;
		Callback->GetProducerInputData_External()->bKinematicWritebackEnabled = bInUpdateKinematicFromSimulation;

		FVec3 Grav(0, 0, 0);
		float Dt = 1;

		auto AdvanceFrameAndRunTest = [&](const FVec3 &CorrectX, const FVec3 &CorrectV)
		{
			Scene.SetUpForFrame(&Grav, Dt, 0, 99999, 99999, 10, false);
			Scene.StartFrame();
			Scene.EndFrame();
			// Test X and V on GT
			// NOTE: GT velocity will not be updated if kinematic writeback from the physics thread is disabled
			ExpectVectorEqual(Particle.X(), CorrectX);
			if (Callback->GetProducerInputData_External()->bKinematicWritebackEnabled)
			{
				ExpectVectorEqual(Particle.V(), CorrectV);
			}
			// Test X and V on PT, this is going to be used in OnPreSimulate_Internal in next frame.
			Callback->GetProducerInputData_External()->CorrectX = CorrectX;
			Callback->GetProducerInputData_External()->CorrectV = CorrectV;
		};

		// Set initial transform
		FVec3 CurrentX = FVec3(1, 2, 3);
		FVec3 CurrentV = FVec3(0, 0, 0);
		FChaosEngineInterface::SetGlobalPose_AssumesLocked(Proxy, FTransform(CurrentX));

		Callback->GetProducerInputData_External()->CorrectX = CurrentX;
		Callback->GetProducerInputData_External()->CorrectV = CurrentV;
		AdvanceFrameAndRunTest(CurrentX, CurrentV);

		// Test SetKinematicTarget_AssumesLocked
		CurrentX = FVec3(2, 3, 4);
		CurrentV = FVec3(1, 1, 1);
		FChaosEngineInterface::SetKinematicTarget_AssumesLocked(Proxy, FTransform(CurrentX));

		// Test if position is immediately updated on GT after SetKinematicTarget_AssumesLocked (if we aren't reading data back from PT)
		if (!bInUpdateKinematicFromSimulation)
		{
			ExpectVectorEqual(Particle.X(), CurrentX);
		}

		// This will fail when bInUpdateKinematicFromSimulation is false becasuse GT and PT disagree on velocity
		AdvanceFrameAndRunTest(CurrentX, CurrentV);

		// Test if velocity becomes zero when no kinematic target is set
		CurrentX = FVec3(2, 3, 4);
		CurrentV = FVec3(0, 0, 0);

		AdvanceFrameAndRunTest(CurrentX, CurrentV);

		// Test if particle positions and velocities are correct after SetKinematicTarget_AssumesLocked, SetKinematicTarget_AssumesLocked
		CurrentX = FVec3(0, 0, 0);
		CurrentV = FVec3(-2, -3, -4);
		FChaosEngineInterface::SetKinematicTarget_AssumesLocked(Proxy, FTransform(FVec3(1, 2, 3)));
		FChaosEngineInterface::SetKinematicTarget_AssumesLocked(Proxy, FTransform(CurrentX));

		AdvanceFrameAndRunTest(CurrentX, CurrentV);

		// Test if velocity is zero if only SetGlobalPose_AssumesLocked is called (Teleport)
		CurrentX = FVec3(0, 0, 0);
		CurrentV = FVec3(0, 0, 0);
		FChaosEngineInterface::SetGlobalPose_AssumesLocked(Proxy, FTransform(CurrentX));

		Callback->GetProducerInputData_External()->CorrectX = CurrentX;
		Callback->GetProducerInputData_External()->CorrectV = CurrentV;
		AdvanceFrameAndRunTest(CurrentX, CurrentV);

		// Test if particle positions and velocities are correct after SetGlobalPose_AssumesLocked, SetKinematicTarget_AssumesLocked
		CurrentX = FVec3(-1, -2, -3);
		CurrentV = FVec3(0, 0, 0);
		FChaosEngineInterface::SetGlobalPose_AssumesLocked(Proxy, FTransform(CurrentX));
		FChaosEngineInterface::SetKinematicTarget_AssumesLocked(Proxy, FTransform(CurrentX));

		Callback->GetProducerInputData_External()->CorrectX = CurrentX;
		AdvanceFrameAndRunTest(CurrentX, CurrentV);

		// Test if particle state to sleeping change after setting a kinematic target it's position and velocity should remain the same
		FChaosEngineInterface::SetKinematicTarget_AssumesLocked(Proxy, FTransform(FVec3(1, 2, 3)));
		Particle.SetObjectState(EObjectStateType::Sleeping);
		AdvanceFrameAndRunTest(CurrentX, CurrentV);

		// Test if particle positions and velocities are correct after SetKinematicTarget_AssumesLocked, SetGlobalPose_AssumesLocked
		CurrentX = FVec3(3, 2, 1);
		CurrentV = FVec3(0, 0, 0);
		FChaosEngineInterface::SetKinematicTarget_AssumesLocked(Proxy, FTransform(CurrentX));
		FChaosEngineInterface::SetGlobalPose_AssumesLocked(Proxy, FTransform(CurrentX));

		Callback->GetProducerInputData_External()->CorrectX = CurrentX;
		AdvanceFrameAndRunTest(CurrentX, CurrentV);

		// Test if the PT positions and velocities are right from previous frame
		CurrentX = FVec3(3, 2, 1);
		CurrentV = FVec3(0, 0, 0);
		AdvanceFrameAndRunTest(CurrentX, CurrentV);
	}

	// Test SetKinematicTarget when writeback from PT is enabled
	GTEST_TEST(EngineInterface, SetKinematicTargetWriteBackEnabled)
	{
		TestKinematicTarget(true);
	}

	// Test SetKinematicTarget when writeback from PT is disabled
	GTEST_TEST(EngineInterface, SetKinematicTargetWriteBackDisabled)
	{
		TestKinematicTarget(false);
	}

	GTEST_TEST(EngineInterface, PerPropertySetOnGT)
	{
		//Need to test:
		//setting transform, velocities, wake state, on external thread means we overwrite results until sim catches up
		//deleted proxy does not incorrectly update after it's deleted on gt
		const FReal FixedDT = 1;
		FChaosScene Scene(nullptr, FixedDT);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
		Scene.GetSolver()->EnableAsyncMode(1);	//tick 1 dt at a time

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);
		Particle.SetObjectState(EObjectStateType::Dynamic);
		const FReal ZVel = 10;
		const FReal ZStart = 100;
		Particle.SetV(FVec3(0, 0, ZVel));
		Particle.SetX(FVec3(0, 0, ZStart));
		const int32 NumGTSteps = 100;
		const FVec3 TeleportLocation(5, 5, ZStart);

		FReal Time = 0;
		const FReal GTDt = FixedDT * 0.5f;
		const int32 ChangeVelStep = 20;
		const FReal ChangeVelTime = ChangeVelStep * GTDt;
		const FReal YVelAfterChange = 10;
		const int32 TeleportStep = 10;
		const FReal TeleportTime = TeleportStep * GTDt;
		bool bHasTeleportedOnGT = false;
		bool bVelHasChanged = false;
		bool bWasPutToSleep = false;
		bool bWasWoken = false;
		const int32 SleepStep = 50;
		const int32 WakeStep = 70;
		const FReal PutToSleepTime = SleepStep * GTDt;
		const FReal WokenTime = WakeStep * GTDt;
		FReal SleepZPosition(0);

		for (int32 Step = 0; Step < NumGTSteps; Step++)
		{
			if (Step == TeleportStep)
			{
				Particle.SetX(TeleportLocation);
				bHasTeleportedOnGT = true;
			}

			if(Step == ChangeVelStep)
			{
				Particle.SetV(FVec3(0, YVelAfterChange, ZVel));
				bVelHasChanged = true;
			}

			if(Step == SleepStep)
			{
				bWasPutToSleep = true;
				Particle.SetObjectState(EObjectStateType::Sleeping);
				SleepZPosition = Particle.X()[2];	//record position when gt wants to sleep
			}

			if(Step == WakeStep)
			{
				bWasWoken = true;
				Particle.SetV(FVec3(0, YVelAfterChange, ZVel));
				Particle.SetObjectState(EObjectStateType::Dynamic);
			}

			FVec3 Grav(0, 0, 0);
			Scene.SetUpForFrame(&Grav, GTDt, 0, 99999, 99999, 10, false);
			Scene.StartFrame();
			Scene.EndFrame();

			Time += GTDt;
			const FReal InterpolatedTime = Time - FixedDT * Chaos::AsyncInterpolationMultiplier;
			if (InterpolatedTime < 0)
			{
				//not enough time to interpolate so just take initial value
				EXPECT_NEAR(Particle.X()[2], ZStart, 1e-2);
			}
			else
			{
				//interpolated
				if(bHasTeleportedOnGT)
				{
					EXPECT_NEAR(Particle.X()[0], TeleportLocation[0], 1e-2);	//X never changes so as soon as gt teleports we should see it

					//if we haven't caught up to teleport, we just use the value set on GT for z value
					if(InterpolatedTime < TeleportTime)
					{
						EXPECT_NEAR(Particle.X()[2], TeleportLocation[2], 1e-3);
					}
					else
					{
						if(!bWasPutToSleep)
						{
							//caught up so expect normal movement to marshal back
							EXPECT_NEAR(Particle.X()[2], TeleportLocation[2] + ZVel * (InterpolatedTime - TeleportTime), 1e-2);
						}
						else if(InterpolatedTime < WokenTime)
						{
							//currently asleep so position is held constant
							EXPECT_NEAR(Particle.X()[2], SleepZPosition, 1e-2);
							if(!bWasWoken)
							{
								EXPECT_NEAR(Particle.V()[2], 0, 1e-2);
							}
							else
							{
								EXPECT_NEAR(Particle.V()[2], ZVel, 1e-2);
							}
						}
						else
						{
							//woke back up so position is moving again
							EXPECT_NEAR(Particle.X()[2], SleepZPosition + ZVel * (InterpolatedTime - WokenTime), 1e-2);
							EXPECT_NEAR(Particle.V()[2], ZVel, 1e-2);
						}
						
					}
				}
				else
				{
					EXPECT_NEAR(Particle.X()[2], ZStart + ZVel * InterpolatedTime, 1e-2);
				}

				if(bVelHasChanged)
				{
					if(!bWasPutToSleep || bWasWoken)
					{
						EXPECT_EQ(Particle.V()[1], YVelAfterChange);
					}
					else
					{
						//asleep so velocity is 0
						EXPECT_EQ(Particle.V()[1], 0);
					}
				}
				else
				{
					EXPECT_EQ(Particle.V()[1], 0);
				}

				if(bWasPutToSleep && !bWasWoken)
				{
					EXPECT_EQ(Particle.ObjectState(), EObjectStateType::Sleeping);
				}
				else
				{
					EXPECT_EQ(Particle.ObjectState(), EObjectStateType::Dynamic);
				}
			}
		}

		const FReal LastInterpolatedTime = NumGTSteps * GTDt - FixedDT * Chaos::AsyncInterpolationMultiplier;
		EXPECT_EQ(Particle.V()[2], ZVel);
		EXPECT_EQ(Particle.V()[1], YVelAfterChange);
	}

	GTEST_TEST(EngineInterface, IterationSetOnGT)
	{
		//Need to test:
		//Before set iteration on game thread the iterations are default (8, 2, 1)
		//After set iteration on game thread the data goes through.
		FPBDPositionConstraints SinglePositionConstraint;
		const FReal FixedDT = 1;
		FChaosScene Scene(nullptr, FixedDT);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		FRigidBodyHandle_External& Particle = Proxy->GetGameThreadAPI();
		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);
		Particle.SetObjectState(EObjectStateType::Dynamic);
		Particle.SetGravityEnabled(true);

		FReal Time = 0;
		const FReal GTDt = FixedDT * 4;
		const int32 BigIteration = 100;

		FVec3 Grav(0, 0, -1);
		Scene.SetUpForFrame(&Grav, GTDt, 0, 99999, 99999, 10, false);
		Scene.StartFrame();
		Scene.EndFrame();
		Chaos::Private::FIterationSettings GroupIterations = Scene.GetSolver()->GetEvolution()->GetIslandGroupManager().GetIslandGroupIterations(0);

		EXPECT_EQ(GroupIterations.GetNumPositionIterations(), 8);
		EXPECT_EQ(GroupIterations.GetNumVelocityIterations(), 2);
		EXPECT_EQ(GroupIterations.GetNumProjectionIterations(), 1);
		
		auto FirstHandle = Scene.GetSolver()->GetEvolution()->GetParticleHandles().Handle(0)->CastToRigidParticle();
		TArray<FPBDRigidParticleHandle*> Dynamics = { FirstHandle };
		TArray<FVec3> Positions = { FVec3(1) };
		FPBDPositionConstraints PositionConstraints(MoveTemp(Positions), MoveTemp(Dynamics), 1.f);
		SinglePositionConstraint = PositionConstraints;
		Scene.GetSolver()->GetEvolution()->AddConstraintContainer(SinglePositionConstraint);

		Particle.SetPositionSolverIterations(BigIteration);
		Particle.SetVelocitySolverIterations(BigIteration);
		Particle.SetProjectionSolverIterations(BigIteration);

		Scene.StartFrame();
		Scene.EndFrame();
		Chaos::Private::FIterationSettings GroupIterationsTemp = Scene.GetSolver()->GetEvolution()->GetIslandGroupManager().GetIslandGroupIterations(0);
		EXPECT_EQ(GroupIterationsTemp.GetNumPositionIterations(), 100);
		EXPECT_EQ(GroupIterationsTemp.GetNumVelocityIterations(), 100);
		EXPECT_EQ(GroupIterationsTemp.GetNumProjectionIterations(), 100);
	}

	GTEST_TEST(EngineInterface, FlushCommand)
	{
		//Need to test:
		//flushing commands works and sees state changes for both fixed dt and not
		//sim callback is not called

		bool bHitOnShutDown = false;
		{
			FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
			Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
			Scene.GetSolver()->EnableAsyncMode(1);	//tick 1 dt at a time

			FActorCreationParams Params;
			Params.Scene = &Scene;

			FPhysicsActorHandle Proxy = nullptr;

			FChaosEngineInterface::CreateActor(Params, Proxy);
			auto& Particle = Proxy->GetGameThreadAPI();
			{
				auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
				Particle.SetGeometry(Sphere);
			}

			TArray<FPhysicsActorHandle> Proxys = { Proxy };
			Scene.AddActorsToScene_AssumesLocked(Proxys);
			Particle.SetX(FVec3(0, 0, 3));

			Scene.GetSolver()->EnqueueCommandImmediate([Proxy]()
				{
					//sees change immediately
					EXPECT_EQ(Proxy->GetPhysicsThreadAPI()->X()[2], 3);
				});

			struct FCallback : public TSimCallbackObject<>
			{
				virtual void OnPreSimulate_Internal() override
				{
					EXPECT_FALSE(true);	//this should never hit
				}
			};

			auto Callback = Scene.GetSolver()->CreateAndRegisterSimCallbackObject_External<FCallback>();

			FVec3 Grav(0, 0, 0);
			Scene.SetUpForFrame(&Grav, 0, 0, 99999, 99999, 10, false);	//flush with dt 0
			Scene.StartFrame();
			Scene.EndFrame();

			Scene.GetSolver()->EnqueueCommandImmediate([&bHitOnShutDown]()
				{
					//command enqueued and then solver shuts down, so flush must happen
					bHitOnShutDown = true;
				});
		}

		EXPECT_TRUE(bHitOnShutDown);
	}

	GTEST_TEST(EngineInterface, SimSubstep)
	{
		//Need to test:
		//forces and torques are extrapolated (i.e. held constant for sub-steps)
		//kinematic targets are interpolated over the sub-step
		//identical inputs are given to sub-steps

		const FReal FixedDT = 1;
		FChaosScene Scene(nullptr, FixedDT);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);
		Particle.SetObjectState(EObjectStateType::Dynamic);
		Particle.SetGravityEnabled(true);

		struct FDummyInput : FSimCallbackInput
		{
			int32 ExternalFrame;
			void Reset() {}
		};

		struct FCallback : public TSimCallbackObject<FDummyInput>
		{
			virtual void OnPreSimulate_Internal() override
			{
				EXPECT_EQ(GetConsumerInput_Internal()->ExternalFrame, ExpectedFrame);
				EXPECT_NEAR(GetSimTime_Internal(), InternalSteps * GetDeltaTime_Internal(), 1e-2);	//sim start is changing per sub-step
				++InternalSteps;
			}

			int32 ExpectedFrame;
			int32 InternalSteps = 0;
		};

		auto Callback = Scene.GetSolver()->CreateAndRegisterSimCallbackObject_External<FCallback>();

		FReal Time = 0;
		const FReal GTDt = FixedDT * 4;
		for (int32 Step = 0; Step < 10; Step++)
		{
			Callback->ExpectedFrame = Step;
			Callback->GetProducerInputData_External()->ExternalFrame = Step;	//make sure input matches for all sub-steps

			//set force every external frame
			Particle.AddForce(FVec3(0, 0, 1 * Particle.M()));	//should counteract gravity
			FVec3 Grav(0, 0, -1);
			Scene.SetUpForFrame(&Grav, GTDt, 0, 99999, 99999, 10, false);
			Scene.StartFrame();
			Scene.EndFrame();

			Time += GTDt;

			//should have no movement because forces cancel out
			EXPECT_NEAR(Particle.X()[2], 0, 1e-2);
			EXPECT_NEAR(Particle.V()[2], 0, 1e-2);
		}
	}

	GTEST_TEST(EngineInterface, SimDestroyedProxy)
	{
		//Need to test:
		//destroyed proxy still valid in callback, but Proxy is nulled out
		//valid for multiple sub-steps

		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);
		Scene.GetSolver()->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
		const FReal FixedDT = 1;
		Scene.GetSolver()->EnableAsyncMode(FixedDT);	//tick 1 dt at a time

		FActorCreationParams Params;
		Params.Scene = &Scene;

		FPhysicsActorHandle Proxy = nullptr;

		FChaosEngineInterface::CreateActor(Params, Proxy);
		auto& Particle = Proxy->GetGameThreadAPI();
		{
			auto Sphere = MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), 3);
			Particle.SetGeometry(Sphere);
		}

		TArray<FPhysicsActorHandle> Proxys = { Proxy };
		Scene.AddActorsToScene_AssumesLocked(Proxys);

		struct FDummyInput : FSimCallbackInput
		{
			FSingleParticlePhysicsProxy* Proxy;
			void Reset() {}
		};

		struct FCallback : public TSimCallbackObject<FDummyInput>
		{
			virtual void OnPreSimulate_Internal() override
			{
				EXPECT_EQ(GetConsumerInput_Internal()->Proxy->GetHandle_LowLevel(), nullptr);
			}
		};

		auto Callback = Scene.GetSolver()->CreateAndRegisterSimCallbackObject_External<FCallback>();

		Callback->GetProducerInputData_External()->Proxy = Proxy;
		Scene.GetSolver()->UnregisterObject(Proxy);

		FVec3 Grav(0, 0, -1);
		Scene.SetUpForFrame(&Grav, FixedDT * 3, 0, 99999, 99999, 10, false);
		Scene.StartFrame();
		Scene.EndFrame();
	}
	
	GTEST_TEST(EngineInterface, OverlapOffsetActor)
	{
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);

		FActorCreationParams Params;
		Params.Scene = &Scene;
		Params.bSimulatePhysics = false;
		Params.bStatic = true;
		Params.InitialTM = FTransform::Identity;
		Params.Scene = &Scene;

		FPhysicsActorHandle StaticCube = nullptr;

		FChaosEngineInterface::CreateActor(Params, StaticCube);
		ASSERT_NE(StaticCube, nullptr);

		// Add geometry, placing a box at the origin
		constexpr FReal BoxSize = static_cast<FReal>(50.0);
		const FVec3 HalfBoxExtent{ BoxSize };

		// We require a union here, although the second geometry isn't used we need the particle to
		// have more than one shape in its shapes array otherwise the query acceleration will treat
		// it as a special case and skip bounds checking during the overlap
		TArray<Chaos::FImplicitObjectPtr> Geoms;
		Geoms.Emplace(MakeImplicitObjectPtr<TBox<FReal, 3>>(-HalfBoxExtent, HalfBoxExtent));
		Geoms.Emplace(MakeImplicitObjectPtr<TBox<FReal, 3>>(-HalfBoxExtent, HalfBoxExtent));

		auto& Particle = StaticCube->GetGameThreadAPI();
		{
			Chaos::FImplicitObjectPtr GeomUnion = MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(Geoms));
			Particle.SetGeometry(GeomUnion);
		}
		
		TArray<FPhysicsActorHandle> Particles{ StaticCube };
		Scene.AddActorsToScene_AssumesLocked(Particles);

		FChaosSQAccelerator SQ{ *Scene.GetSpacialAcceleration() };
		FSQHitBuffer<ChaosInterface::FOverlapHit> HitBuffer;
		FOverlapAllCollisionQueryFilterCallback QueryCallback;
		ChaosInterface::FSceneQueryCommonParams CommonParams(QueryCallback);

		// Here we query from a position under the box, but using a shape that has an offset. This tests
		// a failure case that was previously present where the query system assumed that the QueryTM
		// was inside the geometry being used to query.
		const FTransform QueryTM{ FVec3{0.0f, 0.0f, -110.0f} };
		constexpr FReal SphereRadius = static_cast<FReal>(50.0);
		SQ.Overlap(Chaos::FSphere(FVec3(0.0f, 0.0f, 100.0f), SphereRadius), QueryTM, HitBuffer, CommonParams);

		EXPECT_TRUE(HitBuffer.HasBlockingHit());
	}

	GTEST_TEST(EngineInterface, SweepOffsetActor)
	{
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);

		FActorCreationParams Params;
		Params.Scene = &Scene;
		Params.bSimulatePhysics = false;
		Params.bStatic = true;
		Params.InitialTM = FTransform::Identity;
		Params.Scene = &Scene;

		FPhysicsActorHandle StaticCube = nullptr;

		FChaosEngineInterface::CreateActor(Params, StaticCube);
		ASSERT_NE(StaticCube, nullptr);

		// Add geometry, placing a box at the origin
		constexpr FReal BoxSize = static_cast<FReal>(50.0);
		const FVec3 HalfBoxExtent{ BoxSize };

		// We require a union here, although the second geometry isn't used we need the particle to
		// have more than one shape in its shapes array otherwise the query acceleration will treat
		// it as a special case and skip bounds checking during the overlap
		TArray<Chaos::FImplicitObjectPtr> Geoms;
		Geoms.Emplace(MakeImplicitObjectPtr<TBox<FReal, 3>>(-HalfBoxExtent, HalfBoxExtent));
		Geoms.Emplace(MakeImplicitObjectPtr<TBox<FReal, 3>>(-HalfBoxExtent, HalfBoxExtent));

		auto& Particle = StaticCube->GetGameThreadAPI();
		{
			Chaos::FImplicitObjectPtr GeomUnion = MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(Geoms));
			Particle.SetGeometry(GeomUnion);
		}

		TArray<FPhysicsActorHandle> Particles{ StaticCube };
		Scene.AddActorsToScene_AssumesLocked(Particles);

		FChaosSQAccelerator SQ{ *Scene.GetSpacialAcceleration() };
		FSQHitBuffer<ChaosInterface::FSweepHit> HitBuffer;
		FBlockAllCollisionQueryFilterCallback QueryCallback;
		ChaosInterface::FSceneQueryCommonParams CommonParams(QueryCallback);

		// Another box of same size that is offset from origin by 200.
		FVec3 Offset(200.f,0,0);
		TBox<FReal, 3> QueryBox(-HalfBoxExtent + Offset, HalfBoxExtent + Offset);

		// Sweep positions offset box directly above box at origin, should hit box sweeping downward.
		const FTransform QueryTM{ FVec3{-200.f, 0, 100.0f} };
		const FVec3 Dir(0,0,-1);
		const FReal Length = 200;
		SQ.Sweep(QueryBox, QueryTM, Dir, Length, HitBuffer, EHitFlags::None, CommonParams);

		EXPECT_TRUE(HitBuffer.HasBlockingHit());
	}

	GTEST_TEST(EngineInterface, BodyWithTwoShapes_SweepWithInitialOverlap_MinimalTOIIsReturned)
	{
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);

		FActorCreationParams Params;
		Params.Scene = &Scene;
		Params.bSimulatePhysics = false;
		Params.bStatic = true;
		Params.InitialTM = FTransform::Identity;
		Params.Scene = &Scene;

		FPhysicsActorHandle Actor0 = nullptr;

		FChaosEngineInterface::CreateActor(Params, Actor0);
		ASSERT_NE(Actor0, nullptr);

		TArray<Chaos::FImplicitObjectPtr> Geoms;
		Geoms.Emplace(MakeImplicitObjectPtr<TBox<FReal, 3>>(FVec3(-50, -50, -50), FVec3(0, 50, 50)));
		Geoms.Emplace(MakeImplicitObjectPtr<TBox<FReal, 3>>(FVec3(0, -50, -50), FVec3(50, 50, 50)));

		FRigidBodyHandle_External& Particle = Actor0->GetGameThreadAPI();
		{
			Chaos::FImplicitObjectPtr GeomUnion = MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(Geoms));
			Particle.SetGeometry(GeomUnion);
		}

		TArray<FPhysicsActorHandle> Particles{ Actor0 };
		Scene.AddActorsToScene_AssumesLocked(Particles);

		FChaosSQAccelerator SQ{ *Scene.GetSpacialAcceleration() };
		FSQHitBuffer<ChaosInterface::FSweepHit> HitBuffer;
		FBlockAllCollisionQueryFilterCallback QueryCallback;
		ChaosInterface::FSceneQueryCommonParams CommonParams(QueryCallback);

		const FReal SphereRadius = 25;
		const Chaos::FSphere QuerySphere(FVec3::ZeroVector, SphereRadius);

		// Sweep down over both shapes with an initial overlap, however center over shape 0 more.
		FTransform QueryTM{ FVec3{-5, 0, 50} };
		const FVec3 Dir(0,0,-1);
		const FReal Length = 200;
		SQ.Sweep(QuerySphere, QueryTM, Dir, Length, HitBuffer, EHitFlags::MTD, CommonParams);

		EXPECT_TRUE(HitBuffer.HasBlockingHit());
		const FSweepHit* BlockingHit = HitBuffer.GetBlock(); 
		EXPECT_EQ(BlockingHit->Shape, Particle.ShapesArray()[0].Get());
		EXPECT_EQ(BlockingHit->Distance, -25);
		EXPECT_VECTOR_NEAR(BlockingHit->WorldPosition, FVector(-5, 0, 50), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(BlockingHit->WorldNormal, FVector(0, 0, 1), KINDA_SMALL_NUMBER);

		// Now do a second sweep down, this time over shape 1 more.
		QueryTM = FTransform{ FVec3{5, 0, 50} };
		SQ.Sweep(QuerySphere, QueryTM, Dir, Length, HitBuffer, EHitFlags::MTD, CommonParams);

		EXPECT_TRUE(HitBuffer.HasBlockingHit());
		BlockingHit = HitBuffer.GetBlock();
		EXPECT_EQ(BlockingHit->Shape, Particle.ShapesArray()[1].Get());
		EXPECT_EQ(BlockingHit->Distance, -25);
		EXPECT_VECTOR_NEAR(BlockingHit->WorldPosition, FVector(5, 0, 50), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(BlockingHit->WorldNormal, FVector(0, 0, 1), KINDA_SMALL_NUMBER);
	}

	GTEST_TEST(EngineInterface, TwoBodies_SweepWithInitialOverlap_MinimalTOIIsReturned)
	{
		FChaosScene Scene(nullptr, /*AsyncDt=*/-1);

		FActorCreationParams Params;
		Params.Scene = &Scene;
		Params.bSimulatePhysics = false;
		Params.bStatic = true;
		Params.InitialTM = FTransform::Identity;
		Params.Scene = &Scene;

		auto CreateActor = [](const FActorCreationParams& Params, const FImplicitObjectPtr& Geometry) -> FPhysicsActorHandle
		{
			FPhysicsActorHandle ActorHandle = nullptr;
			FChaosEngineInterface::CreateActor(Params, ActorHandle);
			FRigidBodyHandle_External& Particle = ActorHandle->GetGameThreadAPI();
			Particle.SetGeometry(Geometry);
			return ActorHandle;
		};
		
		FImplicitObjectPtr Box0 = MakeImplicitObjectPtr<TBox<FReal, 3>>(FVec3(-50, -50, -50), FVec3(0, 50, 50));
		FPhysicsActorHandle Actor0 = CreateActor(Params, Box0);
		ASSERT_NE(Actor0, nullptr);
		FSingleParticlePhysicsProxy* Actor0Proxy = Actor0->GetGameThreadAPI().GetProxy();
		
		FImplicitObjectPtr Box1 = MakeImplicitObjectPtr<TBox<FReal, 3>>(FVec3(0, -50, -50), FVec3(50, 50, 50));
		FPhysicsActorHandle Actor1 = CreateActor(Params, Box1);
		ASSERT_NE(Actor1, nullptr);
		FSingleParticlePhysicsProxy* Actor1Proxy = Actor1->GetGameThreadAPI().GetProxy();

		TArray<FPhysicsActorHandle> Particles{ Actor0, Actor1 };
		Scene.AddActorsToScene_AssumesLocked(Particles);

		FChaosSQAccelerator SQ{ *Scene.GetSpacialAcceleration() };
		FSQHitBuffer<ChaosInterface::FSweepHit> HitBuffer;
		FBlockAllCollisionQueryFilterCallback QueryCallback;
		ChaosInterface::FSceneQueryCommonParams CommonParams(QueryCallback);

		const FReal SphereRadius = 25;
		const Chaos::FSphere QuerySphere(FVec3::ZeroVector, SphereRadius);

		// Sweep down over both shapes with an initial overlap, however center over shape 0 more.
		FTransform QueryTM{ FVec3{-5, 0, 50} };
		const FVec3 Dir(0, 0, -1);
		const FReal Length = 200;
		SQ.Sweep(QuerySphere, QueryTM, Dir, Length, HitBuffer, EHitFlags::MTD, CommonParams);

		EXPECT_TRUE(HitBuffer.HasBlockingHit());
		const FSweepHit* BlockingHit = HitBuffer.GetBlock();
		EXPECT_EQ(BlockingHit->Actor->GetProxy(), Actor0Proxy);
		EXPECT_EQ(BlockingHit->Distance, -25);
		EXPECT_VECTOR_NEAR(BlockingHit->WorldPosition, FVector(-5, 0, 50), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(BlockingHit->WorldNormal, FVector(0, 0, 1), KINDA_SMALL_NUMBER);

		// Now do a second sweep down, this time over shape 1 more.
		QueryTM = FTransform{ FVec3{5, 0, 50} };
		SQ.Sweep(QuerySphere, QueryTM, Dir, Length, HitBuffer, EHitFlags::MTD, CommonParams);

		EXPECT_TRUE(HitBuffer.HasBlockingHit());
		BlockingHit = HitBuffer.GetBlock();
		EXPECT_EQ(BlockingHit->Actor->GetProxy(), Actor1Proxy);
		EXPECT_EQ(BlockingHit->Distance, -25);
		EXPECT_VECTOR_NEAR(BlockingHit->WorldPosition, FVector(5, 0, 50), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(BlockingHit->WorldNormal, FVector(0, 0, 1), KINDA_SMALL_NUMBER);
	}

	// Disable a moving kinematic particle and switch it to dynamic while disabled. Verify that when
	// disabled it is not in any active lists, and that when re-enabled it is not duplicated
	//
	// There was a (benign) bug where particles were not removed from the MovingKinematics list until the
	// next call to ApplyKinematicTargets, which means they can be included in collision detection.
	//
	GTEST_TEST(EvolutionTests, TestDisableKinematicEnableDynamic)
	{
		const FReal Dt = FReal(1.0 / 60.0);
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);

		Evolution.GetGravityForces().SetAcceleration(FVec3(0), 0);

		// Create a moving kinematic particle
		TArray<FPBDRigidParticleHandle*> ParticleHandles = Evolution.CreateDynamicParticles(1);
		Evolution.EnableParticle(ParticleHandles[0]);
		Evolution.SetParticleObjectState(ParticleHandles[0], EObjectStateType::Kinematic);
		Evolution.SetParticleKinematicTarget(ParticleHandles[0], FKinematicTarget::MakePositionTarget(FVec3(10,0,0), FRotation3::FromIdentity()));

		Evolution.AdvanceOneTimeStep(Dt);

		EXPECT_FALSE(ParticleHandles[0]->IsDynamic());
		EXPECT_TRUE(ParticleHandles[0]->IsKinematic());

		// Check that it is in the moving kinematics list
		{
			const TParticleView<TPBDRigidParticles<FReal, 3>>& DynamicSleepingView = Particles.GetNonDisabledDynamicView();
			const TParticleView<TPBDRigidParticles<FReal, 3>>& DynamicMovingKinematicView = Particles.GetActiveDynamicMovingKinematicParticlesView();
			EXPECT_EQ(DynamicSleepingView.Num(), 0);
			EXPECT_EQ(DynamicMovingKinematicView.Num(), 1);
		}

		// Disable the kinematic
		Evolution.DisableParticle(ParticleHandles[0]);

		// It should not be in any active views now
		{
			const TParticleView<TPBDRigidParticles<FReal, 3>>& DynamicSleepingView = Particles.GetNonDisabledDynamicView();
			const TParticleView<TPBDRigidParticles<FReal, 3>>& DynamicMovingKinematicView = Particles.GetActiveDynamicMovingKinematicParticlesView();
			EXPECT_EQ(DynamicSleepingView.Num(), 0);
			EXPECT_EQ(DynamicMovingKinematicView.Num(), 0);
		}

		// Make the particle dynamic and enable it
		Evolution.SetParticleObjectState(ParticleHandles[0], EObjectStateType::Dynamic);
		Evolution.EnableParticle(ParticleHandles[0]);

		// Check that it is in the active views, but not duplicated in either
		{
			const TParticleView<TPBDRigidParticles<FReal, 3>>& DynamicSleepingView = Particles.GetNonDisabledDynamicView();
			const TParticleView<TPBDRigidParticles<FReal, 3>>& DynamicMovingKinematicView = Particles.GetActiveDynamicMovingKinematicParticlesView();
			EXPECT_EQ(DynamicSleepingView.Num(), 1);
			EXPECT_EQ(DynamicMovingKinematicView.Num(), 1);
		}
	}

	// Check that we cannot set a kinematic target on a dynamic particle.
	//
	// This would cause the dynamic particle to be added to the MovingKinematics
	// list which means it would appear twice in the GetActiveDynamicMovingKinematicParticlesView
	// which can result in a race condition in collision detection as a particle pair will be
	// considered twice and possibly on different threads.
	GTEST_TEST(EvolutionTests, TestKinematicTargetOnDynamic)
	{
		const FReal Dt = FReal(1.0 / 60.0);
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);

		Evolution.GetGravityForces().SetAcceleration(FVec3(0), 0);

		// Create a dynamic particle
		TArray<FPBDRigidParticleHandle*> ParticleHandles = Evolution.CreateDynamicParticles(1);
		Evolution.EnableParticle(ParticleHandles[0]);

		// Set the kinematic target
		Evolution.SetParticleKinematicTarget(ParticleHandles[0], FKinematicTarget::MakePositionTarget(FVec3(10, 0, 0), FRotation3::FromIdentity()));

		// We should not have a kinematic target
		EXPECT_FALSE(ParticleHandles[0]->KinematicTarget().IsSet());

		Evolution.AdvanceOneTimeStep(Dt);

		// Check that it is in the active views, but not duplicated in either
		{
			const TParticleView<TPBDRigidParticles<FReal, 3>>& DynamicSleepingView = Particles.GetNonDisabledDynamicView();
			const TParticleView<TPBDRigidParticles<FReal, 3>>& DynamicMovingKinematicView = Particles.GetActiveDynamicMovingKinematicParticlesView();
			EXPECT_EQ(DynamicSleepingView.Num(), 1);
			EXPECT_EQ(DynamicMovingKinematicView.Num(), 1);
		}
	}
}
