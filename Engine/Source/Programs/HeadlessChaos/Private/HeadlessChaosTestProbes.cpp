// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Matrix.h"
#include "Chaos/Utilities.h"
#include "Chaos/AABB.h"
#include "Chaos/Core.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/Box.h"
#include "Chaos/Sphere.h"
#include "Chaos/ContactModification.h"
#include "ChaosSolversModule.h"
#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Modules/ModuleManager.h"
#include "PBDRigidsSolver.h"

namespace ChaosTest
{
	using namespace Chaos;

	GTEST_TEST(ProbeTests, ProbeBodyConstraint)
	{
		const FReal BoxHalfSize = 50; // cm
		const FReal Separation = 10; // cm
		const FReal InitialSpeed = BoxHalfSize * .5; // cm/s... slow enough not to need CCD
		const FReal Dt = FReal(2) * Separation / InitialSpeed; // s... twice the time needed to close the separation
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		// Create particles
		TGeometryParticleHandle<FReal, 3>* Static = Evolution.CreateStaticParticles(1)[0];
		TPBDRigidParticleHandle<FReal, 3>* Dynamic = Evolution.CreateDynamicParticles(1)[0];

		// Create box geometry 
		Chaos::FImplicitObjectPtr SmallBox(new TBox<FReal, 3>(FVec3(-BoxHalfSize, -BoxHalfSize, -BoxHalfSize), FVec3(BoxHalfSize, BoxHalfSize, BoxHalfSize)));
		Static->SetGeometry(SmallBox);
		AppendDynamicParticleConvexBox(*Dynamic, FVec3(BoxHalfSize), 0.0f);
		Dynamic->SetGravityEnabled(false);

		// Make the dynamic shape a probe
		Dynamic->ShapesArray()[0]->SetIsProbe(true);

		// Positions
		Static->SetX( FVec3(0, 0, 0));
		Dynamic->SetX(FVec3(-(2 * BoxHalfSize) - Separation, 0, 0));
		Dynamic->SetV(FVec3(InitialSpeed, 0, 0));
		Dynamic->SetCCDEnabled(false);

		// The position of the static has changed and statics don't automatically update bounds, so update explicitly
		Static->UpdateWorldSpaceState(TRigidTransform<FReal, 3>(Static->GetX(), Static->GetR()), FVec3(0));

		// Make sure the particles would collide if Dynamic wasn't a probe
		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		// 1 step should be enough to cause a collision event
		Evolution.AdvanceOneTimeStep(Dt);
		Evolution.EndFrame(Dt);

		// Make sure a constraint got created
		EXPECT_GT(Evolution.GetCollisionConstraints().NumConstraints(), 0);

		// Make sure that the velocity of the dynamic body wasn't affected
		EXPECT_EQ(Dynamic->GetV(), FVec3(InitialSpeed, 0, 0));
	}

	GTEST_TEST(ProbeTests, ProbeBodyConstraintWithCCD)
	{
		const FReal BoxHalfSize = 50; // cm
		const FReal Separation = 10; // cm
		const FReal InitialSpeed = Separation + (3 * BoxHalfSize); // cm/s... fast enough to trigger ccd and close the distance in 1 second
		const FReal Dt = 1;
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		// Create particles
		TGeometryParticleHandle<FReal, 3>* Static = Evolution.CreateStaticParticles(1)[0];
		TPBDRigidParticleHandle<FReal, 3>* Dynamic = Evolution.CreateDynamicParticles(1)[0];

		// Create box geometry 
		Chaos::FImplicitObjectPtr SmallBox(new TBox<FReal, 3>(FVec3(-BoxHalfSize, -BoxHalfSize, -BoxHalfSize), FVec3(BoxHalfSize, BoxHalfSize, BoxHalfSize)));
		Static->SetGeometry(SmallBox);
		AppendDynamicParticleConvexBox(*Dynamic, FVec3(BoxHalfSize), 0.0f);
		Dynamic->SetGravityEnabled(false);

		// Make the dynamic shape a probe
		Dynamic->ShapesArray()[0]->SetIsProbe(true);

		// Positions
		Static->SetX(FVec3(0, 0, 0));
		Dynamic->SetX(FVec3(-(2 * BoxHalfSize) - Separation, 0, 0));
		Dynamic->SetV(FVec3(InitialSpeed, 0, 0));
		Dynamic->SetCCDEnabled(true);

		// The position of the static has changed and statics don't automatically update bounds, so update explicitly
		Static->UpdateWorldSpaceState(TRigidTransform<FReal, 3>(Static->GetX(), Static->GetR()), FVec3(0));

		// Make sure the particles would collide if Dynamic wasn't a probe
		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		// 1 step should be enough to cause a collision event
		Evolution.AdvanceOneTimeStep(Dt);
		Evolution.EndFrame(Dt);

		// Make sure a constraint got created
		EXPECT_GT(Evolution.GetCollisionConstraints().NumConstraints(), 0);

		// Even though the body was moving fast enough to hit the static box
		// with CCD, make sure the constraint doesn't actually think it's CCD
		for (FPBDCollisionConstraint* const Constraint : Evolution.GetCollisionConstraints().GetConstraints())
		{
			EXPECT_EQ(Constraint->GetCCDEnabled(), false);
		}

		// Make sure that the velocity of the dynamic body wasn't affected
		EXPECT_EQ(Dynamic->GetV(), FVec3(InitialSpeed, 0, 0));
	}


	class FCollisionModifier : public ISimCallbackObject
	{
	protected:
		void OnContactModification_Internal(FCollisionContactModifier& Modifier)
		{
			CalledCount++;
			if (CalledCount <= 2) // Lets apply the mofisier only the two first steps
			{
				for (FContactPairModifier& PairModifier : Modifier)
				{
					PairModifier.ConvertToProbe();
				}
			}
		}
		virtual void FreeOutputData_External(FSimCallbackOutput* Output) {}
		virtual void FreeInputData_Internal(FSimCallbackInput* Input) {}
		virtual FSimCallbackInput* AllocateInputData_External() { return nullptr; }

		int CalledCount = 0;
	};

	// This test has been added after a bug where the probe flag was used but not initilaized back when modified by a modifier
	GTEST_TEST(ProbeTests, ProbeBodySetInModifier)
	{
		const FReal BoxHalfSize = 50; // cm
		const FReal Separation = 10; // cm
		const FReal InitialSpeed = Separation + (3 * BoxHalfSize); // cm/s... fast enough to trigger ccd and close the distance in 1 second
		const FReal Dt = 1;
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FCollisionModifier CollisionModifier;
		TArray<ISimCallbackObject*> CollisionModifiers = { &CollisionModifier };
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials, nullptr, nullptr, nullptr, &CollisionModifiers);
		InitEvolutionSettings(Evolution);

		// Create particles
		TGeometryParticleHandle<FReal, 3>* Static = Evolution.CreateStaticParticles(1)[0];
		TPBDRigidParticleHandle<FReal, 3>* Dynamic = Evolution.CreateDynamicParticles(1)[0];

		// Create box geometry 
		Chaos::FImplicitObjectPtr SmallBox(new TBox<FReal, 3>(FVec3(-BoxHalfSize, -BoxHalfSize, -BoxHalfSize), FVec3(BoxHalfSize, BoxHalfSize, BoxHalfSize)));
		Static->SetGeometry(SmallBox);
		AppendDynamicParticleConvexBox(*Dynamic, FVec3(BoxHalfSize), 0.0f);
		Dynamic->SetGravityEnabled(false);

		// Positions
		Static->SetX(FVec3(0, 0, 0));
		Dynamic->SetX(FVec3(-(2 * BoxHalfSize) - Separation, 0, 0));
		Dynamic->SetV(FVec3(InitialSpeed, 0, 0));
		Dynamic->SetCCDEnabled(true);

		// The position of the static has changed and statics don't automatically update bounds, so update explicitly
		Static->UpdateWorldSpaceState(TRigidTransform<FReal, 3>(Static->GetX(), Static->GetR()), FVec3(0));

		// Make sure the particles would collide if Dynamic wasn't a probe
		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		// 1 step should be enough to cause a collision event
		Evolution.AdvanceOneTimeStep(Dt);
		Evolution.EndFrame(Dt);

		// Make sure a constraint got created
		EXPECT_GT(Evolution.GetCollisionConstraints().NumConstraints(), 0);

		for (FPBDCollisionConstraint* const Constraint : Evolution.GetCollisionConstraints().GetConstraints())
		{
			EXPECT_EQ(Constraint->GetBoundsTestFlags().bIsProbe, false);
			EXPECT_EQ(Constraint->GetIsProbe(), true);
		}

		// Make sure that the velocity of the dynamic body wasn't affected
		EXPECT_EQ(Dynamic->GetV(), FVec3(InitialSpeed, 0, 0));

		// In the second step the probe should be initiaized back, then affected by the modifier
		Evolution.AdvanceOneTimeStep(Dt);
		Evolution.EndFrame(Dt);

		// Make sure a constraint is still there
		EXPECT_GT(Evolution.GetCollisionConstraints().NumConstraints(), 0);

		for (FPBDCollisionConstraint* const Constraint : Evolution.GetCollisionConstraints().GetConstraints())
		{
			EXPECT_EQ(Constraint->GetBoundsTestFlags().bIsProbe, false);
			EXPECT_EQ(Constraint->GetIsProbe(), true);
		}

		// Make sure that the velocity of the dynamic body wasn't affected
		EXPECT_EQ(Dynamic->GetV(), FVec3(InitialSpeed, 0, 0));


		// This time the modifier has been disabled 
		// In the third step the probe should be initiaized back, then NOT affected by the modifier
		Evolution.AdvanceOneTimeStep(Dt);
		Evolution.EndFrame(Dt);

		// Make sure a constraint is still there
		EXPECT_GT(Evolution.GetCollisionConstraints().NumConstraints(), 0);

		for (FPBDCollisionConstraint* const Constraint : Evolution.GetCollisionConstraints().GetConstraints())
		{
			EXPECT_EQ(Constraint->GetBoundsTestFlags().bIsProbe, false);
			EXPECT_EQ(Constraint->GetIsProbe(), false);
		}

		// Make sure that the velocity of the dynamic body WAS affected
		EXPECT_NE(Dynamic->GetV(), FVec3(InitialSpeed, 0, 0));

	}
}
