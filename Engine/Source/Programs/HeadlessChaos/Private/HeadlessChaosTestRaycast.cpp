// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestRaycast.h"

#include "HeadlessChaos.h"
#include "Chaos/Sphere.h"
#include "Chaos/Capsule.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"

namespace ChaosTest
{
	using namespace Chaos;

	/*In general we want to test the following for each geometry type:
	- time represents how far a swept object travels
	- position represents the world position where an intersection first occurred. If multiple first intersections we should do something well defined (what?)
	- normal represents the world normal where an intersection first occurred.
	- time vs position (i.e. in a thick raycast we want point of impact)
	- initial overlap blocks
	- near hit
	- near miss
	*/

	void SphereRaycast()
	{
		TSphere<FReal,3> Sphere(FVec3(1), 15);

		FReal Time;
		FVec3 Position, Normal;
		int32 FaceIndex;

		//simple
		bool bHit = Sphere.Raycast(FVec3(1,1,17), FVec3(0, 0, -1), 30, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, (FReal)1);
		
		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 16);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		//initial overlap
		bHit = Sphere.Raycast(FVec3(1, 1, 14), FVec3(0, 0, -1), 15, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 0);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		//near hit
		bHit = Sphere.Raycast(FVec3(16, 1, 16), FVec3(0, 0, -1), 30, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 15);

		EXPECT_FLOAT_EQ(Normal.X, 1);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 0);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Position.X, 16);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 1);

		//near miss
		bHit = Sphere.Raycast(FVec3(16 + 1e-4, 1, 16), FVec3(0, 0, -1), 30, 0, Time, Position, Normal, FaceIndex);
		EXPECT_FALSE(bHit);

		//time vs position
		bHit = Sphere.Raycast(FVec3(21, 1, 16), FVec3(0, 0, -1), 30, 5, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 15);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 1);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 0);

		EXPECT_FLOAT_EQ(Position.X, 16);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 1);

		//passed miss
		bHit = Sphere.Raycast(FVec3(1, 1, -14 - 1e-4), FVec3(0, 0, -1), 30, 0, Time, Position, Normal, FaceIndex);
		EXPECT_FALSE(bHit);
	}

		void PlaneRaycast()
	{
		TPlane<FReal, 3> Plane(FVec3(1), FVec3(1, 0, 0));

		FReal Time;
		FVec3 Position, Normal;
		int32 FaceIndex;

		//simple
		bool bHit = Plane.Raycast(FVec3(2, 1, 1), FVec3(-1, 0, 0), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 1);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 0);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 1);

		//Other side of plane
		bHit = Plane.Raycast(FVec3(-1, 1, 1), FVec3(1, 0, 0), 4, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 2);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, -1);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 0);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 1);

		//initial overlap
		bHit = Plane.Raycast(FVec3(2, 1, 1), FVec3(1, 0, 0), 2, 3, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 0);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		//near hit
		bHit = Plane.Raycast(FVec3(1+1, 1, 1), FVec3(-1e-2, 0, 1).GetUnsafeNormal(), 100.01, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 1);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 0);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 101);

		//near miss
		bHit = Plane.Raycast(FVec3(1 + 1, 1, 1), FVec3(-1e-2, 0, 1).GetUnsafeNormal(), 99.9, 0, Time, Position, Normal, FaceIndex);
		EXPECT_FALSE(bHit);

		//time vs position
		bHit = Plane.Raycast(FVec3(-1, 1, 1), FVec3(1, 0, 0), 4, 1, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(FaceIndex, INDEX_NONE);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 1);
	}

	void CapsuleRaycast()
	{
		FReal Time;
		FVec3 Position;
		FVec3 Normal;
		int32 FaceIndex;

		//straight down
		FCapsule Capsule(FVec3(1, 1, 1), FVec3(1, 1, 9), 1);
		bool bHit = Capsule.Raycast(FVec3(1, 1, 11), FVec3(0, 0, -1), 2, 0, Time, Position, Normal, FaceIndex);
		
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 10);

		//straight up
		bHit = Capsule.Raycast(FVec3(1, 1, -1), FVec3(0, 0, 1), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, -1);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 0);

		//cylinder
		bHit = Capsule.Raycast(FVec3(3, 1, 7), FVec3(-1, 0, 0), 2, 0, Time, Position, Normal, FaceIndex);

		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 1);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 0);

		EXPECT_FLOAT_EQ(Position.X, 2);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 7);

		//cylinder away
		bHit = Capsule.Raycast(FVec3(3, 1, 7), FVec3(1, 0, 0), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_FALSE(bHit);
		
		// initial overlap cap
		bHit = Capsule.Raycast(FVec3(1, 1, 9.5), FVec3(-1, 0, 0), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(Time, 0);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		// initial overlap cylinder
		bHit = Capsule.Raycast(FVec3(1, 1, 7), FVec3(-1, 0, 0), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(Time, 0);

		//cylinder time vs position
		bHit = Capsule.Raycast(FVec3(4, 1, 7), FVec3(-1, 0, 0), 4, 1, Time, Position, Normal, FaceIndex);

		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 1);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 0);

		EXPECT_FLOAT_EQ(Position.X, 2);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 7);

		//normal independent of ray dir
		bHit = Capsule.Raycast(FVec3(4, 1, 7), FVec3(-1, 0, -1).GetUnsafeNormal(), 4, 1, Time, Position, Normal, FaceIndex);

		EXPECT_TRUE(bHit);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 1);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 0);

		EXPECT_FLOAT_EQ(Position.X, 2);

		//near hit orthogonal
		bHit = Capsule.Raycast(FVec3(2, 3, 7), FVec3(0, -1, 0), 4, 0, Time, Position, Normal, FaceIndex);

		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 2);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 1);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 0);

		EXPECT_FLOAT_EQ(Position.X, 2);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 7);

		//near miss
		bHit = Capsule.Raycast(FVec3(2 + 1e-4, 3, 7), FVec3(0, -1, 0), 4, 0, Time, Position, Normal, FaceIndex);
		EXPECT_FALSE(bHit);

		//near hit straight down
		bHit = Capsule.Raycast(FVec3(0, 1, 11), FVec3(0, 0, -1), 20, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, -1);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 0);

		EXPECT_FLOAT_EQ(Position.X, 0);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 9);

		bHit = Capsule.Raycast(FVec3(-1e-4, 1, 11), FVec3(0, 0, -1), 20, 0, Time, Position, Normal, FaceIndex);
		EXPECT_FALSE(bHit);
	}

	void CapsuleRaycastFastLargeDistance()
	{
		// This ray is 35k units from origin, and casts towards a 13 unit radius capsule at origin.
		// Precision issues lead to incorrect normal, in this case we previously got a zero normal, otherwise we may also have gotten normals of non-unit length.
		// Fix was to make RaycastFast clip ray against bounds near capsule, to avoid precision issues with ray start far from capsule when solving quadratic.


		Chaos::FReal InMRadius = 13.0000000;
		Chaos::FReal InMHeight = 10.0000000;
		Chaos::FVec3 InMVector(0.00000000, 0.00000000, 1.00000000);
		Chaos::FVec3 InX1(0.00000000, 0.00000000, -5.00000000);
		Chaos::FVec3 InX2(0.00000000, 0.00000000, 5.00000000);
		Chaos::FVec3 InStartPoint(-18115.0938, 30080.6074, -1756.17285);
		Chaos::FVec3 InDir(0.515248418, -0.855584025, 0.0499509014);
		Chaos::FReal InLength = 35157.9805;
		Chaos::FReal InThickness = 0.00000000;

		Chaos::FReal TestTime;
		Chaos::FVec3 TestPosition;
		Chaos::FVec3 TestNormal;
		int32 TestFaceIndex;

		const bool bHit = Chaos::FCapsule::RaycastFast(InMRadius, InMHeight, InMVector, InX1, InX2, InStartPoint, InDir, InLength, InThickness, TestTime, TestPosition, TestNormal, TestFaceIndex);

 		EXPECT_NEAR(TestNormal.Size(), 1.0f, KINDA_SMALL_NUMBER);
		EXPECT_TRUE(bHit);
	}
	
	void CapsuleRaycastMissWithEndPointOnBounds()
	{
		// Ray goes towards cap of capsule from x axis, endpoint of ray is exactly on bounds of capsule, but short of hitting the cap.
		// This caused an edge case in which ray is clipped against bounds, and tried to continue calculation with clipped ray length of zero.
		// This previously triggered ensure on Length > 0 when casting against spheres for cap test.
		// This should miss.

		Chaos::FReal InMRadius = 10.0000000;
		Chaos::FReal InMHeight = 10.0000000;
		Chaos::FVec3 InMVector(0.00000000, 0.00000000, 1.00000000);
		Chaos::FVec3 InX1(0.00000000, 0.00000000, -5.00000000);
		Chaos::FVec3 InX2(0.00000000, 0.00000000, 5.00000000);
		Chaos::FVec3 InStartPoint(100,0,8);
		Chaos::FVec3 InDir(-1,0,0);
		Chaos::FReal InLength = 90;
		Chaos::FReal InThickness = 0.00000000;

		Chaos::FReal TestTime;
		Chaos::FVec3 TestPosition;
		Chaos::FVec3 TestNormal;
		int32 TestFaceIndex;

		const bool bHit = Chaos::FCapsule::RaycastFast(InMRadius, InMHeight, InMVector, InX1, InX2, InStartPoint, InDir, InLength, InThickness, TestTime, TestPosition, TestNormal, TestFaceIndex);
		EXPECT_FALSE(bHit);
	}

	void TriangleRaycast()
	{
		FReal Time;
		FVec3 Position;
		FVec3 Normal;
		TArray<uint16> DummyMaterials;
		int32 FaceIndex;

		FTriangleMeshImplicitObject::ParticlesType Particles;
		Particles.AddParticles(3);
		Particles.SetX(0, FVec3(1, 1, 1));
		Particles.SetX(1, FVec3(5, 1, 1));
		Particles.SetX(2, FVec3(1, 5, 1));
		TArray<TVec3<int32>> Indices;
		Indices.Emplace(0, 1, 2);
		FTriangleMeshImplicitObject Tri(MoveTemp(Particles), MoveTemp(Indices), MoveTemp(DummyMaterials));

		//simple into the triangle
		bool bHit = Tri.Raycast(FVec3(3, 2, 2), FVec3(0, 0, -1), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, 0);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 3);
		EXPECT_FLOAT_EQ(Position.Y, 2);
		EXPECT_FLOAT_EQ(Position.Z, 1);

		//double sided
		bHit = Tri.Raycast(FVec3(3, 2, 0), FVec3(0, 0, 1), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, 0);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, -1);

		EXPECT_FLOAT_EQ(Position.X, 3);
		EXPECT_FLOAT_EQ(Position.Y, 2);
		EXPECT_FLOAT_EQ(Position.Z, 1);

		//time vs position
		bHit = Tri.Raycast(FVec3(3, 2, 3), FVec3(0, 0, -1), 2, 1, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, 0);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 3);
		EXPECT_FLOAT_EQ(Position.Y, 2);
		EXPECT_FLOAT_EQ(Position.Z, 1);

		//initial miss, border hit
		bHit = Tri.Raycast(FVec3(0.5, 2, 3), FVec3(0, 0, -1), 2, 1, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(FaceIndex, 0);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 2);
		EXPECT_FLOAT_EQ(Position.Z, 1);

		//initial overlap with plane, but miss triangle
		bHit = Tri.Raycast(FVec3(10, 1, 1), FVec3(0, 0, -1), 2, 1, Time, Position, Normal, FaceIndex);
		EXPECT_FALSE(bHit);

		//parallel with triangle
		bHit = Tri.Raycast(FVec3(-1, 1, 1), FVec3(1, 0, 0), 2, 1, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, 0);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 1);
	}

	void TriangleRaycastDenegerated()
	{
		FReal Time;
		FVec3 Position;
		FVec3 Normal;
		TArray<uint16> DummyMaterials;
		int32 FaceIndex;

		FTriangleMeshImplicitObject::ParticlesType Particles;
		Particles.AddParticles(3);
		Particles.SetX(0, FVec3(1, 1, 1));
		Particles.SetX(1, FVec3(1, 1, 2));
		Particles.SetX(2, FVec3(1, 1, 3));
		TArray<TVec3<int32>> Indices;
		Indices.Emplace(0, 1, 2);
		FTriangleMeshImplicitObject Tri(MoveTemp(Particles), MoveTemp(Indices), MoveTemp(DummyMaterials));

		//simple into the triangle
		bool bHit = Tri.Raycast(FVec3(1, -1, 1.5), FVec3(0, 1, 0), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_FALSE(bHit);
	}

	void BoxRaycast()
	{
		FReal Time;
		FVec3 Position;
		FVec3 Normal;
		int32 FaceIndex;

		FAABB3 Box(FVec3(1, 1, 1), FVec3(3, 5, 3));
		
		//simple into the box
		bool bHit = Box.Raycast(FVec3(2, 3, 4), FVec3(0, 0, -1), 2, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 2);
		EXPECT_FLOAT_EQ(Position.Y, 3);
		EXPECT_FLOAT_EQ(Position.Z, 3);

		//time vs position
		bHit = Box.Raycast(FVec3(2, 3, 5), FVec3(0, 0, -1), 2, 1, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 2);
		EXPECT_FLOAT_EQ(Position.Y, 3);
		EXPECT_FLOAT_EQ(Position.Z, 3);

		//edge
		bHit = Box.Raycast(FVec3(0.5, 2, -1), FVec3(0, 0, 1), 2, 1, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(FaceIndex, INDEX_NONE);
				
		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 2);
		EXPECT_FLOAT_EQ(Position.Z, 1);

		//corner
		bHit = Box.Raycast(FVec3(0.5, 1, -1), FVec3(0, 0, 1), 2, 1, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 1);

		//near hit by corner edge
		const FVec3 StartEmptyRegion(1 - FMath::Sqrt(static_cast<FReal>(2)) / 2, 1 - FMath::Sqrt(static_cast<FReal>(2)) / 2, -1);
		bHit = Box.Raycast(StartEmptyRegion, FVec3(0, 0, 1), 2, 1, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 1);

		//near miss by corner edge
		const FVec3 StartEmptyRegionMiss(StartEmptyRegion[0] - 1e-4, StartEmptyRegion[1] - 1e-4, StartEmptyRegion[2]);
		bHit = Box.Raycast(StartEmptyRegionMiss, FVec3(0, 0, 1), 2, 1, Time, Position, Normal, FaceIndex);
		EXPECT_FALSE(bHit);

		//start in corner voronoi but end in edge voronoi
		bHit = Box.Raycast(FVec3(0,0, 0.8), FVec3(1, 1, 5).GetUnsafeNormal(), 2, 1, Time, Position, Normal, FaceIndex);

		EXPECT_TRUE(bHit);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_GT(Position.Z, 1);

		//start in voronoi and miss
		bHit = Box.Raycast(FVec3(0, 0, 0.8), FVec3(-1, -1, 0).GetUnsafeNormal(), 2, 1, Time, Position, Normal, FaceIndex);
		EXPECT_FALSE(bHit);

		//initial overlap
		bHit = Box.Raycast(FVec3(1, 1, 2), FVec3(-1, -1, 0).GetUnsafeNormal(), 2, 1, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_EQ(FaceIndex, INDEX_NONE);
		EXPECT_EQ(Time, 0);
	}

	void VectorizedAABBRaycast()
	{
		// AABB with 12cm thickness
		FAABBVectorized AABB(MakeVectorRegisterFloat(565429.188f-12.0f, -17180.4355f-12.0f, -95264.4219f-12.0f, 0.f), MakeVectorRegisterFloat(568988.312f+12.0f, -13372.2793f+12.0f, -93649.4609f+12.0f, 0.f));

		constexpr float Length = 371.331360f;
		const VectorRegister4Float RayStart = MakeVectorRegisterFloat(565223.812f, -13919.9111f, -93982.5078f, 0.f);
		const VectorRegister4Float RayDir = MakeVectorRegisterFloat(0.0622759163f, -0.997566581f, -0.0313483514f, 0.f);
		const VectorRegister4Float RayInvDir = MakeVectorRegisterFloat(16.0575714f, -1.00243938f, -31.8996048f, 0.f);
		const VectorRegister4Float RayLength = MakeVectorRegisterFloat(Length, Length, Length, Length);
		const VectorRegister4Float RayInvLength = VectorDivide(MakeVectorRegisterFloatConstant(1.f, 1.f, 1.f, 1.f), RayLength);
		const VectorRegister4Float Parallel = VectorZero();
		bool bHit = AABB.RaycastFast(RayStart, RayInvDir, Parallel, RayLength);
		EXPECT_FALSE(bHit);

	}
	
	void ScaledRaycast()
	{
		// Note: Spheres cannot be thickened by adding a margin to a wrapper type (such as TImplicitObjectScaled) 
		// because Spheres already have their margin set to maximum (margins are always internal to the shape).
		// Therefore we expect the "thickened" results below to be the same as the unthickened.

		FReal Time;
		FVec3 Position;
		FVec3 Normal;
		int32 FaceIndex;
		const FReal Thickness = 0.1;

		FSpherePtr Sphere( new TSphere<FReal,3>(FVec3(1), 2));
		TImplicitObjectScaled<Chaos::FSphere> Unscaled(Sphere, FVec3(1));
		TImplicitObjectScaled<Chaos::FSphere> UnscaledThickened(Sphere, FVec3(1), Thickness);
		TImplicitObjectScaled<Chaos::FSphere> UniformScaled(Sphere, FVec3(2));
		TImplicitObjectScaled<Chaos::FSphere> UniformScaledThickened(Sphere, FVec3(2), Thickness);
		TImplicitObjectScaled<Chaos::FSphere> NonUniformScaled(Sphere, FVec3(2,1,1));
		TImplicitObjectScaled<Chaos::FSphere> NonUniformScaledThickened(Sphere, FVec3(2, 1, 1), Thickness);

		//simple
		bool bHit = Unscaled.Raycast(FVec3(1, 1, 8), FVec3(0, 0, -1), 8, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 5);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 3);

		bHit = UnscaledThickened.Raycast(FVec3(1, 1, 8), FVec3(0, 0, -1), 8, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 5);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 1);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 3);

		bHit = UniformScaled.Raycast(FVec3(2, 2, 8), FVec3(0, 0, -1), 8, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 2);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 2);
		EXPECT_FLOAT_EQ(Position.Y, 2);
		EXPECT_FLOAT_EQ(Position.Z, 6);

		bHit = UniformScaledThickened.Raycast(FVec3(2, 2, 8), FVec3(0, 0, -1), 8, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 2);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 2);
		EXPECT_FLOAT_EQ(Position.Y, 2);
		EXPECT_FLOAT_EQ(Position.Z, 6);

		bHit = NonUniformScaled.Raycast(FVec3(2, 1, 8), FVec3(0, 0, -1), 8, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 5);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 2);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 3);

		bHit = NonUniformScaledThickened.Raycast(FVec3(2, 1, 8), FVec3(0, 0, -1), 8, 0, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 5);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 2);
		EXPECT_FLOAT_EQ(Position.Y, 1);
		EXPECT_FLOAT_EQ(Position.Z, 3);

		//scaled thickness
		bHit = UniformScaled.Raycast(FVec3(2, 2, 8), FVec3(0, 0, -1), 8, 1, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 2);
		EXPECT_FLOAT_EQ(Position.Y, 2);
		EXPECT_FLOAT_EQ(Position.Z, 6);

		bHit = UniformScaledThickened.Raycast(FVec3(2, 2, 8), FVec3(0, 0, -1), 8, 1, Time, Position, Normal, FaceIndex);
		EXPECT_TRUE(bHit);
		EXPECT_FLOAT_EQ(Time, 1);
		EXPECT_EQ(FaceIndex, INDEX_NONE);

		EXPECT_FLOAT_EQ(Normal.X, 0);
		EXPECT_FLOAT_EQ(Normal.Y, 0);
		EXPECT_FLOAT_EQ(Normal.Z, 1);

		EXPECT_FLOAT_EQ(Position.X, 2);
		EXPECT_FLOAT_EQ(Position.Y, 2);
		EXPECT_FLOAT_EQ(Position.Z, 6);
	}
}