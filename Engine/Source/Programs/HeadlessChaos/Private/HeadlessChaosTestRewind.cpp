// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestUtility.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ErrorReporter.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Chaos/Utilities.h"
#include "PBDRigidsSolver.h"
#include "ChaosSolversModule.h"

#include "Modules/ModuleManager.h"
#include "RewindData.h"
#include "GeometryCollection/GeometryCollectionTestFramework.h"
#include "Chaos/SimCallbackObject.h"
#include "Chaos/Framework/ChaosResultsManager.h"

#ifndef REWIND_DESYNC
#define REWIND_DESYNC 0
#endif

namespace ChaosTest {

    using namespace Chaos;
	using namespace GeometryCollectionTest;

	template <typename TSolver>
	void TickSolverHelper(TSolver* Solver, FReal Dt = 1.0)
	{
		Solver->AdvanceAndDispatch_External(Dt);
		Solver->UpdateGameThreadStructures();
	}

	template<typename T_SOLVER>
	void EnableRewindCaptureHelper(T_SOLVER& Solver, const int32 RewindHistorySize, const int32& Optimization)
	{
		/** int32 Optimization starts at 0 and iterate up to 2
		* 0 = No optimizations
		* 1 = CollisionResimCache
		* 2 = Optimized RewindData */

		Solver->EnableRewindCapture(RewindHistorySize, Optimization == 1); // Use CollisionResimCache optimization as optimization 1
		if (Chaos::FRewindData* RewindData = Solver->GetRewindData())
		{
			RewindData->SetRewindDataOptimization(Optimization == 2); // Use optimized RewindData in as optimization 2
		}
	}

	auto* CreateSolverHelper(int32 StepMode, int32 RewindHistorySize, int32 Optimization, FReal& OutSimDt)
	{
		constexpr FReal FixedDt = 1;
		constexpr FReal DtSizes[] = { FixedDt, FixedDt, FixedDt * 0.25, FixedDt * 4 };	//test fixed dt, sub-stepping, step collapsing

		// Make a solver
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto* Solver = Module->CreateSolver(nullptr,/*AsyncDt=*/-1);
		InitSolverSettings(Solver);

		EnableRewindCaptureHelper(Solver, RewindHistorySize, Optimization);

		Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
		OutSimDt = DtSizes[StepMode];
		if (StepMode > 0)
		{
			Solver->EnableAsyncMode(DtSizes[StepMode]);
		}

		return Solver;
	}

	struct TRewindHelper
	{
		template <typename TLambda>
		static void TestEmpty(const TLambda& Lambda, int32 RewindHistorySize = 200)
		{
			for (int Optimization = 0; Optimization < 3; ++Optimization)
			{
				for (int DtMode = 0; DtMode < 4; ++DtMode)
				{
					FChaosSolversModule* Module = FChaosSolversModule::GetModule();
					FReal SimDt;
					auto* Solver = CreateSolverHelper(DtMode, RewindHistorySize, Optimization, SimDt);
					Solver->SetMaxDeltaTime_External(SimDt);	//make sure it can step even for huge steps

					Lambda(Solver, SimDt, Optimization);

					Module->DestroySolver(Solver);
				}
			}
		}

		template <typename TLambda>
		static void TestDynamicSphere(const TLambda& Lambda, int32 RewindHistorySize = 200)
		{
			TestEmpty([&Lambda, RewindHistorySize](auto* Solver, FReal SimDt, int32 Optimization)
			{
					Chaos::FImplicitObjectPtr Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));

				// Make particles
					auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
					auto& Particle = Proxy->GetGameThreadAPI();

					Particle.SetGeometry(Sphere);
					Solver->RegisterObject(Proxy);

					Lambda(Solver, SimDt, Optimization, Proxy, Sphere.GetReference());

			}, RewindHistorySize);
		}
	};

	GTEST_TEST(AllTraits, RewindTest_MovingGeomChange)
	{
		TRewindHelper::TestEmpty([](auto* Solver, FReal SimDt, int32 Optimization)
		{
			auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
			auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(0), FVec3(1)));
			auto Box2 = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(2), FVec3(3)));

			// Make particles
				auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FKinematicGeometryParticle::CreateParticle());
				auto& Particle = Proxy->GetGameThreadAPI();

				Particle.SetGeometry(Sphere);
				Solver->RegisterObject(Proxy);
			const int32 LastGameStep = 20;

			for (int Step = 0; Step <= LastGameStep; ++Step)
			{
				//property that changes every step
					Particle.SetX(FVec3(0, 0, 100 - Step));

				//property that changes once half way through
				if (Step == 3)
				{
						Particle.SetGeometry(Box);
				}

				if (Step == 5)
				{
						Particle.SetGeometry(Box2);
				}

				if (Step == 7)
				{
						Particle.SetGeometry(Box);
				}

				TickSolverHelper(Solver);
			}

			//ended up at z = 100 - LastGameStep
				EXPECT_EQ(Particle.X()[2], 100 - LastGameStep);

			//ended up with box geometry
				EXPECT_EQ(Box.GetReference(), Particle.GetGeometry());

			const FRewindData* RewindData = Solver->GetRewindData();

			//check state at every step except latest
			const int32 LastSimStep = LastGameStep / SimDt;
			for (int SimStep = 0; SimStep < LastSimStep - 1; ++SimStep)
			{
				const FReal TimeStart = SimStep * SimDt;
				const FReal TimeEnd = (SimStep + 1) * SimDt;
				const FReal LastInputTime = SimDt <= 1 ? TimeStart : TimeEnd - 1;	//latest gt time associated with this interval

				/** Grab the cached data from the beginning of the next frame. When the particle is set by GT data we cache the pre-GT state in the RewindData's PrePushData phase, 
				* the dirty GT data is separately cached as marshaled PushData and the result of the GT being applied will be cached in the next frames PrePushData. */
				const auto ParticleState = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), SimStep + 1, Chaos::FFrameAndPhase::PrePushData);
				EXPECT_EQ(ParticleState.GetX()[2], 100 - FMath::FloorToInt(LastInputTime));	//We teleported on GT so no interpolation

				if (LastInputTime < 3)
				{
					//was sphere
					EXPECT_EQ(ParticleState.GetGeometry(), Sphere.GetReference());
				}
				else if (LastInputTime < 5 || LastInputTime >= 7)
				{
					//then became box
					EXPECT_EQ(ParticleState.GetGeometry(), Box.GetReference());
				}
				else
				{
					//second box
					EXPECT_EQ(ParticleState.GetGeometry(), Box2.GetReference());
				}
			}

				Solver->UnregisterObject(Proxy);
		});
	}

	struct FRewindCallbackTestHelper : public IRewindCallback
	{
		FRewindCallbackTestHelper(const int32 InStepToRewindOn, const int32 InRewindToStep = 0)
			: StepToRewindOn(InStepToRewindOn)
			, RewindToStep(InRewindToStep)
		{
		}

		virtual int32 TriggerRewindIfNeeded_Internal(int32 PhysStep) override
			{
			return bRewound ? INDEX_NONE : TriggerRewindFunc(PhysStep);
		}

		virtual void ProcessInputs_Internal(int32 PhysicsStep, const TArray<FSimCallbackInputAndObject>& SimCallbackInputs) override
		{
			ProcessInputsFunc(PhysicsStep, bRewound);
		}

		virtual void PreResimStep_Internal(int32 Step, bool bFirst) override
		{
			bRewound = true;
		}

		virtual void PostResimStep_Internal(int32 Step) override
		{
			PostFunc(Step);
			bRewound = false;
		}

		int32 StepToRewindOn;
		int32 RewindToStep;
		bool bRewound = false;
		TFunction<void(int32, bool)> ProcessInputsFunc = [](int32, bool) {};
		TFunction<void(int32)> PostFunc = [](int32) {};
		TFunction<int32(int32)> TriggerRewindFunc = [this](int32 PhysicsStep) -> int32
		{ 
			return PhysicsStep == StepToRewindOn ? RewindToStep : INDEX_NONE;
		};
	};

	template <typename TSolver>
	FRewindCallbackTestHelper* RegisterCallbackHelper(TSolver* Solver, const int32 NumStepsBeforeRewind = 0, const int32 RewindTo = 0)
	{
		auto Callback = MakeUnique<FRewindCallbackTestHelper>(NumStepsBeforeRewind - 1, RewindTo);
		FRewindCallbackTestHelper* Result = Callback.Get();
		Solver->SetRewindCallback(MoveTemp(Callback));
		return Result;
	}

	GTEST_TEST(AllTraits, RewindTest_AddImpulseFromGT)
	{
		//We expect anything that came from GT to automatically be reapplied during rewind
		//This is for things that come outside of the net prediction system, like a teleport
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
		{
			const int32 LastGameStep = 20;
			FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver, FMath::TruncToInt(LastGameStep / SimDt));
			Helper->PostFunc = [Proxy, SimDt](int32 Step)
			{
				const int32 GameStep = SimDt <= 1 ? Step * SimDt : (Step + 1) * SimDt;
				if (GameStep < 5)
				{
					EXPECT_EQ(Proxy->GetPhysicsThreadAPI()->V()[2], 0);
				}
				else
				{
					EXPECT_EQ(Proxy->GetPhysicsThreadAPI()->V()[2], 100);
				}
			};

			auto& Particle = Proxy->GetGameThreadAPI();
			Particle.SetGravityEnabled(false);


			for (int Step = 0; Step <= LastGameStep; ++Step)
			{
				if (Step == 5)
				{
					Particle.SetLinearImpulse(FVec3(0, 0, 100), /*bIsVelocity=*/false);
				}

				TickSolverHelper(Solver);
			}
		});
	}

	GTEST_TEST(AllTraits, RewindTest_DeleteFromGT)
	{
		//GT writes of a deleted particle should be ignored during a resim (not crash)
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				const int32 LastGameStep = 20;
				RegisterCallbackHelper(Solver, FMath::TruncToInt(LastGameStep / SimDt));
				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);


				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					if (Step == 5)
					{
						Particle.SetLinearImpulse(FVec3(0, 0, 100), /*bIsVelocity=*/false);
					}

					if(Step == 15)
					{
						Solver->UnregisterObject(Proxy);
					}

					TickSolverHelper(Solver);
				}
			});
	}

	GTEST_TEST(AllTraits, RewindTest_ResimBeforeCreation)
	{
		//GT creates object half way through sim - we want to make sure resim properly ignores this
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				const int32 LastGameStep = 20;
				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver, FMath::TruncToInt(LastGameStep / SimDt));
				Helper->PostFunc = [Proxy, SimDt](int32 Step)
				{
					//simple movement without hitting object until we hit floor, should not ever hit sphere
					const FReal NoHitZ = 100 - (Step + 1) * SimDt * 10;
					if (NoHitZ > 40)
					{
						EXPECT_EQ(Proxy->GetPhysicsThreadAPI()->X()[2], NoHitZ);
					}
					else
					{
						EXPECT_GE(Proxy->GetPhysicsThreadAPI()->X()[2], 35);	//floor should stop us (gave 5 units of error for solver)
					}
				};

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				ChaosTest::SetParticleSimDataToCollide({ Proxy->GetParticle_LowLevel() });


				Particle.SetX(FVec3(0, 0, 100));
				Particle.SetV(FVec3(0, 0, -10));

				auto SphereGeom = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
				FSingleParticlePhysicsProxy* SecondSphere = nullptr;

				auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100, -100, 0), FVec3(100, 100, 30)));
				FSingleParticlePhysicsProxy* Floor = nullptr;

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					if(Step == 5)
					{
						// Make blocking floor
						Floor = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
						auto& FloorParticle = Floor->GetGameThreadAPI();

						FloorParticle.SetGeometry(FloorGeom);
						Solver->RegisterObject(Floor);

						ChaosTest::SetParticleSimDataToCollide({ Floor->GetParticle_LowLevel() });
					}
					if (Step == 15)
					{
						// Make particles
						SecondSphere = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
						auto& SecondSphereParticle = SecondSphere->GetGameThreadAPI();

						SecondSphereParticle.SetGravityEnabled(false);
						SecondSphereParticle.SetX(FVec3(0, 0, 80));	//if resim doesn't disable, this will be in the Sphere's way even though it didn't exist yet
						SecondSphereParticle.SetGeometry(SphereGeom);
						Solver->RegisterObject(SecondSphere);

						ChaosTest::SetParticleSimDataToCollide({ SecondSphere->GetParticle_LowLevel() });
					}

					TickSolverHelper(Solver);
				}

				Solver->UnregisterObject(SecondSphere);
				Solver->UnregisterObject(Floor);
			});
	}

	// @todo(chaos): Rewind does not support SimData changes yet
	// (note this test used to pass, but there was a bug in UpdateShapesArrayFromGeometry that would leave SimData
	// as all-zero if you don't have a Union at the root. That bug is fixed which exposes this test failure)
	GTEST_TEST(AllTraits, DISABLED_RewindTest_ResimShapeFilter)
	{
		//GT modifies filter after object passes through, want to make sure resim restores this state correctly
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				const int32 LastGameStep = 20;
				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver, FMath::TruncToInt(LastGameStep / SimDt));
				Helper->PostFunc = [Proxy, SimDt](int32 Step)
				{
					//simple movement without hitting object
					const FReal NoHitZ = 100 - (Step + 1) * SimDt * 10;
					EXPECT_EQ(Proxy->GetPhysicsThreadAPI()->X()[2], NoHitZ);
				};

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				ChaosTest::SetParticleSimDataToCollide({ Proxy->GetParticle_LowLevel() });


				Particle.SetX(FVec3(0, 0, 100));
				Particle.SetV(FVec3(0, 0, -10));

				auto SphereGeom = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
				FSingleParticlePhysicsProxy* SecondSphere = nullptr;

				// Make particles
				SecondSphere = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
				auto& SecondSphereParticle = SecondSphere->GetGameThreadAPI();

				SecondSphereParticle.SetGravityEnabled(false);
				SecondSphereParticle.SetX(FVec3(0, 0, 10));	//if resim doesn't reset collision, this will be in the Sphere's way
				SecondSphereParticle.SetGeometry(SphereGeom);
				Solver->RegisterObject(SecondSphere);

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					if (Step == 15)
					{
						//enable collision after particle passed
						ChaosTest::SetParticleSimDataToCollide({ SecondSphere->GetParticle_LowLevel() });
					}

					TickSolverHelper(Solver);
				}

				Solver->UnregisterObject(SecondSphere);
			});
	}

	GTEST_TEST(AllTraits, RewindTest_ResimSleepChange)
	{
		//change object state on physics thread, and make sure the state change is properly recorded in rewind data
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver);
				Helper->ProcessInputsFunc = [Proxy, RewindData = Solver->GetRewindData()](int32 PhysicsStep, bool bResim)
					{
						for (int32 Step = 0; Step < PhysicsStep; ++Step)
						{
							const EObjectStateType OldState = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), Step).ObjectState();
							if (Step < 4)	//we set to sleep on step 3, which means we won't see it as input state until 4
							{
								EXPECT_EQ(OldState, EObjectStateType::Dynamic);
							}
							else
							{
								EXPECT_EQ(OldState, EObjectStateType::Sleeping);
							}
						}

						if (PhysicsStep == 3)
						{
							Proxy->GetPhysicsThreadAPI()->SetObjectState(EObjectStateType::Sleeping);
						}
				};

				const int32 LastGameStep = 32;

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				Particle.SetV(FVec3(0, 0, 1));

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_ModifyVelocityFromSimCallback)
	{
		//modify velocity from sim callback and from solver
		//rewind data should give us the velocity _before_ the sim callback (i.e. the solver is not stomping with the wrong PreV)
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver);
				Helper->ProcessInputsFunc = [Proxy, RewindData = Solver->GetRewindData(), SimDt](int32 PhysicsStep, bool bResim)
					{
						if (SimDt * PhysicsStep == 4)
						{
							Proxy->GetPhysicsThreadAPI()->SetV(FVec3(0, 0, 0));
						}

						for (int32 Step = 0; Step < PhysicsStep; ++Step)
						{
							const FReal OldV = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), Step, Chaos::FFrameAndPhase::PrePushData).GetV()[2];
							const FReal Time = Step * SimDt;

							if (Time == 0)
							{
								// First frame is not cached since the proxy is marked as Uninitialized at that point
							}
							else if (Time <= 2)	//simple acceleration
							{
								EXPECT_NEAR(OldV, -Time, 1e-2);
							}
							else if (Time <= 3)	//simple acceleration after GT reset the velocity on step 2
							{
								EXPECT_NEAR(OldV, -(Time - 2), 1e-2);
							}
							else if (Time <= 4)	//we set velocity at time = 4, so rewind data will not see it until frame 5
							{
								if (SimDt > 2)	//if SimDt is this large, the reset of zero at time 2 is swallowed by first step so we don't really know about it
								{
									EXPECT_NEAR(OldV, -Time, 1e-2);
								}
								else
								{
									//otherwise we see integration but from time 2 instead of time 0
									EXPECT_NEAR(OldV, -(Time - 2), 1e-2);
								}
							}
							else
							{
								//everyone sees reset after time 4 so integration is from that time
								EXPECT_NEAR(OldV, -(Time - 4), 1e-2);
							}
						}
				};

				const int32 LastGameStep = 32;
				const int32 NumPhysSteps = FMath::TruncToInt(LastGameStep / SimDt);

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(true);
				Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -1), 0);

				Particle.SetV(FVec3(0, 0, 0));

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					if (Step == 2)
					{
						Particle.SetV(FVec3(0, 0, 0));	//reset velocity
					}
					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_SpawnEarlierCorrection)
	{
		// Test resim when object spawned earlier as part of correction
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				FSingleParticlePhysicsProxy* Floor = nullptr;
				bool bHasResimmed = false;

				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver);
				Helper->TriggerRewindFunc = [&Floor, &bHasResimmed](int32 PhysicsStep) -> int32
				{
					if (!bHasResimmed && Floor && Floor->GetPhysicsThreadAPI())
					{
						bHasResimmed = true;
						return 0;
					}
					return INDEX_NONE;
				};

				Helper->ProcessInputsFunc = [Proxy, RewindData = Solver->GetRewindData(), SimDt, &Floor, &bHasResimmed](int32 PhysicsStep, bool bIsResimming)
					{
						const FReal Time = PhysicsStep * SimDt;

						if (bIsResimming && RewindData)
						{
							if (PhysicsStep == 1)
							{
								RewindData->SpawnProxyIfNeeded(*Floor);
							}
						}
						if(Proxy)
						{ 
							if (FRigidBodyHandle_Internal* PT0 = Proxy->GetPhysicsThreadAPI())
							{ 
								if (!bHasResimmed || Time < 4.5)
								{
									//simply movement without hitting floor because it's spawned too late
									const float ExpectedLocation = FMath::Max(1.5, 14.5 - Time);
									EXPECT_NEAR(PT0->X()[2], ExpectedLocation, 1e-2);
								}
								else
								{
									//floor spawned earlier so we hit it
									EXPECT_GE(PT0->X()[2], 10 - 1e-2);
								}
							}
						}
				};

				const int32 LastGameStep = 32;

				auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal,3>(FVec3(-100, -100, -1), FVec3(100, 100, 0)));

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				Particle.SetV(FVec3(0, 0, -1));
				Particle.SetX(FVec3(0, 0, 14.5));

				ChaosTest::SetParticleSimDataToCollide({ Proxy->GetParticle_LowLevel() });

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					TickSolverHelper(Solver);

					//spawn floor way late to ensure no collision on first run
					if (Step == 12)
					{
						Floor = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
						auto& FloorParticle = Floor->GetGameThreadAPI();
						FloorParticle.SetGeometry(FloorGeom);
						Solver->RegisterObject(Floor);
						ChaosTest::SetParticleSimDataToCollide({ Floor->GetParticle_LowLevel() });
					}
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_SpawnEarlierCorrection2)
	{
		// Test resim when object spawned earlier as part of correction
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				FSingleParticlePhysicsProxy* SpawnedProxy = nullptr;
				FSingleParticlePhysicsProxy* SpawnedProxyNoCorrection = nullptr;

				bool bHasResimmed = false;

				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver);
				Helper->TriggerRewindFunc = [&SpawnedProxy, &SpawnedProxyNoCorrection, &bHasResimmed](int32 PhysicsStep) -> int32
				{
					if(!bHasResimmed && SpawnedProxy && SpawnedProxy->GetPhysicsThreadAPI())
					{
						bHasResimmed = true;
						return 0;
					}
					
					return INDEX_NONE;
				};

				Helper->ProcessInputsFunc = [RewindData = Solver->GetRewindData(), SimDt, &SpawnedProxy, &SpawnedProxyNoCorrection, &bHasResimmed](int32 PhysicsStep, bool bIsResimming)
				{
					const FReal Time = PhysicsStep * SimDt;

					if(SpawnedProxy && RewindData)
					{ 
						if(FRigidBodyHandle_Internal* PT0 = SpawnedProxy->GetPhysicsThreadAPI())
						{ 
							if (bIsResimming && (PhysicsStep == 10))
							{
								RewindData->SpawnProxyIfNeeded(*SpawnedProxy);
								PT0->SetX(FVec3(500, 0, 100.0));
							}

							if (SpawnedProxyNoCorrection)
							{
								if (FRigidBodyHandle_Internal* PT1 = SpawnedProxyNoCorrection->GetPhysicsThreadAPI())  
								{
									// After we've applied the correction and simmed past frame 10, we expect the first proxy to be "ahead" of the second one that didn't get corrected
									if (bHasResimmed && PhysicsStep > 10)
									{
										EXPECT_LT(PT0->X()[2], PT1->X()[2]);
									}
								}
							}	
						}
					}
				};

				const int32 LastGameStep = 32;
				
				auto SphereGeom = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				Particle.SetV(FVec3(0, 0, -1));
				Particle.SetX(FVec3(0, 0, 14.5));

				ChaosTest::SetParticleSimDataToCollide({ Proxy->GetParticle_LowLevel() });

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					TickSolverHelper(Solver);

					if (Step == 12)
					{
						// Make particles. E.g:
						//	we just found out from the server these were spawned and these are the latest positions replicated from the server.
						//	but actually these positions are the server positions from frame 10. Lets see if we can correct this.
						{
							SpawnedProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
							auto& GT = SpawnedProxy->GetGameThreadAPI();

							GT.SetV(FVec3(0, 0, 0.0));
							GT.SetX(FVec3(500, 0, 100.0));

							GT.SetGeometry(SphereGeom);
							Solver->RegisterObject(SpawnedProxy);
							ChaosTest::SetParticleSimDataToCollide({ SpawnedProxy->GetParticle_LowLevel() });
						}

						{
							SpawnedProxyNoCorrection = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
							auto& GT = SpawnedProxyNoCorrection->GetGameThreadAPI();

							GT.SetV(FVec3(0, 0, 0.0));
							GT.SetX(FVec3(100, 0, 100.0));

							GT.SetGeometry(SphereGeom);
							Solver->RegisterObject(SpawnedProxyNoCorrection);
							ChaosTest::SetParticleSimDataToCollide({ SpawnedProxyNoCorrection->GetParticle_LowLevel() });
						}
					}
				}
			});
	}

	GTEST_TEST(AllTraits, RewindTest_MovingToNotMovingInterpolation)
	{
		//
		//This tests that even though it's only dirty for one frame, we still interpolate it over many
		//Makes sure rewind data is still passed back to GT even though particle is asleep before and after it moves (i.e. not dirty during rewind step)
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				//only care about resim when async results are used
				if (Solver->IsUsingAsyncResults() == false)
				{
					return;
				}

				const FReal ResimTime = 20;
				const FReal SleepTime = 12;

				bool bHasResimmed = false;
				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver);
				Helper->TriggerRewindFunc = [&bHasResimmed, ResimTime, SimDt](int32 PhysicsStep) -> int32
				{
					const FReal Time = PhysicsStep * SimDt;
					if (!bHasResimmed && Time == ResimTime)
				{
						bHasResimmed = true;
						return 2;
					}

					return INDEX_NONE;
				};

				Helper->ProcessInputsFunc = [Proxy, RewindData = Solver->GetRewindData(), SimDt, ResimTime, SleepTime, &bHasResimmed](int32 PhysicsStep, bool bIsResimming)
					{
						const FReal Time = PhysicsStep * SimDt;

					if (bHasResimmed)
						{


							if (Proxy->GetPhysicsThreadAPI()->V()[2] == 1)
							{
								Proxy->GetPhysicsThreadAPI()->SetV(FVec3(0, 0, 0));
							}


						}
						else
						{
							if (Time >= SleepTime && Proxy->GetPhysicsThreadAPI()->ObjectState() == EObjectStateType::Dynamic)
							{
								Proxy->GetPhysicsThreadAPI()->SetObjectState(EObjectStateType::Sleeping);
							}
						}
				};

				const int32 LastGameStep = 64;

				auto FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100, -100, -1), FVec3(100, 100, 0)));

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				Particle.SetV(FVec3(0, 0, 0));
				Particle.SetX(FVec3(0, 0, 0));

				FReal Time = 0;
				const FReal StartMovingTime = 4;
				const FReal StartMovingTimeDiscrete = FMath::FloorToInt(StartMovingTime / SimDt) * SimDt;
				const FReal GTDt = 1;
				const FReal InterpStartTime = ResimTime + SimDt;
				const FReal SleepLocation = SleepTime - StartMovingTimeDiscrete;
				const FReal CorrectedLocation = SimDt <= 1 ? 0 : 4;
				FReal PrevZDuringInterp = SleepLocation;	//shouldn't be further then this because already interpolating back to 0

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{

					if (Time == StartMovingTime)
					{
						Particle.SetV(FVec3(0, 0, 1));
					}

					TickSolverHelper(Solver);

					Time += GTDt;
					const FReal InterpolatedTime = Time - SimDt * Solver->GetAsyncInterpolationMultiplier();


					if (InterpolatedTime <= InterpStartTime)
					{
						if(InterpolatedTime < StartMovingTimeDiscrete)
						{
							//No movement yet
							EXPECT_NEAR(Particle.X()[2], 0, 1e-2);
						}
						else
						{
							//simple movement with constant velocity until goes to sleep
							if(InterpolatedTime >= SleepTime)
							{
								EXPECT_NEAR(Particle.X()[2], SleepLocation, 1e-2);
							}
							else
							{
								EXPECT_NEAR(Particle.X()[2], InterpolatedTime - StartMovingTimeDiscrete, 1e-2);
							}
						}
					}
					else
					{
						//leash mode
						EXPECT_GE(Particle.X()[2], CorrectedLocation);
						EXPECT_LE(Particle.X()[2], PrevZDuringInterp);
						PrevZDuringInterp = Particle.X()[2];
					}
				}

			});
	}


	GTEST_TEST(AllTraits, RewindTest_ResimInSync)
	{
		//apply forces in the same way until resim step 4 which should trigger a desync
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				struct FRewindCallback : public IRewindCallback
				{
					FSingleParticlePhysicsProxy* Proxy;
					FRewindData* RewindData;

					virtual void ProcessInputs_Internal(int32 PhysicsStep, const TArray<FSimCallbackInputAndObject>& SimCallbackInputs) override
					{
						bool bIsResimming = RewindData->IsResim();
						if(PhysicsStep == 2)
						{
							Proxy->GetPhysicsThreadAPI()->AddForce(FVec3(1, 0, 0));
						}

						if(bIsResimming && PhysicsStep == 4)
						{
							Proxy->GetPhysicsThreadAPI()->AddForce(FVec3(1, 0, 0));	//cause a desync
						}

						if(bIsResimming)
						{
							if(PhysicsStep <= 4)
							{
								EXPECT_EQ(Proxy->GetHandle_LowLevel()->SyncState(), ESyncState::InSync);
							}
							else
							{
								EXPECT_EQ(Proxy->GetHandle_LowLevel()->SyncState(), ESyncState::HardDesync);
							}
						}
						else
						{
							EXPECT_EQ(Proxy->GetHandle_LowLevel()->SyncState(), ESyncState::InSync);
						}
					}

					virtual int32 TriggerRewindIfNeeded_Internal(int32 LastCompletedStep) override
					{
						if (LastCompletedStep == ResimEndFrame)
						{
							return ResimStartFrame;
						}
						return INDEX_NONE;
					}

					int32 ResimStartFrame = 1;
					int32 ResimEndFrame = 10;
				};

				const int32 LastGameStep = 32;
				const int32 NumPhysSteps = FMath::TruncToInt(LastGameStep / SimDt);

				auto UniqueRewindCallback = MakeUnique<FRewindCallback>();
				auto RewindCallback = UniqueRewindCallback.Get();
				RewindCallback->Proxy = Proxy;
				RewindCallback->RewindData = Solver->GetRewindData();

				Solver->SetRewindCallback(MoveTemp(UniqueRewindCallback));

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				Particle.SetV(FVec3(0, 0, 1));

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					if(Step == 3)
					{
						Particle.AddForce(FVec3(0, 0, -10));
					}
					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_ResimInSync2)
	{
		//different velocity during resim step 5 which should cause a desync
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				struct FRewindCallback : public IRewindCallback
				{
					FSingleParticlePhysicsProxy* Proxy;
					FRewindData* RewindData;

					virtual void ProcessInputs_Internal(int32 PhysicsStep, const TArray<FSimCallbackInputAndObject>& SimCallbackInputs) override
					{
						bool bIsResimming = RewindData->IsResim();
						if (PhysicsStep == 2)
						{
							Proxy->GetPhysicsThreadAPI()->SetV(FVec3(0,0,2));
						}

						if (bIsResimming)
						{
							if(PhysicsStep == 3)
							{
								//only setting velocity during resim, but exact same value so should still be in sync
								Proxy->GetPhysicsThreadAPI()->SetV(FVec3(0, 0, 2));
							}

							if(PhysicsStep == 5)
							{
								Proxy->GetPhysicsThreadAPI()->SetV(FVec3(0, 0, 5));
							}
						}

						if (bIsResimming)
						{
							EXPECT_EQ(Proxy->GetHandle_LowLevel()->SyncState(), PhysicsStep <= 5 ? ESyncState::InSync : ESyncState::HardDesync);
						}
						else
						{
							EXPECT_EQ(Proxy->GetHandle_LowLevel()->SyncState(), ESyncState::InSync);
						}
					}

					virtual int32 TriggerRewindIfNeeded_Internal(int32 LastCompletedStep) override
					{
						if (LastCompletedStep == ResimEndFrame)
						{
							return ResimStartFrame;
						}

						return INDEX_NONE;
					}

					int32 ResimStartFrame = 1;
					int32 ResimEndFrame = 10;
				};

				const int32 LastGameStep = 32;
				const int32 NumPhysSteps = FMath::TruncToInt(LastGameStep / SimDt);

				auto UniqueRewindCallback = MakeUnique<FRewindCallback>();
				auto RewindCallback = UniqueRewindCallback.Get();
				RewindCallback->Proxy = Proxy;
				RewindCallback->RewindData = Solver->GetRewindData();

				Solver->SetRewindCallback(MoveTemp(UniqueRewindCallback));

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				Particle.SetV(FVec3(0, 0, 1));

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_ResimInSync3)
	{
		//different position during resim step 5 which should cause a desync
		//want a completely clean property to make sure we properly update resim buffer when property is dirtied only on second run
		TRewindHelper::TestEmpty([](auto* Solver, FReal SimDt, int32 Optimization)
			{
				struct FRewindCallback : public IRewindCallback
				{
					FSingleParticlePhysicsProxy* Proxy;
					FRewindData* RewindData;

					virtual void ProcessInputs_Internal(int32 PhysicsStep, const TArray<FSimCallbackInputAndObject>& SimCallbackInputs) override
					{
						bool bIsResimming = RewindData->IsResim();
						if (bIsResimming)
						{
							if (PhysicsStep == 5)
							{
								Proxy->GetPhysicsThreadAPI()->SetX(FVec3(0, 0, 5));
							}
						}

						if (bIsResimming)
						{
							EXPECT_EQ(Proxy->GetHandle_LowLevel()->SyncState(), PhysicsStep <= 5 ? ESyncState::InSync : ESyncState::HardDesync);
						}
						else
						{
							EXPECT_EQ(Proxy->GetHandle_LowLevel()->SyncState(), ESyncState::InSync);
						}
					}

					virtual int32 TriggerRewindIfNeeded_Internal(int32 LastCompletedStep) override
					{
						if (LastCompletedStep == ResimEndFrame)
						{
							return ResimStartFrame;
						}
						return INDEX_NONE;
					}

					int32 ResimStartFrame = 1;
					int32 ResimEndFrame = 10;
				};

				auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
				auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
				Proxy->GetGameThreadAPI().SetGeometry(Sphere);
				Solver->RegisterObject(Proxy);

				const int32 LastGameStep = 32;
				const int32 NumPhysSteps = FMath::TruncToInt(LastGameStep / SimDt);

				auto UniqueRewindCallback = MakeUnique<FRewindCallback>();
				auto RewindCallback = UniqueRewindCallback.Get();
				RewindCallback->Proxy = Proxy;
				RewindCallback->RewindData = Solver->GetRewindData();

				Solver->SetRewindCallback(MoveTemp(UniqueRewindCallback));

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_ResimInSync4)
	{
		//different position during resim step 5 which should cause a desync
		//want a completely clean property to make sure we properly update resim buffer when property is dirtied only on second run
		//add a dirty property every frame to make sure we are getting an entry in the original sim, but with a clean property
		//unrelated property must be dirtied on GT because simcallback dirty just copies all properties at the moment
		TRewindHelper::TestEmpty([](auto* Solver, FReal SimDt, int32 Optimization)
			{
				//this test requires dirty writes from GT
				//we want those to line up with PT tick 1 to 1
				if (SimDt != 1) { return; }

				struct FRewindCallback : public IRewindCallback
				{
					FSingleParticlePhysicsProxy* Proxy;
					FRewindData* RewindData;

					virtual void ProcessInputs_Internal(int32 PhysicsStep, const TArray<FSimCallbackInputAndObject>& SimCallbackInputs) override
					{
						bool bIsResimming = true;
						if (PhysicsStep > HighestStep)
						{
							bIsResimming = false;
							HighestStep = PhysicsStep;
						}
						ensure(bIsResimming == RewindData->IsResim()); // this would catch if IsResim is lieing to us

						if (bIsResimming)
						{
							if (PhysicsStep == 5)
							{
								Proxy->GetPhysicsThreadAPI()->SetX(FVec3(0, 0, 5));
							}
						}

						if(bIsResimming)
						{
							EXPECT_EQ(Proxy->GetHandle_LowLevel()->SyncState(), PhysicsStep <= 5 ? ESyncState::InSync : ESyncState::HardDesync);
						}
						else
						{
							EXPECT_EQ(Proxy->GetHandle_LowLevel()->SyncState(), ESyncState::InSync);
						}
					}

					virtual int32 TriggerRewindIfNeeded_Internal(int32 LastCompletedStep) override
					{
						if (LastCompletedStep == ResimEndFrame)
						{
							return ResimStartFrame;
						}

						return INDEX_NONE;
					}

					int32 ResimStartFrame = 1;
					int32 ResimEndFrame = 10;

					int32 HighestStep = INDEX_NONE;

				};

				auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
				auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
				Proxy->GetGameThreadAPI().SetGeometry(Sphere);
				Solver->RegisterObject(Proxy);


				const int32 LastGameStep = 32;
				const int32 NumPhysSteps = FMath::TruncToInt(LastGameStep / SimDt);

				auto UniqueRewindCallback = MakeUnique<FRewindCallback>();
				auto RewindCallback = UniqueRewindCallback.Get();
				RewindCallback->Proxy = Proxy;
				RewindCallback->RewindData = Solver->GetRewindData();

				Solver->SetRewindCallback(MoveTemp(UniqueRewindCallback));

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					Proxy->GetGameThreadAPI().SetV(FVec3(0, 0, Step));	//dirty an unrelated property to still get an entry in the history buffer

					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_ResimSleepChangeRewind)
	{
		// Test puting object to sleep during Resim
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				int32 ExpectedSleepFrame = INDEX_NONE;
				const int32 ResimStartFrame = 1;
				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver, 11, ResimStartFrame);
				Helper->ProcessInputsFunc = [&ExpectedSleepFrame, ResimStartFrame, Proxy, RewindData = Solver->GetRewindData()](const int32 PhysicsStep, bool bIsResimming)
					{
						if (bIsResimming)
						{
						if (PhysicsStep >= ResimStartFrame + 2)
							{
								Proxy->GetPhysicsThreadAPI()->SetObjectState(EObjectStateType::Sleeping, false, false);
							ExpectedSleepFrame = ResimStartFrame + 3;
							}
							else
							{
								Proxy->GetPhysicsThreadAPI()->SetObjectState(EObjectStateType::Dynamic, false, false);
							}
							return;
						}

						for (int32 Step = ResimStartFrame; Step < PhysicsStep; ++Step)
						{
							const EObjectStateType OldState = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), Step).ObjectState();

							if (ExpectedSleepFrame != INDEX_NONE && Step >= ExpectedSleepFrame)
							{
								EXPECT_EQ(OldState, EObjectStateType::Sleeping);
							}
							else
							{
							EXPECT_EQ(OldState, EObjectStateType::Dynamic);
						}
					}
				};

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				Particle.SetV(FVec3(0, 0, 1));

				for (int Step = 0; Step <= 32; ++Step)
				{
					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_ResimSleepChangeRewind2)
	{
		// This does two ObjectState corrections
		//	-Object starts out asleep
		//	-On step 10 we force a rewind to frame 4 and wake it up during the resim (apply "correction")
		//		-This on its own works fine
		//	-On step 12 we force another rewind to frame 6 and put it to sleep during the resim
		//		-This seems to have no effect: the object stays awake
		//
		//
	
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				struct FRewindCallback : public IRewindCallback
				{
					FSingleParticlePhysicsProxy* Proxy;
					FRewindData* RewindData;

					virtual void ProcessInputs_Internal(int32 PhysicsStep, const TArray<FSimCallbackInputAndObject>& SimCallbackInputs) override
					{
						if (bIsResimming)
						{
							if (ForceAwakeFrame != INDEX_NONE && PhysicsStep == ForceAwakeFrame)
							{
								Proxy->GetPhysicsThreadAPI()->SetObjectState(EObjectStateType::Dynamic);
							}

							if (ForceSleepFrame != INDEX_NONE && PhysicsStep == ForceSleepFrame)
							{
								Proxy->GetPhysicsThreadAPI()->SetObjectState(EObjectStateType::Sleeping);
							}
							
							return;
						}

						for (int32 Step = ResimStartFrame; Step < PhysicsStep; ++Step)
						{
							const EObjectStateType OldState = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), Step).ObjectState();
							if (TestAwakeFrame == INDEX_NONE || Step < TestAwakeFrame)
							{
								// If we haven't applied the first awake correction yet, or this is a step prior to that correction, we should be asleep
								EXPECT_EQ(OldState, EObjectStateType::Sleeping);
							}
							else if (TestSleepFrame != INDEX_NONE && Step >= TestSleepFrame)
							{
								// If we *have* applied the second sleep correction and this is a step on or after that correction, we should be asleep
								EXPECT_EQ(OldState, EObjectStateType::Sleeping);
							}
							else
							{
								// Everything else falls in the middle, we should be awake
								EXPECT_EQ(OldState, EObjectStateType::Dynamic);
							}
						}
					}

					virtual int32 TriggerRewindIfNeeded_Internal(int32 LastCompletedStep) override
					{
						if (bIsResimming)
						{
							return INDEX_NONE;
						}

						if (LastCompletedStep == 10)
						{
							ForceAwakeFrame = 4;
							bIsResimming = true;
							ResimEndFrame = LastCompletedStep;
							TestAwakeFrame = ForceAwakeFrame+1;
							ResimStartFrame = ForceAwakeFrame;
							return ResimStartFrame;
						}

						if (LastCompletedStep == 12)
						{
							ForceSleepFrame = 6;
							bIsResimming = true;
							ResimEndFrame = LastCompletedStep;
							TestSleepFrame = ForceSleepFrame+1;
							ResimStartFrame = ForceSleepFrame;
							return ResimStartFrame;
						}

						return INDEX_NONE;
					}

					void PostResimStep_Internal(int32 PhysicsStep) override
					{
						if (ResimEndFrame == PhysicsStep)
						{
							ForceAwakeFrame = INDEX_NONE;
							ForceSleepFrame = INDEX_NONE;
							bIsResimming = false;
						}
					}
					
					int32 ForceAwakeFrame = INDEX_NONE;
					int32 ForceSleepFrame = INDEX_NONE;

					int32 TestAwakeFrame = INDEX_NONE;
					int32 TestSleepFrame = INDEX_NONE;
					int32 ResimStartFrame = 0;
					
					bool bIsResimming = false;
					int32 ResimEndFrame = INDEX_NONE;
				};

				const int32 LastGameStep = 32;
				const int32 NumPhysSteps = FMath::TruncToInt(LastGameStep / SimDt);

				auto UniqueRewindCallback = MakeUnique<FRewindCallback>();
				auto RewindCallback = UniqueRewindCallback.Get();
				RewindCallback->Proxy = Proxy;
				RewindCallback->RewindData = Solver->GetRewindData();

				Solver->SetRewindCallback(MoveTemp(UniqueRewindCallback));

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				Particle.SetV(FVec3(0, 0, 1));
				Particle.SetObjectState(EObjectStateType::Sleeping);

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_RewindBeforeSleep)
	{
		// Rewind to before an object was put to sleep and see that the Active view is valid
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				int32 ExpectedSleepFrame = INDEX_NONE;
				const int32 ResimStartFrame = 0;
				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver, 11, ResimStartFrame);
				Helper->ProcessInputsFunc = [&ExpectedSleepFrame, ResimStartFrame, Proxy, RewindData = Solver->GetRewindData(), Solver](const int32 PhysicsStep, bool bIsResimming)
				{
					if(PhysicsStep == 5)
					{
						Proxy->GetPhysicsThreadAPI()->SetV(FVec3(0, 0, 0));
						Proxy->GetPhysicsThreadAPI()->SetObjectState(EObjectStateType::Sleeping);
					}

					//before sleep the active view contains the particle, after we put it to sleep the active view is empty
					EXPECT_EQ(Solver->GetParticles().GetActiveParticlesView().Num(), PhysicsStep < 5 ? 1 : 0);
				};

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				Particle.SetV(FVec3(0, 0, 1));

				for (int Step = 0; Step <= 32; ++Step)
				{
					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_RewindBeforeAwake)
	{
		// Rewind to before an object was made awake and confirm that the active view is valid
		// This test rewinds to step 3 because it ensures there's no PushData which may fixup the view. Need to make sure it's fixed regardless of push data
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				int32 ExpectedSleepFrame = INDEX_NONE;
				const int32 ResimStartFrame = 3;
				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver, 13, ResimStartFrame);
				Helper->ProcessInputsFunc = [&ExpectedSleepFrame, ResimStartFrame, Proxy, RewindData = Solver->GetRewindData(), Solver](const int32 PhysicsStep, bool bIsResimming)
				{
					if (PhysicsStep == 12)
					{
						Proxy->GetPhysicsThreadAPI()->SetV(FVec3(0, 0, 1));
						Proxy->GetPhysicsThreadAPI()->SetObjectState(EObjectStateType::Dynamic);
					}

					//before wake up the active view was empty, after we wake it up the view contains the particle
					EXPECT_EQ(Solver->GetParticles().GetActiveParticlesView().Num(), PhysicsStep < 12 ? 0 : 1);
				};

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				Particle.SetObjectState(EObjectStateType::Sleeping);

				for (int Step = 0; Step <= 32; ++Step)
				{
					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_RewindBeforeMadeKinematic)
	{
		// Rewind before a dynamic was made kinematic and check the view is properly handling the rewind
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				int32 ExpectedSleepFrame = INDEX_NONE;
				const int32 ResimStartFrame = 3;
				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver, 13, ResimStartFrame);
				Helper->ProcessInputsFunc = [&ExpectedSleepFrame, ResimStartFrame, Proxy, RewindData = Solver->GetRewindData(), Solver](const int32 PhysicsStep, bool bIsResimming)
				{
					if (PhysicsStep == 12)
					{
						Proxy->GetPhysicsThreadAPI()->SetObjectState(EObjectStateType::Kinematic);
					}

					if(PhysicsStep < 12)
					{
						EXPECT_EQ(Solver->GetParticles().GetNonDisabledDynamicView().Num(), 1);
						EXPECT_EQ(Solver->GetParticles().GetActiveKinematicParticlesView().Num(), 0);
					}
					else
					{
						EXPECT_EQ(Solver->GetParticles().GetNonDisabledDynamicView().Num(), 0);
						EXPECT_EQ(Solver->GetParticles().GetActiveKinematicParticlesView().Num(), 1);
					}
				};

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				Particle.SetV(FVec3(0, 0, 1));

				for (int Step = 0; Step <= 32; ++Step)
				{
					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_RewindBeforeMadeDynamic)
	{
		// Rewind before a kinematic was made dynamic and check the view is properly handling the rewind
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				int32 ExpectedSleepFrame = INDEX_NONE;
				const int32 ResimStartFrame = 3;
				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver, 13, ResimStartFrame);
				Helper->ProcessInputsFunc = [&ExpectedSleepFrame, ResimStartFrame, Proxy, RewindData = Solver->GetRewindData(), Solver](const int32 PhysicsStep, bool bIsResimming)
				{
					if (PhysicsStep == 12)
					{
						Proxy->GetPhysicsThreadAPI()->SetObjectState(EObjectStateType::Dynamic);
					}

					if (PhysicsStep < 12)
					{
						EXPECT_EQ(Solver->GetParticles().GetNonDisabledDynamicView().Num(), 0);
						EXPECT_EQ(Solver->GetParticles().GetActiveKinematicParticlesView().Num(), 1);
					}
					else
					{
						EXPECT_EQ(Solver->GetParticles().GetNonDisabledDynamicView().Num(), 1);
						EXPECT_EQ(Solver->GetParticles().GetActiveKinematicParticlesView().Num(), 0);
					}
				};

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);
				Proxy->GetGameThreadAPI().SetObjectState(EObjectStateType::Kinematic);

				for (int Step = 0; Step <= 32; ++Step)
				{
					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_RecordForcesInSimCallback)
	{
		//Makes sure that we record the forces applied during sim callback
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver);
				Helper->ProcessInputsFunc = [Proxy, RewindData = Solver->GetRewindData(), Optimization](const int32 PhysicsStep, bool bIsResimming)
					{
						if (PhysicsStep == 3)
						{
							Proxy->GetPhysicsThreadAPI()->SetAcceleration(FVec3(0, 0, 10) + Proxy->GetPhysicsThreadAPI()->Acceleration(), 0);
							Proxy->GetPhysicsThreadAPI()->SetAngularAcceleration(FVec3(0, 0, 10) + Proxy->GetPhysicsThreadAPI()->AngularAcceleration());
						}

						for (int32 Step = 0; Step < PhysicsStep; ++Step)
						{
							{
								//Before push no force or torque at all
								FGeometryParticleState PrePushState = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), Step, Chaos::FFrameAndPhase::PrePushData);
								EXPECT_EQ(PrePushState.Acceleration()[2], 0);
								EXPECT_EQ(PrePushState.AngularAcceleration()[2], 0);
							}

							if (Optimization != 2) // Optimization 2 does not cache data in PostPushState so no need to check that phase
							{
								//After push (but before callback) only see GT force, and no torque
								FGeometryParticleState PostPushState = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), Step, Chaos::FFrameAndPhase::PostPushData);
								EXPECT_EQ(PostPushState.Acceleration()[2], 1);	//GT always sets force of 1
								EXPECT_EQ(PostPushState.AngularAcceleration()[2], 0);	//GT never sets torque
							}

							if (Optimization != 2) // Optimization 2 does not cache data in PostCallbacks so no need to check that phase
							{
							FGeometryParticleState State = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), Step, Chaos::FFrameAndPhase::PostCallbacks);
								if (Step == 3)
								{
									EXPECT_EQ(State.Acceleration()[2], 11);	//1 from GT + 10 from callback
									EXPECT_EQ(State.AngularAcceleration()[2], 10);	//10 from callback (nothing from GT)
								}
								else
								{
									EXPECT_EQ(State.Acceleration()[2], 1);	//GT always sets force of 1
									EXPECT_EQ(State.AngularAcceleration()[2], 0);	//GT never sets torque
								}
							}
						}
					};

				auto& Particle = Proxy->GetGameThreadAPI();
				Particle.SetGravityEnabled(false);

				for (int Step = 0; Step <= 32; ++Step)
				{
					Particle.AddForce(FVec3(0, 0, 1));
					TickSolverHelper(Solver);
				}

			});
	}

	GTEST_TEST(AllTraits, RewindTest_AddForce)
	{
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
		{
			auto& Particle = Proxy->GetGameThreadAPI();
			Particle.SetGravityEnabled(false);

			const int32 LastGameStep = 20;

			for (int Step = 0; Step <= LastGameStep; ++Step)
			{
				//sim-writable property that changes every step
				Particle.AddForce(FVec3(0, 0, Step + 1));
				TickSolverHelper(Solver);
			}

			const FRewindData* RewindData = Solver->GetRewindData();

			//check state at every step except latest
			const int32 LastSimStep = LastGameStep / SimDt;
			FReal ExpectedX = 0;
			FReal ExpectedVelocity = 0;
			FReal PreviousForce = 0;

			for (int Step = 0; Step < LastSimStep - 1; ++Step)
			{
				const auto ParticleState = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), Step);
				FReal ExpectedForce = Step + 1;

				if (SimDt < 1)
				{
					//each sub-step gets a constant force applied
					ExpectedForce = FMath::FloorToFloat(Step * SimDt) + 1;
				}
				else if (SimDt > 1)
				{
					//each step gets an average of the forces applied ((step+1) + (step+2) + (step+3) + (step+4))/4 = step + (1+2+3+4)/4 = step + 2.5
					//where step is game step: so really it's step * 4
					ExpectedForce = Step * 4 + 2.5;
				}

				ExpectedVelocity += PreviousForce * SimDt;
				ExpectedX += ExpectedVelocity * SimDt;

				EXPECT_EQ(ParticleState.GetV()[2], ExpectedVelocity);
				EXPECT_EQ(ParticleState.GetX()[2], ExpectedX);

				/** With Optimization 2 we only cache PrePushData where Acceleration is always 0. Calling GetPastStateAtFrame for PostPushData will return next frames PrePushData since it's the next available data */
				if (Optimization == 2)
				{
					EXPECT_EQ(ParticleState.Acceleration()[2], 0);
				}
				else
				{
					EXPECT_EQ(ParticleState.Acceleration()[2], ExpectedForce);
				}
				PreviousForce = ExpectedForce;
			}
		});
	}

	GTEST_TEST(AllTraits, RewindTest_IntermittentForce)
	{
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
		{
			auto& Particle = Proxy->GetGameThreadAPI();
			Particle.SetGravityEnabled(false);
			const int32 LastGameStep = 20;

			for (int Step = 0; Step <= LastGameStep; ++Step)
			{
				//sim-writable property that changes infrequently and not at beginning
				if (Step == 3)
				{
					Particle.AddForce(FVec3(0, 0, Step));
				}

				if (Step == 5)
				{
					Particle.AddForce(FVec3(0, 0, Step));
				}

				TickSolverHelper(Solver);
			}

			const FRewindData* RewindData = Solver->GetRewindData();
			FReal ExpectedX = 0;
			FReal ExpectedVelocity = 0;
			FReal PreviousForce = 0;

			//check state at every step except latest
			const int32 LastSimStep = LastGameStep / SimDt;
			for (int Step = 0; Step < LastSimStep - 1; ++Step)
			{
				const float SimTime = Step * SimDt;

				const auto ParticleState = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), Step, (Optimization == 2 ? Chaos::FFrameAndPhase::PrePushData : Chaos::FFrameAndPhase::PostPushData));

				FReal ExpectedForce = 0;

				ExpectedVelocity += PreviousForce * SimDt;
				ExpectedX += ExpectedVelocity * SimDt;

				EXPECT_EQ(ParticleState.GetV()[2], ExpectedVelocity);
				EXPECT_EQ(ParticleState.GetX()[2], ExpectedX);

				/** With Optimization 2 we only cache PrePushData where Acceleration is always 0. */
				if (SimDt <= 1)
				{
					if (SimTime >= 3 && SimTime < 4)
					{
						ExpectedForce = 3;
						EXPECT_EQ(ParticleState.Acceleration()[2], (Optimization == 2) ? 0.0f : ExpectedForce);
					}
					else if (SimTime >= 5 && SimTime < 6)
					{
						ExpectedForce = 5;
						EXPECT_EQ(ParticleState.Acceleration()[2], (Optimization == 2) ? 0.0f : ExpectedForce);
					}
					else
					{
						EXPECT_EQ(ParticleState.Acceleration()[2], (Optimization == 2) ? 0.0f : ExpectedForce);
					}
				}
				else
				{
					//we get an average
					if (Step == 0)
					{
						ExpectedForce = 3 / 4.f;
						EXPECT_EQ(ParticleState.Acceleration()[2], (Optimization == 2) ? 0.0f : ExpectedForce);
					}
					else if (Step == 1)
					{
						ExpectedForce = 5 / 4.f;
						EXPECT_EQ(ParticleState.Acceleration()[2], (Optimization == 2) ? 0.0f : ExpectedForce);
					}
					else
					{
						EXPECT_EQ(ParticleState.Acceleration()[2], (Optimization == 2) ? 0.0f : ExpectedForce);
					}
				}
				PreviousForce = ExpectedForce;
			}
		});
	}

	GTEST_TEST(AllTraits, RewindTest_IntermittentGeomChange)
	{
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
		{
				auto& Particle = Proxy->GetGameThreadAPI();
			auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(0), FVec3(1)));
			auto Box2 = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(2), FVec3(3)));

			const int32 LastGameStep = 20;

			for (int Step = 0; Step <= LastGameStep; ++Step)
			{
				//property that changes once half way through
					if (Step == 3)
				{
						Particle.SetGeometry(Box);
				}

					if (Step == 5)
				{
						Particle.SetGeometry(Box2);
				}

					if (Step == 7)
				{
						Particle.SetGeometry(Box);
				}

				TickSolverHelper(Solver);
			}

			const FRewindData* RewindData = Solver->GetRewindData();

			//check state at every step except latest
			const int32 LastSimStep = LastGameStep / SimDt;
			for (int Step = 0; Step < LastSimStep - 1; ++Step)
			{
				/** Grab the cached data from the beginning of the next frame. When the particle is set by GT data we cache the pre-GT state in the RewindData's PrePushData phase,
				* the dirty GT data is separately cached as marshaled PushData and the result of the GT being applied will be cached in the next frames PrePushData. */
				const auto ParticleState = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), Step + 1, Chaos::FFrameAndPhase::PrePushData);
				if (SimDt <= 1)
				{
					const float SimTime = Step * SimDt;
					if (SimTime < 3)
					{
						//was sphere
						EXPECT_EQ(ParticleState.GetGeometry(), Sphere);
					}
					else if (SimTime < 5 || SimTime >= 7)
					{
						//then became box
						EXPECT_EQ(ParticleState.GetGeometry(), Box.GetReference());
					}
					else
					{
						//second box
						EXPECT_EQ(ParticleState.GetGeometry(), Box2.GetReference());
					}
				}
				else
				{
					//changes happen within interval so stays box entire time
					EXPECT_EQ(ParticleState.GetGeometry(), Box.GetReference());
				}
			}
		});
	}

	GTEST_TEST(AllTraits, RewindTest_FallingObjectWithTeleport)
	{
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
		{
			auto& Particle = Proxy->GetGameThreadAPI();
			Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -1), 0);
			Particle.SetGravityEnabled(true);
			Particle.SetX(FVec3(0, 0, 100));

			const int32 LastGameStep = 20;
			for (int Step = 0; Step <= LastGameStep; ++Step)
			{
				//teleport from GT
				if (Step == 5)
				{
					Particle.SetX(FVec3(0, 0, 10));
					Particle.SetV(FVec3(0, 0, 0));
				}

				TickSolverHelper(Solver);
			}

			const FRewindData* RewindData = Solver->GetRewindData();

			//check state at every step except latest
			const int32 LastSimStep = LastGameStep / SimDt;
			FReal ExpectedVZ = 0;
			FReal ExpectedXZ = 100;

			for (int Step = 0; Step < LastSimStep - 1; ++Step)
			{
				const auto ParticleState = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), Step, (Optimization == 2 ? Chaos::FFrameAndPhase::PrePushData : Chaos::FFrameAndPhase::PostPushData));

				const FReal SimStart = SimDt * Step;
				const FReal SimEnd = SimDt * (Step + 1);

				if (Optimization == 2)
				{
					/* Teleport was made on GT and is cached in PushData when Optimization 2 is enabled so we will have the first RewindData cached on the next physics frame after the teleport
					* Track expected V and X from the next frame by adding SimDt to the teleport frame and then step the expected V and X once*/
					if (SimStart <= (5 + SimDt) && SimEnd > (5 + SimDt))
					{
						ExpectedVZ = 0;
						ExpectedXZ = 10;

						ExpectedVZ -= SimDt;
						ExpectedXZ += ExpectedVZ * SimDt;
					}
				}
				else
				{
					if (SimStart <= 5 && SimEnd > 5)
					{
						ExpectedVZ = 0;
						ExpectedXZ = 10;
					}
				}

				EXPECT_NEAR(ParticleState.GetX()[2], ExpectedXZ, 1e-4);
				EXPECT_NEAR(ParticleState.GetV()[2], ExpectedVZ, 1e-4);

				// Step expected V and X once
				ExpectedVZ -= SimDt;
				ExpectedXZ += ExpectedVZ * SimDt;
			}
		});
	}

	struct FSimCallbackHelperInput : FSimCallbackInput
	{
		void Reset() {}
		int InCounter;
	};

	struct FSimCallbackHelperOutput : FSimCallbackOutput
	{
		void Reset() {}
		int OutCounter;
	};

	GTEST_TEST(AllTraits, RewindTest_SimCallbackInputsOutputs)
	{
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				struct FSimCallbackHelper : TSimCallbackObject<FSimCallbackHelperInput, FSimCallbackHelperOutput>
				{
					int32 TriggerCount = 0;
					virtual void OnPreSimulate_Internal() override
					{
						GetProducerOutputData_Internal().OutCounter = TriggerCount++;
					}
				};

				struct FRewindCallback : public IRewindCallback
				{
					FRewindCallback(int32 InNumPhysicsSteps, FReal InSimDt)
						: NumPhysicsSteps(InNumPhysicsSteps)
						, SimDt(InSimDt)
					{

					}

					TArray<int32> InCounters;
					int32 NumPhysicsSteps;
					bool bResim = false;
					FReal SimDt;

					virtual int32 TriggerRewindIfNeeded_Internal(int32 LastCompletedStep) override
					{
						if (LastCompletedStep + 1 == NumPhysicsSteps && NumPhysicsSteps != INDEX_NONE)
						{
							NumPhysicsSteps = INDEX_NONE;	//don't resim again after this
							return 0;
						}

						return INDEX_NONE;
					}

					virtual void ProcessInputs_External(int32 PhysicsStep, const TArray<FSimCallbackInputAndObject>& SimCallbackInputs) override
					{
						EXPECT_EQ(bResim, false);	//should never be triggered during resim, since resim happens on internal thread
						EXPECT_EQ(SimCallbackInputs.Num(), 1);
						FSimCallbackHelperInput* Input = static_cast<FSimCallbackHelperInput*>(SimCallbackInputs[0].Input);
						if(SimDt > 1)
						{
							//several external ticks before we finally get the final input
							EXPECT_EQ(FMath::TruncToInt((PhysicsStep+1) * SimDt - 1), Input->InCounter);
						}
						else
						{
							//potentially the same input over multiple sub-steps
							EXPECT_EQ(FMath::TruncToInt(PhysicsStep * SimDt), Input->InCounter);
						}
					}

					virtual void ProcessInputs_Internal(int32 PhysicsStep, const TArray<FSimCallbackInputAndObject>& SimCallbackInputs) override
					{
						EXPECT_EQ(SimCallbackInputs.Num(), 1);
						FSimCallbackHelperInput* Input = static_cast<FSimCallbackHelperInput*>(SimCallbackInputs[0].Input);
						if(bResim)
						{
							EXPECT_EQ(InCounters[PhysicsStep], Input->InCounter);
						}
						else
						{
							InCounters.Add(Input->InCounter);
						}
					}

					virtual void PreResimStep_Internal(int32 PhysicsStep, bool bFirst) override
					{
						bResim = true;
					}

					virtual void PostResimStep_Internal(int32 PhysicsStep) override
					{
						bResim = false;
					}
				};

				const int32 LastGameStep = 20;
				const int32 NumPhysSteps = FMath::TruncToInt(LastGameStep / SimDt) - 1;

				Solver->SetRewindCallback(MakeUnique<FRewindCallback>(NumPhysSteps, SimDt));
				FSimCallbackHelper* SimCallback = Solver->template CreateAndRegisterSimCallbackObject_External<FSimCallbackHelper>();

				{
					auto& Particle = Proxy->GetGameThreadAPI();
					Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -1), 0);
					Particle.SetGravityEnabled(true);
					Particle.SetX(FVec3(0, 0, 100));

					for (int Step = 0; Step < LastGameStep; ++Step)
					{
						SimCallback->GetProducerInputData_External()->InCounter = Step;
						TickSolverHelper(Solver);
					}
				}

				//during async we can't consume all outputs right away, so we won't get the resim results right away
				//with sync we should be able to get all the results right away and this acts as a good test
				if(!Solver->IsUsingAsyncResults())
				{
					//we get all the original outputs plus the rewind outputs, they counter just keeps going up, but the InternalTime should reflect the rewind
					int32 Count = 0;
					FReal CurTime = 0.f;
					while (TSimCallbackOutputHandle<FSimCallbackHelperOutput> Output = SimCallback->PopOutputData_External())
					{
						EXPECT_FLOAT_EQ(Output->InternalTime, CurTime);
						EXPECT_EQ(Count, Output->OutCounter);
						++Count;

						if(Count == NumPhysSteps)
						{
							CurTime = 0;	//reset time for resim
						}
						else
						{
							CurTime += SimDt;
						}
					}
					const int32 NumSteps = NumPhysSteps * 2 + (LastGameStep - NumPhysSteps);
					EXPECT_EQ(Count, NumSteps);	
				}
				

			});
	}

	struct FSimCallbackHelperInput2 : FSimCallbackInput
	{
		void Reset() { StepToCounter.Reset(); }
		TMap<int32, int32> StepToCounter;
	};

	GTEST_TEST(AllTraits, RewindTest_SimCallbackProcessExternalInputs)
	{
		//If inputs are not set until external callback, make sure they are associated with the right frame
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				struct FSimCallbackHelper : TSimCallbackObject<FSimCallbackHelperInput2>
				{
					FPBDRigidsSolver* Solver = nullptr;
					virtual void OnPreSimulate_Internal() override
					{
						EXPECT_EQ(GetConsumerInput_Internal()->StepToCounter[Solver->GetCurrentFrame()], Solver->GetCurrentFrame());
					}
				};

				struct FRewindCallback : public IRewindCallback
				{
					FSimCallbackHelper* SimCallback;
					FRewindCallback(FSimCallbackHelper* Callback) : SimCallback(Callback){}
					virtual void InjectInputs_External(int32 PhysicsStep, int32 NumSteps) override
					{
						for(int32 Idx = 0; Idx < NumSteps; ++Idx)
						{
							const int32 Step = Idx + PhysicsStep;
							SimCallback->GetProducerInputData_External()->StepToCounter.Add(Step, Step);
						}
					}
				};

				FSimCallbackHelper* SimCallback = Solver->template CreateAndRegisterSimCallbackObject_External<FSimCallbackHelper>();
				SimCallback->Solver = Solver;
				Solver->SetRewindCallback(MakeUnique<FRewindCallback>(SimCallback));

				for (int Step = 0; Step < 32; ++Step)
				{
					TickSolverHelper(Solver);
				}
			});
	}

	GTEST_TEST(AllTraits, RewindTest_SimCallbackInputsCorrection)
	{
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				struct FSimCallbackHelper : TSimCallbackObject<FSimCallbackHelperInput, FSimCallbackHelperOutput>
				{
					int32 TriggerCount = 0;
					virtual void OnPreSimulate_Internal() override
					{
						GetProducerOutputData_Internal().OutCounter = GetConsumerInput_Internal()->InCounter;
					}
				};

				struct FRewindCallback : public IRewindCallback
				{
					FRewindCallback(int32 InNumPhysicsSteps)
						: NumPhysicsSteps(InNumPhysicsSteps)
					{

					}

					TArray<int32> InCounters;
					int32 NumPhysicsSteps;
					bool bResim = false;
					const int32 Correction = -10;

					virtual int32 TriggerRewindIfNeeded_Internal(int32 LastCompletedStep) override
					{
						if (LastCompletedStep + 1 == NumPhysicsSteps && NumPhysicsSteps != INDEX_NONE)
						{
							NumPhysicsSteps = INDEX_NONE;	//don't resim again after this
							return 0;
						}

						return INDEX_NONE;
					}

					virtual void ProcessInputs_Internal(int32 PhysicsStep, const TArray<FSimCallbackInputAndObject>& SimCallbackInputs) override
					{
						EXPECT_EQ(SimCallbackInputs.Num(), 1);
						FSimCallbackHelperInput* Input = static_cast<FSimCallbackHelperInput*>(SimCallbackInputs[0].Input);
						if (bResim)
						{
							Input->InCounter = InCounters[PhysicsStep] + Correction;
							EXPECT_EQ(InCounters[PhysicsStep] + Correction, Input->InCounter);
						}
						else
						{
							InCounters.Add(Input->InCounter);
						}
					}

					virtual void PreResimStep_Internal(int32 PhysicsStep, bool bFirst) override
					{
						bResim = true;
					}

					virtual void PostResimStep_Internal(int32 PhysicsStep) override
					{
						bResim = false;
					}
				};

				const int32 LastGameStep = 20;
				const int32 NumPhysSteps = FMath::TruncToInt(LastGameStep / SimDt) - 1;

				auto UniqueRewindCallback = MakeUnique<FRewindCallback>(NumPhysSteps);
				auto RewindCallback = UniqueRewindCallback.Get();

				Solver->SetRewindCallback(MoveTemp(UniqueRewindCallback));
				FSimCallbackHelper* SimCallback = Solver->template CreateAndRegisterSimCallbackObject_External<FSimCallbackHelper>();

				{
					auto& Particle = Proxy->GetGameThreadAPI();
					Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -1), 0);
					Particle.SetGravityEnabled(true);
					Particle.SetX(FVec3(0, 0, 100));

					for (int Step = 0; Step < LastGameStep; ++Step)
					{
						SimCallback->GetProducerInputData_External()->InCounter = Step;
						TickSolverHelper(Solver);
					}
				}

				//during async we can't consume all outputs right away, so we won't get the resim results right away
				//with sync we should be able to get all the results right away and this acts as a good test
				if (!Solver->IsUsingAsyncResults())
				{
					//we get all the original outputs plus the rewind outputs, the correction happens at NumPhysSteps and then the outputs should be the resim + correction
					int32 Count = 0;
					FReal CurTime = 0.f;
					int32 Correction = 0;
					while (TSimCallbackOutputHandle<FSimCallbackHelperOutput> Output = SimCallback->PopOutputData_External())
					{
						EXPECT_FLOAT_EQ(Output->InternalTime, CurTime);
						const int32 ExpectedResult = FMath::FloorToInt(Output->InternalTime * SimDt) + Correction;
						EXPECT_EQ(ExpectedResult, Output->OutCounter);
						++Count;
						
						if (Count == NumPhysSteps)
						{
							CurTime = 0;	//reset time for resim
							Correction = RewindCallback->Correction;
						}
						else
						{
							if (Count == 2 * NumPhysSteps)
							{
								Correction = 0.0;
							}
							CurTime += SimDt;
						}
					}

					const int32 NumSteps = NumPhysSteps * 2 + (LastGameStep - NumPhysSteps);
					EXPECT_EQ(Count, NumSteps);	
				}


			});
	}

	GTEST_TEST(AllTraits, DISABLED_RewindTest_SimCallbackCorrectionInterpolation)
	{
		// NOTE: Disabled because the logic that is being tested is no longer used.

		/*want to interpolate between original sim and correction
		edge cases to test:
		- gt write should stomp interpolation
		- make sure sleep state works
		- not moving in original, but moving in resim
		- moving in original, but not moving in resim
		- moving along one axis and corrected on another
		- resim while interpolation is still happening
		*/

		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
			{
				//only care about resim when async results are used
				if (Solver->IsUsingAsyncResults() == false)
				{
					return;
				}

				struct FRewindCallback : public IRewindCallback
				{
					FRewindCallback(int32 InNumPhysicsSteps)
						: NumPhysicsSteps(InNumPhysicsSteps)
					{

					}

					int32 NumPhysicsSteps;
					bool bResim = false;
					bool bPendingCorrection = true;
					const FReal ZCorrection = 100;
					FSingleParticlePhysicsProxy* Proxy;

					virtual int32 TriggerRewindIfNeeded_Internal(int32 LastCompletedStep) override
					{
						if (LastCompletedStep + 1 == NumPhysicsSteps && NumPhysicsSteps != INDEX_NONE)
						{
							NumPhysicsSteps = INDEX_NONE;	//don't resim again after this
							return 0;
						}

						return INDEX_NONE;
					}

					virtual void ProcessInputs_Internal(int32 PhysicsStep, const TArray<FSimCallbackInputAndObject>& SimCallbackInputs) override
					{
						if (bResim && bPendingCorrection)
						{
							Proxy->GetPhysicsThreadAPI()->SetX(FVec3(0, 0, ZCorrection));
							bPendingCorrection = false;
						}
					}

					virtual void PreResimStep_Internal(int32 PhysicsStep, bool bFirst) override
					{
						bResim = true;
					}

					virtual void PostResimStep_Internal(int32 PhysicsStep) override
					{
						bResim = false;
					}
				};

				const int32 LastGameStep = 20;
				const int32 NumPhysSteps = FMath::TruncToInt(LastGameStep / SimDt);

				auto UniqueRewindCallback = MakeUnique<FRewindCallback>(NumPhysSteps);
				auto RewindCallback = UniqueRewindCallback.Get();
				RewindCallback->Proxy = Proxy;
				const FReal GTDt = 1;
				FReal Time = 0;
				FReal ZStart = 0;
				const FReal ZVel = -1;
				FReal LastCorrectionStep = 0;

				Solver->SetRewindCallback(MoveTemp(UniqueRewindCallback));
				Solver->SetAsyncInterpolationMultiplier(4.0f);
				{
					auto& Particle = Proxy->GetGameThreadAPI();
					Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -1), 0);
					Particle.SetGravityEnabled(false);
					Particle.SetX(FVec3(0, 0, 0));
					Particle.SetV(FVec3(0, 0, -1));

					for (int Step = 0; Step < LastGameStep; ++Step)
					{
						TickSolverHelper(Solver);

						Time += GTDt;
						const FReal InterpolatedTime = Time - SimDt * Solver->GetAsyncInterpolationMultiplier();
						if (InterpolatedTime < 0)
						{
							//not enough time to interpolate so just take initial value
							EXPECT_NEAR(Particle.X()[2], ZStart, 1e-2);
						}
						else
						{
							//interpolated
							EXPECT_NEAR(Particle.X()[2], ZStart + ZVel * InterpolatedTime, 1e-2);
						}
					}

					//resim happened
					Solver->SetAsyncInterpolationMultiplier(2.0f);
					for (int Step = LastGameStep; Step < 2*LastGameStep; ++Step)
					{
						const FReal PrevZ = Particle.X()[2];
						TickSolverHelper(Solver);

						Time += GTDt;
						const FReal InterpolatedTime = Time - SimDt * Solver->GetAsyncInterpolationMultiplier();

						if(InterpolatedTime > 20)	//resim happened
						{
							ZStart = 100;	//corrected to 100 start point
						}

						//expected interpolation from pt to gt
						const int32 NextSimStep = FMath::CeilToInt(InterpolatedTime / SimDt);
						const FReal NextSimStepTime = NextSimStep * SimDt;
						const FReal ExpectedValue = ZStart + ZVel * InterpolatedTime;
					//	const FReal TargetValue = ZStart + ZVel * NextSimStepTime;

						FProxyInterpolationBase* InterpolationData = Proxy->GetInterpolationData();
						if (InterpolationData && InterpolationData->IsErrorSmoothing())
						{
							if (!InterpolationData->IsErrorVelocitySmoothing())
							{
								const FReal CorrectionStep = Particle.X()[2] - PrevZ;
								if (LastCorrectionStep != 0)
								{
									// Make sure we have a linear correction
									EXPECT_NEAR(LastCorrectionStep, CorrectionStep, 1e-2);
								}

								LastCorrectionStep = CorrectionStep;
							}
						}
						else
						{
							if (!!LastCorrectionStep)
							{
								// Correction is now done, check if the object arrived at the current position by the correction or if it was snapped into place via not doing correction anymore
								EXPECT_NEAR(PrevZ, Particle.X()[2] - LastCorrectionStep, 1e-2);
								LastCorrectionStep = 0;
							}

							//no resim interpolation, just simple value interpolation
							EXPECT_NEAR(Particle.X()[2], ExpectedValue, 1e-2);
						}
					}
				}
			});
	}

	GTEST_TEST(AllTraits, RewindTest_ResimFallingObjectWithTeleport)
	{
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
		{
			const int32 LastGameStep = 20;
			FReal ExpectedVZ = 0;
			FReal ExpectedXZ = 100;
			int32 FirstStepResim = INT_MAX;
			int32 NumResimSteps = 0;

			const int32 NumPhysSteps = FMath::TruncToInt(LastGameStep / SimDt) - 1;
			FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver, NumPhysSteps);
			Helper->ProcessInputsFunc = [Proxy, SimDt, &ExpectedVZ, &ExpectedXZ, &FirstStepResim, &NumResimSteps](const int32 Step, const bool bResim)
			{
				if(bResim)
				{
					FirstStepResim = FMath::Min(Step, FirstStepResim);
					NumResimSteps++;
					auto& Particle = *Proxy->GetPhysicsThreadAPI();
					const float SimStart = SimDt * Step;
					const float SimEnd = SimDt * (Step + 1);
					if (SimStart <= 5 && SimEnd > 5)
					{
						ExpectedVZ = 0;
						ExpectedXZ = 10;
						Particle.SetX(FVec3(0, 0, 10));
						Particle.SetV(FVec3(0, 0, 0));
					}

					EXPECT_NEAR(Particle.X()[2], ExpectedXZ, 1e-4);
					EXPECT_NEAR(Particle.V()[2], ExpectedVZ, 1e-4);
				}
				
			};

			Helper->PostFunc = [Proxy, SimDt, &ExpectedVZ, &ExpectedXZ](const int32 Step)
			{
				auto& Particle = *Proxy->GetPhysicsThreadAPI();
				ExpectedVZ -= SimDt;
				ExpectedXZ += ExpectedVZ * SimDt;

				EXPECT_NEAR(Particle.X()[2], ExpectedXZ, 1e-4);
				EXPECT_NEAR(Particle.V()[2], ExpectedVZ, 1e-4);
			};

			{
				auto& Particle = Proxy->GetGameThreadAPI();
				Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -1), 0);
				Particle.SetGravityEnabled(true);
				Particle.SetX(FVec3(0, 0, 100));

				for (int Step = 0; Step < LastGameStep; ++Step)
				{
					//teleport from GT
					if (Step == 5)
					{
						Particle.SetX(FVec3(0, 0, 10));
						Particle.SetV(FVec3(0, 0, 0));
					}

					TickSolverHelper(Solver);
				}
			}

			//no desync so should be empty
#if REWIND_DESYNC
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 0);
#endif
		});
	}

	GTEST_TEST(AllTraits, RewindTest_ResimFallingObjectWithTeleportAsFollower)
	{
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
		{
			const int32 LastGameStep = 20;

			{
				auto& Particle = Proxy->GetGameThreadAPI();
				Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -1), 0);
				Particle.SetGravityEnabled(true);
				Particle.SetX(FVec3(0, 0, 100));
				Particle.SetResimType(EResimType::ResimAsFollower);

				for (int Step = 0; Step <= LastGameStep; ++Step)
				{
					//teleport from GT
					if (Step == 5)
					{
						Particle.SetX(FVec3(0, 0, 10));
						Particle.SetV(FVec3(0, 0, 0));
					}

					TickSolverHelper(Solver);
				}
			}
			
			FPhysicsThreadContextScope Scope(true);
			FRewindData* RewindData = Solver->GetRewindData();
			//RewindData->RewindToFrame(0);
			Solver->DisableAsyncMode();	//during resim we sim directly at fixed dt

			auto& Particle = *Proxy->GetPhysicsThreadAPI();

			const int32 LastSimStep = LastGameStep / SimDt;
			FReal ExpectedVZ = 0;
			FReal ExpectedXZ = 100;

			for (int Step = 0; Step < LastSimStep - 1; ++Step)
			{
				const float SimStart = SimDt * Step;
				const float SimEnd = SimDt * (Step + 1);
				if (SimStart <= 5 && SimEnd > 5)
				{
					ExpectedVZ = 0;
					ExpectedXZ = 10;
				}
				else
				{
#if REWIND_DESYNC
					//we'll see the teleport automatically because ResimAsFollower
					//but it's done by solver so before tick teleport is not known
					EXPECT_NEAR(Particle.X()[2], ExpectedXZ, 1e-4);
					EXPECT_NEAR(Particle.GetV()[2], ExpectedVZ, 1e-4);
#endif
				}

				TickSolverHelper(Solver, SimDt);

				ExpectedVZ -= SimDt;
				ExpectedXZ += ExpectedVZ * SimDt;

#if REWIND_DESYNC
				EXPECT_NEAR(Particle.X()[2], ExpectedXZ, 1e-4);
				EXPECT_NEAR(Particle.GetV()[2], ExpectedVZ, 1e-4);
#endif
			}

#if REWIND_DESYNC
			//no desync so should be empty
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 0);
#endif
		});
	}

	GTEST_TEST(AllTraits, RewindTest_NumDirty)
	{
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);

			//note: this 5 is just a suggestion, there could be more frames saved than that
			EnableRewindCaptureHelper(Solver, 5, Optimization);

			// Make particles
			auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto& Particle = Proxy->GetGameThreadAPI();

			Particle.SetGeometry(Sphere);
			Solver->RegisterObject(Proxy);
			Particle.SetGravityEnabled(true);

			for (int Step = 0; Step < 10; ++Step)
			{
				TickSolverHelper(Solver);

				const FRewindData* RewindData = Solver->GetRewindData();
				EXPECT_EQ(RewindData->GetNumDirtyParticles(), 1);
			}

			//stop movement
			Particle.SetGravityEnabled(false);
			Particle.SetV(FVec3(0));

			// Wait for sleep (active particles get added to the dirty list)
			// NOTE: Sleep requires 20 frames of inactivity by default, plus the time for smoothed velocity to damp to zero
			// (see FPBDIslandManager::SleepInactive)
			for(int Step = 0; Step < 500; ++Step)
			{
				TickSolverHelper(Solver);
			}

			{
				//enough frames with no changes so no longer dirty
				const FRewindData* RewindData = Solver->GetRewindData();
				EXPECT_EQ(RewindData->GetNumDirtyParticles(), 0);
			}

			{
				//single change so back to being dirty
				Particle.SetGravityEnabled(true);
				TickSolverHelper(Solver);

				const FRewindData* RewindData = Solver->GetRewindData();
				EXPECT_EQ(RewindData->GetNumDirtyParticles(), 1);
			}

			// Throw out the proxy
			Solver->UnregisterObject(Proxy);

			Module->DestroySolver(Solver);
		}
	}

	GTEST_TEST(AllTraits, RewindTest_Resim)
	{
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);


			InitSolverSettings(Solver);

			EnableRewindCaptureHelper(Solver, 5, Optimization);

			// Make particles
			auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto KinematicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FKinematicGeometryParticle::CreateParticle());

			const int32 LastStep = 12;
			TArray<FVec3> X;

			const int RewindStep = 7;
			FRewindCallbackTestHelper* Helper = RegisterCallbackHelper(Solver, LastStep, RewindStep);
			Helper->ProcessInputsFunc = [Proxy, KinematicProxy](const int32 Step, const bool bResim)
			{
				if (bResim)
				{
					if(Step == 7)
					{
						Proxy->GetPhysicsThreadAPI()->SetX(FVec3(0, 0, 100));
						KinematicProxy->GetPhysicsThreadAPI()->SetX(FVec3(2));
					}
					else if (Step == 8)
					{
						KinematicProxy->GetPhysicsThreadAPI()->SetX(FVec3(50));
					}
					
				}

			};

			{
				auto& Particle = Proxy->GetGameThreadAPI();

				Particle.SetGeometry(Sphere);
				Solver->RegisterObject(Proxy);
				Particle.SetGravityEnabled(true);

				auto& Kinematic = KinematicProxy->GetGameThreadAPI();

				Kinematic.SetGeometry(Sphere);
				Solver->RegisterObject(KinematicProxy);
				Kinematic.SetX(FVec3(2, 2, 2));


				for (int Step = 0; Step <= LastStep; ++Step)
				{
					X.Add(Particle.X());

					if (Step == 8)
					{
						Kinematic.SetX(FVec3(50, 50, 50));
					}

					if (Step == 10)
					{
						Kinematic.SetX(FVec3(60, 60, 60));
					}

					TickSolverHelper(Solver);
				}
			}

#if REWIND_DESYNC

			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
				

				X[Step] = Particle.X();
				TickSolverHelper(Solver);

				auto PTParticle = Proxy->GetHandle_LowLevel()->CastToRigidParticle();	//using handle directly because outside sim callback scope and we have ensures for that
				auto PTKinematic = KinematicProxy->GetHandle_LowLevel()->CastToKinematicParticle();

				//see that particle has desynced
				if (Step < LastStep)
				{

					//If we're still in the past make sure future has been marked as desync
					FGeometryParticleState State(*Proxy->GetHandle_LowLevel());
					EXPECT_EQ(EFutureQueryResult::Desync, RewindData->GetFutureStateAtFrame(State, Step));

					EXPECT_EQ(PTParticle->SyncState(), ESyncState::HardDesync);

					FGeometryParticleState KinState(*KinematicProxy->GetHandle_LowLevel());
					const EFutureQueryResult KinFutureStatus = RewindData->GetFutureStateAtFrame(KinState, Step);
					if (Step < 10)
					{
						EXPECT_EQ(KinFutureStatus, EFutureQueryResult::Ok);
						EXPECT_EQ(PTKinematic->SyncState(), ESyncState::InSync);
					}
					else
					{
						EXPECT_EQ(KinFutureStatus, EFutureQueryResult::Desync);
						EXPECT_EQ(PTKinematic->SyncState(), ESyncState::HardDesync);
					}
				}
				else
				{
					//Last resim frame ran so everything is marked as in sync
					EXPECT_EQ(PTParticle->SyncState(), ESyncState::InSync);
					EXPECT_EQ(PTKinematic->SyncState(), ESyncState::InSync);
				}
			}

			//expect both particles to be hard desynced
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 2);
			EXPECT_EQ(DesyncedParticles[0].MostDesynced, ESyncState::HardDesync);
			EXPECT_EQ(DesyncedParticles[1].MostDesynced, ESyncState::HardDesync);
#endif

#if 0	//can't read from resim data in public API
			EXPECT_EQ(Kinematic.X()[2], 50);	//Rewound kinematic and only did one update, so use that first update

			//Make sure we recorded the new data
			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
				const FGeometryParticleState State = RewindData->GetPastStateAtFrame(*Proxy->GetHandle_LowLevel(), Step);
				EXPECT_EQ(State.X()[2], X[Step][2]);

				const FGeometryParticleState KinState = RewindData->GetPastStateAtFrame(*KinematicProxy->GetHandle_LowLevel(), Step);
				if (Step < 8)
				{
					EXPECT_EQ(KinState.X()[2], 2);
				}
				else
				{
					EXPECT_EQ(KinState.X()[2], 50);	//in resim we didn't do second move, so recorded data must be updated
				}
			}

#endif

			// Throw out the proxy
			Solver->UnregisterObject(Proxy);

			Module->DestroySolver(Solver);
		}
	}

	GTEST_TEST(AllTraits, RewindTest_ResimDesyncAfterMissingTeleport)
	{
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);

			EnableRewindCaptureHelper(Solver, 7, Optimization);

			// Make particles
			auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			const int LastStep = 11;
			TArray<FVec3> X;

			{
				auto& Particle = Proxy->GetGameThreadAPI();

				Particle.SetGeometry(Sphere);
				Solver->RegisterObject(Proxy);
				Particle.SetGravityEnabled(true);

				for (int Step = 0; Step <= LastStep; ++Step)
				{
					if (Step == 7)
					{
						Particle.SetX(FVec3(0, 0, 5));
					}

					if (Step == 9)
					{
						Particle.SetX(FVec3(0, 0, 1));
					}
					X.Add(Particle.X());
					TickSolverHelper(Solver);
				}
				X.Add(Particle.X());

			}
			
			const int RewindStep = 5;
#if PHYSICS_THREAD_CONTEXT
			FPhysicsThreadContextScope Scope(true);
#endif
			auto& Particle = *Proxy->GetPhysicsThreadAPI();
			FRewindData* RewindData = Solver->GetRewindData();
			//EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
#if REWIND_DESYNC

				FGeometryParticleState FutureState(*Proxy->GetHandle_LowLevel());
				EXPECT_EQ(RewindData->GetFutureStateAtFrame(FutureState, Step + 1), Step < 10 ? EFutureQueryResult::Ok : EFutureQueryResult::Desync);
				if (Step < 10)
				{
					EXPECT_EQ(X[Step + 1][2], FutureState.X()[2]);
				}
#endif

				if (Step == 7)
				{
					Particle.SetX(FVec3(0, 0, 5));
				}

				//skip step 9 SetX to trigger a desync

				TickSolverHelper(Solver);

#if REWIND_DESYNC
				//can't compare future with end of frame because we overwrite the result
				if (Step != 6 && Step != 8 && Step < 9)
				{
					EXPECT_EQ(Particle.X()[2], FutureState.X()[2]);
				}
#endif
			}

#if REWIND_DESYNC
			//expected desync
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 1);
			EXPECT_EQ(DesyncedParticles[0].MostDesynced, ESyncState::HardDesync);
			EXPECT_EQ(DesyncedParticles[0].Particle, Proxy->GetHandle_LowLevel());
#endif

			// Throw out the proxy
			Solver->UnregisterObject(Proxy);

			Module->DestroySolver(Solver);
		}
	}

	GTEST_TEST(AllTraits, RewindTest_ResimDesyncAfterChangingMass)
	{
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);

			EnableRewindCaptureHelper(Solver, 7, Optimization);

			FReal CurMass = 1.0;
			int32 LastStep = 11;

			// Make particles
			auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			{
				auto& Particle = Proxy->GetGameThreadAPI();

				Particle.SetGeometry(Sphere);
				Solver->RegisterObject(Proxy);
				Particle.SetGravityEnabled(true);

				Particle.SetM(CurMass);

				for (int Step = 0; Step <= LastStep; ++Step)
				{
					if (Step == 7)
					{
						Particle.SetM(2);
					}

					if (Step == 9)
					{
						Particle.SetM(3);
					}
					TickSolverHelper(Solver);
				}

			}
			
			const int RewindStep = 5;

#if PHYSICS_THREAD_CONTEXT
			FPhysicsThreadContextScope Scope(true);
#endif
			FRewindData* RewindData = Solver->GetRewindData();
			//EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));
			auto& Particle = *Proxy->GetPhysicsThreadAPI();

			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
#if REWIND_DESYNC
				FGeometryParticleState FutureState(*Proxy->GetHandle_LowLevel());
				EXPECT_EQ(RewindData->GetFutureStateAtFrame(FutureState, Step), Step < 10 ? EFutureQueryResult::Ok : EFutureQueryResult::Desync);
				if (Step < 7)
				{
					EXPECT_EQ(1, FutureState.M());
				}
#endif

				if (Step == 7)
				{
					Particle.SetM(2);
				}

				//skip step 9 SetM to trigger a desync

				TickSolverHelper(Solver);
			}

#if REWIND_DESYNC
			//expected desync
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 1);
			EXPECT_EQ(DesyncedParticles[0].MostDesynced, ESyncState::HardDesync);
			EXPECT_EQ(DesyncedParticles[0].Particle, Proxy->GetHandle_LowLevel());
#endif

			// Throw out the proxy
			Solver->UnregisterObject(Proxy);

			Module->DestroySolver(Solver);
		}
	}

	GTEST_TEST(AllTraits, RewindTest_DesyncFromPT)
	{
#if REWIND_DESYNC

		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			//We want to detect when sim results change
			//Detecting output of position and velocity is expensive and hard to track
			//Instead we need to rely on fast forward mechanism, this is still in progress
			auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
			auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100, -100, -100), FVec3(100, 100, 0)));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);

			EnableRewindCaptureHelper(Solver, 7, Optimization);

			// Make particles


			auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto KinematicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FKinematicGeometryParticle::CreateParticle());
			const int32 LastStep = 11;

			{
				auto& Dynamic = DynamicProxy->GetGameThreadAPI();
				auto& Kinematic = KinematicProxy->GetGameThreadAPI();

				Dynamic.SetGeometry(Sphere);
				Dynamic.SetGravityEnabled(true);
				Solver->RegisterObject(DynamicProxy);

				Kinematic.SetGeometry(Box);
				Solver->RegisterObject(KinematicProxy);

				Dynamic.SetX(FVec3(0, 0, 17));
				Dynamic.SetGravityEnabled(false);
				Dynamic.SetV(FVec3(0, 0, -1));
				Dynamic.SetObjectState(EObjectStateType::Dynamic);

				Kinematic.SetX(FVec3(0, 0, 0));

				ChaosTest::SetParticleSimDataToCollide({ DynamicProxy->GetParticle_LowLevel(),KinematicProxy->GetParticle_LowLevel() });


				for (int Step = 0; Step <= LastStep; ++Step)
				{
					TickSolverHelper(Solver);
				}

				// We may end up a bit away from the surface (dt * V), due to solving for 0 velocity and not 0 position error
				EXPECT_GE(Dynamic.X()[2], 10);
				EXPECT_LE(Dynamic.X()[2], 11);
			}

			const int RewindStep = 5;
			auto& Dynamic = *DynamicProxy->GetPhysicsThreadAPI();
			auto& Kinematic = *KinematicProxy->GetPhysicsThreadAPI();
			FPhysicsThreadContextScope Scope(true);

			FRewindData* RewindData = Solver->GetRewindData();
			EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

			Kinematic.SetX(FVec3(0, 0, -1));

			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
				//at the end of frame 6 a desync occurs because velocity is no longer clamped (kinematic moved)
				//because of this desync will happen for any step after 6
				if (Step <= 6)
				{
					FGeometryParticleState FutureState(*DynamicProxy->GetHandle_LowLevel());
					EXPECT_EQ(RewindData->GetFutureStateAtFrame(FutureState, Step), EFutureQueryResult::Ok);
				}
				else if (Step >= 8)
				{
					//collision would have happened at frame 7, so anything after will desync. We skip a few frames because solver is fuzzy at that point
					//that is we can choose to solve velocity in a few ways. Main thing we want to know is that a desync eventually happened
					FGeometryParticleState FutureState(*DynamicProxy->GetHandle_LowLevel());
					EXPECT_EQ(RewindData->GetFutureStateAtFrame(FutureState, Step), EFutureQueryResult::Desync);
				}


				TickSolverHelper(Solver);
			}

			//both kinematic and simulated are desynced
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 2);
			EXPECT_EQ(DesyncedParticles[0].MostDesynced, ESyncState::HardDesync);
			EXPECT_EQ(DesyncedParticles[1].MostDesynced, ESyncState::HardDesync);

			// We may end up a bit away from the surface (dt * V), due to solving for 0 velocity and not 0 position error
			EXPECT_GE(Dynamic.X()[2], 9);
			EXPECT_LE(Dynamic.X()[2], 10);

			Module->DestroySolver(Solver);
		}
#endif
	}

	GTEST_TEST(AllTraits, DISABLED_RewindTest_ResimDesyncFromChangeForce)
	{
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);

			EnableRewindCaptureHelper(Solver, 7, Optimization);

			// Make particles
			auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			int32 LastStep = 11;

			{

				auto& Particle = Proxy->GetGameThreadAPI();

				Particle.SetGeometry(Sphere);
				Solver->RegisterObject(Proxy);
				Particle.SetGravityEnabled(false);
				Particle.SetV(FVec3(0, 0, 10));


				for (int Step = 0; Step <= LastStep; ++Step)
				{
					if (Step == 7)
					{
						Particle.AddForce(FVec3(0, 1, 0));
					}

					if (Step == 9)
					{
						Particle.AddForce(FVec3(100, 0, 0));
					}
					TickSolverHelper(Solver);
				}
			}
			const int RewindStep = 5;
			auto& Particle = *Proxy->GetPhysicsThreadAPI();

#if PHYSICS_THREAD_CONTEXT
			FPhysicsThreadContextScope Scope(true);
#endif
			{
				FRewindData* RewindData = Solver->GetRewindData();
				//EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

				for (int Step = RewindStep; Step <= LastStep; ++Step)
				{
#if REWIND_DESYNC
					FGeometryParticleState FutureState(*Proxy->GetHandle_LowLevel());
					EXPECT_EQ(RewindData->GetFutureStateAtFrame(FutureState, Step), Step < 10 ? EFutureQueryResult::Ok : EFutureQueryResult::Desync);
#endif

					if (Step == 7)
					{
						Particle.AddForce(FVec3(0, 1, 0));
					}

					//skip step 9 SetF to trigger a desync

					TickSolverHelper(Solver);
				}
				EXPECT_EQ(Particle.V()[0], 0);

#if REWIND_DESYNC
				//desync
				const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
				EXPECT_EQ(DesyncedParticles.Num(), 1);
				EXPECT_EQ(DesyncedParticles[0].MostDesynced, ESyncState::HardDesync);
#endif
			}

			//rewind to exactly step 7 to make sure force is not already applied for us
			{
				FRewindData* RewindData = Solver->GetRewindData();
				//EXPECT_TRUE(RewindData->RewindToFrame(7));
				EXPECT_EQ(Particle.Acceleration()[1], 0);
			}

			// Throw out the proxy
			Solver->UnregisterObject(Proxy);

			Module->DestroySolver(Solver);
		}
	}

	GTEST_TEST(AllTraits, RewindTest_ResimAsFollower)
	{
#if REWIND_DESYNC
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
			auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100, -100, -100), FVec3(100, 100, 0)));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);

			EnableRewindCaptureHelper(Solver, 7, Optimization);

			// Make particles
			auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto KinematicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FKinematicGeometryParticle::CreateParticle());
			const int32 LastStep = 11;
			TArray<FVec3> Xs;

			{

				auto& Dynamic = DynamicProxy->GetGameThreadAPI();
				auto& Kinematic = KinematicProxy->GetGameThreadAPI();

				Dynamic.SetGeometry(Sphere);
				Dynamic.SetGravityEnabled(true);
				Solver->RegisterObject(DynamicProxy);

				Kinematic.SetGeometry(Box);
				Solver->RegisterObject(KinematicProxy);

				Dynamic.SetX(FVec3(0, 0, 17));
				Dynamic.SetGravityEnabled(false);
				Dynamic.SetV(FVec3(0, 0, -1));
				Dynamic.SetObjectState(EObjectStateType::Dynamic);
				Dynamic.SetResimType(EResimType::ResimAsFollower);

				Kinematic.SetX(FVec3(0, 0, 0));

				ChaosTest::SetParticleSimDataToCollide({ DynamicProxy->GetParticle_LowLevel(),KinematicProxy->GetParticle_LowLevel() });

				for (int Step = 0; Step <= LastStep; ++Step)
				{
					TickSolverHelper(Solver);
					Xs.Add(Dynamic.X());
				}


				EXPECT_GE(Dynamic.X()[2], 10);
				EXPECT_LE(Dynamic.X()[2], 11);
			}

			const int RewindStep = 5;

			FPhysicsThreadContextScope Scope(true);
			auto& Dynamic = *DynamicProxy->GetPhysicsThreadAPI();
			auto& Kinematic = *KinematicProxy->GetPhysicsThreadAPI();

			FRewindData* RewindData = Solver->GetRewindData();
			EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

			//make avoid collision
			Kinematic.SetX(FVec3(0, 0, 100000));

			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
				//Resim but dynamic will take old path since it's marked as ResimAsFollower
				TickSolverHelper(Solver);

				EXPECT_VECTOR_FLOAT_EQ(Dynamic.X(), Xs[Step]);
			}

#if REWIND_DESYNC
			// follower - so dynamic in sync, kinematic desync
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 1);
			EXPECT_EQ(DesyncedParticles[0].MostDesynced, ESyncState::HardDesync);
			EXPECT_EQ(DesyncedParticles[0].Particle, KinematicProxy->GetHandle_LowLevel());
#endif

			// We may end up a bit away from the surface (dt * V), due to solving for 0 velocity and not 0 position error
			EXPECT_GE(Dynamic.X()[2], 10);
			EXPECT_LE(Dynamic.X()[2], 11);

			Module->DestroySolver(Solver);
		}
#endif
	}

	GTEST_TEST(AllTraits, DISABLED_RewindTest_FullResimFallSeeCollisionCorrection)
	{
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
			auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100, -100, -100), FVec3(100, 100, 0)));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);

			EnableRewindCaptureHelper(Solver, 100, Optimization);

			// Make particles
			auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto KinematicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FKinematicGeometryParticle::CreateParticle());
			const int32 LastStep = 11;

			TArray<FVec3> Xs;
			{

				auto& Dynamic = DynamicProxy->GetGameThreadAPI();
				auto& Kinematic = KinematicProxy->GetGameThreadAPI();

				Dynamic.SetGeometry(Sphere);
				Dynamic.SetGravityEnabled(true);
				Solver->RegisterObject(DynamicProxy);

				Kinematic.SetGeometry(Box);
				Solver->RegisterObject(KinematicProxy);

				Dynamic.SetX(FVec3(0, 0, 17));
				Dynamic.SetGravityEnabled(false);
				Dynamic.SetV(FVec3(0, 0, -1));
				Dynamic.SetObjectState(EObjectStateType::Dynamic);

				Kinematic.SetX(FVec3(0, 0, -1000));

				ChaosTest::SetParticleSimDataToCollide({ DynamicProxy->GetParticle_LowLevel(),KinematicProxy->GetParticle_LowLevel() });

				for (int Step = 0; Step <= LastStep; ++Step)
				{
					TickSolverHelper(Solver);
					Xs.Add(Dynamic.X());
				}

				// We may end up a bit away from the surface (dt * V), due to solving for 0 velocity and not 0 position error
				EXPECT_GE(Dynamic.X()[2], 5);
				EXPECT_LE(Dynamic.X()[2], 6);
			}

			const int RewindStep = 0;

#if PHYSICS_THREAD_CONTEXT
			FPhysicsThreadContextScope Scope(true);
#endif
			auto& Dynamic = *DynamicProxy->GetPhysicsThreadAPI();
			auto& Kinematic = *KinematicProxy->GetPhysicsThreadAPI();

			FRewindData* RewindData = Solver->GetRewindData();
			//EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

			//force collision
			Kinematic.SetX(FVec3(0, 0, 0));

			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
				//Resim sees collision since it's ResimAsFull
				TickSolverHelper(Solver);
#if REWIND_DESYNC
				EXPECT_GE(Dynamic.X()[2], 10);
#endif
			}

#if REWIND_DESYNC
			// We may end up a bit away from the surface (dt * V), due to solving for 0 velocity and not 0 position error
			EXPECT_GE(Dynamic.X()[2], 10);
			EXPECT_LE(Dynamic.X()[2], 11);
#endif

#if REWIND_DESYNC
			//both desync
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 2);
			EXPECT_EQ(DesyncedParticles[0].MostDesynced, ESyncState::HardDesync);
			EXPECT_EQ(DesyncedParticles[1].MostDesynced, ESyncState::HardDesync);
#endif

			Module->DestroySolver(Solver);
		}
	}

	GTEST_TEST(AllTraits, DISABLED_RewindTest_ResimAsFollowerFallIgnoreCollision)
	{
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
			auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100, -100, -100), FVec3(100, 100, 0)));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);

			EnableRewindCaptureHelper(Solver, 100, Optimization);

			// Make particles
			auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto KinematicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FKinematicGeometryParticle::CreateParticle());
			const int32 LastStep = 11;
			TArray<FVec3> Xs;

			{
				auto& Dynamic = DynamicProxy->GetGameThreadAPI();
				auto& Kinematic = KinematicProxy->GetGameThreadAPI();

				Dynamic.SetGeometry(Sphere);
				Dynamic.SetGravityEnabled(true);
				Solver->RegisterObject(DynamicProxy);

				Kinematic.SetGeometry(Box);
				Solver->RegisterObject(KinematicProxy);

				Dynamic.SetX(FVec3(0, 0, 17));
				Dynamic.SetGravityEnabled(false);
				Dynamic.SetV(FVec3(0, 0, -1));
				Dynamic.SetObjectState(EObjectStateType::Dynamic);
				Dynamic.SetResimType(EResimType::ResimAsFollower);

				Kinematic.SetX(FVec3(0, 0, -1000));

				ChaosTest::SetParticleSimDataToCollide({ DynamicProxy->GetParticle_LowLevel(),KinematicProxy->GetParticle_LowLevel() });

				for (int Step = 0; Step <= LastStep; ++Step)
				{
					TickSolverHelper(Solver);
					Xs.Add(Dynamic.X());
				}

				// We may end up a bit away from the surface (dt * V), due to solving for 0 velocity and not 0 position error
				EXPECT_GE(Dynamic.X()[2], 5);
				EXPECT_LE(Dynamic.X()[2], 6);
			}

#if PHYSICS_THREAD_CONTEXT
			FPhysicsThreadContextScope Scope(true);
#endif
			auto& Dynamic = *DynamicProxy->GetPhysicsThreadAPI();
			auto& Kinematic = *KinematicProxy->GetPhysicsThreadAPI();
			const int RewindStep = 0;

			FRewindData* RewindData = Solver->GetRewindData();
			//EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

			//force collision
			Kinematic.SetX(FVec3(0, 0, 0));

			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
				//Resim ignores collision since it's ResimAsFollower
				TickSolverHelper(Solver);

				EXPECT_VECTOR_FLOAT_EQ(Dynamic.X(), Xs[Step]);
			}

			// We may end up a bit away from the surface (dt * V), due to solving for 0 velocity and not 0 position error
			EXPECT_GE(Dynamic.X()[2], 5);
			EXPECT_LE(Dynamic.X()[2], 6);

#if REWIND_DESYNC
			//dynamic follower so only kinematic desyncs
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 1);
			EXPECT_EQ(DesyncedParticles[0].MostDesynced, ESyncState::HardDesync);
			EXPECT_EQ(DesyncedParticles[0].Particle, KinematicProxy->GetHandle_LowLevel());
#endif

			Module->DestroySolver(Solver);
		}
	}

	GTEST_TEST(AllTraits, RewindTest_ResimAsFollowerWithForces)
	{
#if REWIND_DESYNC
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-10, -10, -10), FVec3(10, 10, 10)));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);

			EnableRewindCaptureHelper(Solver, 7, Optimization);

			// Make particles
			auto FullSimProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto FollowerSimProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			const int32 LastStep = 11;
			TArray<FVec3> Xs;

			{
				auto& FullSim = FullSimProxy->GetGameThreadAPI();
				auto& FollowerSim = FollowerSimProxy->GetGameThreadAPI();

				FullSim.SetGeometry(Box);
				FullSim.SetGravityEnabled(false);
				Solver->RegisterObject(FullSimProxy);

				FollowerSim.SetGeometry(Box);
				FollowerSim.SetGravityEnabled(false);
				Solver->RegisterObject(FollowerSimProxy);

				FullSim.SetX(FVec3(0, 0, 20));
				FullSim.SetObjectState(EObjectStateType::Dynamic);
				FullSim.SetM(1);
				FullSim.SetInvM(1);

				FollowerSim.SetX(FVec3(0, 0, 0));
				FollowerSim.SetResimType(EResimType::ResimAsFollower);
				FollowerSim.SetM(1);
				FollowerSim.SetInvM(1);

				ChaosTest::SetParticleSimDataToCollide({ FullSimProxy->GetParticle_LowLevel(),FollowerSimProxy->GetParticle_LowLevel() });

				for (int Step = 0; Step <= LastStep; ++Step)
				{
					FollowerSim.SetLinearImpulse(FVec3(0, 0, 0.5));
					TickSolverHelper(Solver);
					Xs.Add(FullSim.X());
				}
			}

			FPhysicsThreadContextScope Scope(true);
			const int RewindStep = 5;
			auto& FullSim = *FullSimProxy->GetPhysicsThreadAPI();
			auto& FollowerSim = *FollowerSimProxy->GetPhysicsThreadAPI();

			FRewindData* RewindData = Solver->GetRewindData();
			EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
				//resim - follower sim should have its impulses automatically added thus moving FullSim in the exact same way
				TickSolverHelper(Solver);

				EXPECT_VECTOR_FLOAT_EQ(FullSim.X(), Xs[Step]);
			}

#if REWIND_DESYNC
			//follower so no desync
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 0);
#endif

			Module->DestroySolver(Solver);
		}
#endif
	}

	GTEST_TEST(AllTraits, RewindTest_ResimAsFollowerWokenUp)
	{
#if REWIND_DESYNC
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-10, -10, -10), FVec3(10, 10, 10)));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);

			EnableRewindCaptureHelper(Solver, 7, Optimization);

			// Make particles
			auto ImpulsedObjProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto HitObjProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());

			const int32 ApplyImpulseStep = 8;
			const int32 LastStep = 11;

			TArray<FVec3> Xs;
			{
				auto& ImpulsedObj = ImpulsedObjProxy->GetGameThreadAPI();
				auto& HitObj = HitObjProxy->GetGameThreadAPI();

				ImpulsedObj.SetGeometry(Box);
				ImpulsedObj.SetGravityEnabled(false);
				Solver->RegisterObject(ImpulsedObjProxy);

				HitObj.SetGeometry(Box);
				HitObj.SetGravityEnabled(false);
				Solver->RegisterObject(HitObjProxy);

				ImpulsedObj.SetX(FVec3(0, 0, 20));
				ImpulsedObj.SetM(1);
				ImpulsedObj.SetInvM(1);
				ImpulsedObj.SetResimType(EResimType::ResimAsFollower);
				ImpulsedObj.SetObjectState(EObjectStateType::Sleeping);

				HitObj.SetX(FVec3(0, 0, 0));
				HitObj.SetM(1);
				HitObj.SetInvM(1);
				HitObj.SetResimType(EResimType::ResimAsFollower);
				HitObj.SetObjectState(EObjectStateType::Sleeping);


				ChaosTest::SetParticleSimDataToCollide({ ImpulsedObjProxy->GetParticle_LowLevel(),HitObjProxy->GetParticle_LowLevel() });


				for (int Step = 0; Step <= LastStep; ++Step)
				{
					if (ApplyImpulseStep == Step)
					{
						ImpulsedObj.SetLinearImpulse(FVec3(0, 0, -10));
					}

					TickSolverHelper(Solver);
					Xs.Add(HitObj.X());
				}
			}

			auto& ImpulsedObj = *ImpulsedObjProxy->GetPhysicsThreadAPI();
			auto& HitObj = *HitObjProxy->GetPhysicsThreadAPI();

			FPhysicsThreadContextScope Scope(true);
			const int RewindStep = 5;

			FRewindData* RewindData = Solver->GetRewindData();
			EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
				TickSolverHelper(Solver);

				EXPECT_VECTOR_FLOAT_EQ(HitObj.X(), Xs[Step]);
			}

#if REWIND_DESYNC
			//follower so no desync
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 0);
#endif

			Module->DestroySolver(Solver);
		}
#endif
	}

	GTEST_TEST(AllTraits, RewindTest_ResimAsFollowerWokenUpNoHistory)
	{
#if REWIND_DESYNC
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-10, -10, -10), FVec3(10, 10, 10)));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);

			EnableRewindCaptureHelper(Solver, 7, Optimization);

			// Make particles
			auto ImpulsedObjProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto HitObjProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());

			const int32 ApplyImpulseStep = 97;
			const int32 LastStep = 100;

			TArray<FVec3> Xs;
			{
				auto& ImpulsedObj = ImpulsedObjProxy->GetGameThreadAPI();
				auto& HitObj = HitObjProxy->GetGameThreadAPI();

				ImpulsedObj.SetGeometry(Box);
				ImpulsedObj.SetGravityEnabled(false);
				Solver->RegisterObject(ImpulsedObjProxy);

				HitObj.SetGeometry(Box);
				HitObj.SetGravityEnabled(false);
				Solver->RegisterObject(HitObjProxy);

				ImpulsedObj.SetX(FVec3(0, 0, 20));
				ImpulsedObj.SetM(1);
				ImpulsedObj.SetInvM(1);
				ImpulsedObj.SetObjectState(EObjectStateType::Sleeping);

				HitObj.SetX(FVec3(0, 0, 0));
				HitObj.SetM(1);
				HitObj.SetInvM(1);
				HitObj.SetResimType(EResimType::ResimAsFollower);
				HitObj.SetObjectState(EObjectStateType::Sleeping);


				ChaosTest::SetParticleSimDataToCollide({ ImpulsedObjProxy->GetParticle_LowLevel(),HitObjProxy->GetParticle_LowLevel() });

				for (int Step = 0; Step <= LastStep; ++Step)
				{
					TickSolverHelper(Solver);
					Xs.Add(HitObj.X());	//not a full re-sim so we should end up with exact same result
				}
			}

			const int RewindStep = 95;
			FPhysicsThreadContextScope Scope(true);
			auto& ImpulsedObj = *ImpulsedObjProxy->GetPhysicsThreadAPI();
			auto& HitObj = *HitObjProxy->GetPhysicsThreadAPI();

			FRewindData* RewindData = Solver->GetRewindData();
			EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
				//during resim apply correction impulse
				if (ApplyImpulseStep == Step)
				{
					ImpulsedObj.SetLinearImpulse(FVec3(0, 0, -10));
				}

				TickSolverHelper(Solver);

				//even though there's now a different collision in the sim, the final result of follower is the same as before
				EXPECT_VECTOR_FLOAT_EQ(HitObj.X(), Xs[Step]);
			}

#if REWIND_DESYNC
			//only desync non-follower
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 1);
			EXPECT_EQ(DesyncedParticles[0].MostDesynced, ESyncState::HardDesync);
			EXPECT_EQ(DesyncedParticles[0].Particle, ImpulsedObjProxy->GetHandle_LowLevel());
#endif

			Module->DestroySolver(Solver);
		}
#endif
	}

	GTEST_TEST(AllTraits, DISABLED_RewindTest_DesyncSimOutOfCollision)
	{
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
			auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100, -100, -100), FVec3(100, 100, 0)));

			FChaosSolversModule* Module = FChaosSolversModule::GetModule();

			// Make a solver
			auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);

			EnableRewindCaptureHelper(Solver, 100, Optimization);

			// Make particles
			auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto KinematicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FKinematicGeometryParticle::CreateParticle());

			const int32 LastStep = 11;

			TArray<FVec3> Xs;

			{
				auto& Dynamic = DynamicProxy->GetGameThreadAPI();
				auto& Kinematic = KinematicProxy->GetGameThreadAPI();

				Dynamic.SetGeometry(Sphere);
				Dynamic.SetGravityEnabled(true);
				Solver->RegisterObject(DynamicProxy);
				Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -1), 0);

				Kinematic.SetGeometry(Box);
				Solver->RegisterObject(KinematicProxy);

				Dynamic.SetX(FVec3(0, 0, 17));
				Dynamic.SetGravityEnabled(true);
				Dynamic.SetObjectState(EObjectStateType::Dynamic);

				Kinematic.SetX(FVec3(0, 0, 0));

				ChaosTest::SetParticleSimDataToCollide({ DynamicProxy->GetParticle_LowLevel(),KinematicProxy->GetParticle_LowLevel() });

				for (int Step = 0; Step <= LastStep; ++Step)
				{
					TickSolverHelper(Solver);
					Xs.Add(Dynamic.X());
				}

				EXPECT_GE(Dynamic.X()[2], 10);
			}

#if PHYSICS_THREAD_CONTEXT
			FPhysicsThreadContextScope Scope(true);
#endif
			auto& Dynamic = *DynamicProxy->GetPhysicsThreadAPI();
			auto& Kinematic = *KinematicProxy->GetPhysicsThreadAPI();

			const int RewindStep = 8;

			FRewindData* RewindData = Solver->GetRewindData();
			//EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

			//remove from collision, should wakeup entire island and force a soft desync
			Kinematic.SetX(FVec3(0, 0, -10000));

			auto PTDynamic = DynamicProxy->GetHandle_LowLevel()->CastToRigidParticle();	//using handle directly because outside sim callback scope and we have ensures for that
			auto PTKinematic = KinematicProxy->GetHandle_LowLevel()->CastToKinematicParticle();

			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
				//physics sim desync will not be known until the next frame because we can only compare inputs (teleport overwrites result of end of frame for example)
				if (Step > RewindStep + 1)
				{
#if REWIND_DESYNC
					EXPECT_EQ(PTDynamic->SyncState(), ESyncState::HardDesync);
#endif
				}

				TickSolverHelper(Solver);
				EXPECT_LE(Dynamic.X()[2], 10 + KINDA_SMALL_NUMBER);

#if REWIND_DESYNC
				//kinematic desync will be known at end of frame because the simulation doesn't write results (so we know right away it's a desync)
				if (Step < LastStep)
				{

					EXPECT_EQ(PTKinematic->SyncState(), ESyncState::HardDesync);
				}
				else
				{
					//everything in sync after last step
					EXPECT_EQ(PTKinematic->SyncState(), ESyncState::InSync);
					EXPECT_EQ(PTDynamic->SyncState(), ESyncState::InSync);
				}
#endif

			}

#if REWIND_DESYNC
			//both desync
			const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
			EXPECT_EQ(DesyncedParticles.Num(), 2);
			EXPECT_EQ(DesyncedParticles[0].MostDesynced, ESyncState::HardDesync);
			EXPECT_EQ(DesyncedParticles[1].MostDesynced, ESyncState::HardDesync);
#endif

			Module->DestroySolver(Solver);
		}
	}

	GTEST_TEST(AllTraits, DISABLED_RewindTest_SoftDesyncFromSameIsland)
	{
		auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
		auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100, -100, -100), FVec3(100, 100, 0)));

		FChaosSolversModule* Module = FChaosSolversModule::GetModule();

		// Make a solver
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);

		Solver->EnableRewindCapture(100,true);	//soft desync only exists when resim optimization is on

		// Make particles
		auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto KinematicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FKinematicGeometryParticle::CreateParticle());

		const int32 LastStep = 11;

		TArray<FVec3> Xs;
		{
			auto& Dynamic = DynamicProxy->GetGameThreadAPI();
			auto& Kinematic = KinematicProxy->GetGameThreadAPI();

			Dynamic.SetGeometry(Sphere);
			Dynamic.SetGravityEnabled(true);
			Solver->RegisterObject(DynamicProxy);
			Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -1), 0);

			Kinematic.SetGeometry(Box);
			Solver->RegisterObject(KinematicProxy);

			Dynamic.SetX(FVec3(0, 0, 37));
			Dynamic.SetGravityEnabled(true);
			Dynamic.SetObjectState(EObjectStateType::Dynamic);

			Kinematic.SetX(FVec3(0, 0, 0));

			ChaosTest::SetParticleSimDataToCollide({ DynamicProxy->GetParticle_LowLevel(),KinematicProxy->GetParticle_LowLevel() });


			for (int Step = 0; Step <= LastStep; ++Step)
			{
				TickSolverHelper(Solver);
				Xs.Add(Dynamic.X());
			}

			// We may end up a bit away from the surface (dt * V), due to solving for 0 velocity and not 0 position error
			EXPECT_GE(Dynamic.X()[2], 10);
			EXPECT_LE(Dynamic.X()[2], 12);
		}

#if PHYSICS_THREAD_CONTEXT
		FPhysicsThreadContextScope Scope(true);
#endif
		auto& Dynamic = *DynamicProxy->GetPhysicsThreadAPI();
		auto& Kinematic = *KinematicProxy->GetPhysicsThreadAPI();

		const int RewindStep = 0;

		FRewindData* RewindData = Solver->GetRewindData();
		//EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

		//mark kinematic as desynced (this should give us identical results which will trigger all particles in island to be soft desync)

		auto PTDynamic = DynamicProxy->GetHandle_LowLevel()->CastToRigidParticle();	//using handle directly because outside sim callback scope and we have ensures for that
		auto PTKinematic = KinematicProxy->GetHandle_LowLevel()->CastToKinematicParticle();
		PTKinematic->SetSyncState(ESyncState::HardDesync);
		bool bEverSoft = false;

		for (int Step = RewindStep; Step <= LastStep; ++Step)
		{
			TickSolverHelper(Solver);

#if REWIND_DESYNC
			//kinematic desync will be known at end of frame because the simulation doesn't write results (so we know right away it's a desync)
			if (Step < LastStep)
			{
				EXPECT_EQ(PTKinematic->SyncState(), ESyncState::HardDesync);

				//islands merge and split depending on internal solve
				//but we should see dynamic being soft desync at least once when islands merge
				if (PTDynamic->SyncState() == ESyncState::SoftDesync)
				{
					bEverSoft = true;
				}
			}
			else
			{
				//everything in sync after last step
				EXPECT_EQ(PTKinematic->SyncState(), ESyncState::InSync);
				EXPECT_EQ(PTDynamic->SyncState(), ESyncState::InSync);
			}
#endif
		}

#if REWIND_DESYNC
		//kinematic hard desync, dynamic only soft desync
		const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
		EXPECT_EQ(DesyncedParticles.Num(), 2);
		EXPECT_EQ(DesyncedParticles[0].MostDesynced, DesyncedParticles[0].Particle == KinematicProxy->GetHandle_LowLevel() ? ESyncState::HardDesync : ESyncState::SoftDesync);
		EXPECT_EQ(DesyncedParticles[1].MostDesynced, DesyncedParticles[1].Particle == KinematicProxy->GetHandle_LowLevel() ? ESyncState::HardDesync : ESyncState::SoftDesync);

		EXPECT_TRUE(bEverSoft);

		// We may end up a bit away from the surface (dt * V), due to solving for 0 velocity and not 0 position error
		EXPECT_GE(Dynamic.X()[2], 10);
		EXPECT_LE(Dynamic.X()[2], 12);
#endif

		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, DISABLED_RewindTest_SoftDesyncFromSameIslandThenBackToInSync)
	{
		auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
		auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-100, -100, -10), FVec3(100, 100, 0)));

		FChaosSolversModule* Module = FChaosSolversModule::GetModule();

		// Make a solver
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);

		Solver->EnableRewindCapture(100,true);	//soft desync only exists when resim optimization is on

		// Make particles
		auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto KinematicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FKinematicGeometryParticle::CreateParticle());

		const int32 LastStep = 15;

		TArray<FVec3> Xs;

		{
			auto& Dynamic = DynamicProxy->GetGameThreadAPI();
			auto& Kinematic = KinematicProxy->GetGameThreadAPI();

			Dynamic.SetGeometry(Sphere);
			Dynamic.SetGravityEnabled(true);
			Solver->RegisterObject(DynamicProxy);
			Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -1), 0);

			Kinematic.SetGeometry(Box);
			Solver->RegisterObject(KinematicProxy);

			Dynamic.SetX(FVec3(1000, 0, 37));
			Dynamic.SetGravityEnabled(true);
			Dynamic.SetObjectState(EObjectStateType::Dynamic);

			Kinematic.SetX(FVec3(0, 0, 0));

			ChaosTest::SetParticleSimDataToCollide({ DynamicProxy->GetParticle_LowLevel(),KinematicProxy->GetParticle_LowLevel() });

			for (int Step = 0; Step <= LastStep; ++Step)
			{
				TickSolverHelper(Solver);
				Xs.Add(Dynamic.X());
			}
		}

#if PHYSICS_THREAD_CONTEXT
		FPhysicsThreadContextScope Scope(true);
#endif
		auto& Dynamic = *DynamicProxy->GetPhysicsThreadAPI();
		auto& Kinematic = *KinematicProxy->GetPhysicsThreadAPI();

		const int RewindStep = 0;

		FRewindData* RewindData = Solver->GetRewindData();
		//EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

		//move kinematic very close but do not alter dynamic
		//should be soft desync while in island and then get back to in sync

		auto PTDynamic = DynamicProxy->GetHandle_LowLevel()->CastToRigidParticle();	//using handle directly because outside sim callback scope and we have ensures for that
		auto PTKinematic = KinematicProxy->GetHandle_LowLevel()->CastToKinematicParticle();
		Kinematic.SetX(FVec3(1000 - 110, 0, 0));

		bool bEverSoft = false;

		for (int Step = RewindStep; Step <= LastStep; ++Step)
		{
			TickSolverHelper(Solver);

#if REWIND_DESYNC
			//kinematic desync will be known at end of frame because the simulation doesn't write results (so we know right away it's a desync)
			if (Step < LastStep)
			{
				EXPECT_EQ(PTKinematic->SyncState(), ESyncState::HardDesync);

				//islands merge and split depending on internal solve
				//but we should see dynamic being soft desync at least once when islands merge
				if (PTDynamic->SyncState() == ESyncState::SoftDesync)
				{
					bEverSoft = true;
				}

				//by end should be in sync because islands should definitely be split at this point
				if (Step == LastStep - 1)
				{
					EXPECT_EQ(PTDynamic->SyncState(), ESyncState::InSync);
				}

			}
			else
			{
				//everything in sync after last step
				EXPECT_EQ(PTKinematic->SyncState(), ESyncState::InSync);
				EXPECT_EQ(PTDynamic->SyncState(), ESyncState::InSync);
			}
#endif

		}

#if REWIND_DESYNC
		//kinematic hard desync, dynamic only soft desync
		const TArray<FDesyncedParticleInfo> DesyncedParticles = RewindData->ComputeDesyncInfo();
		EXPECT_EQ(DesyncedParticles.Num(), 2);
		EXPECT_EQ(DesyncedParticles[0].MostDesynced, DesyncedParticles[0].Particle == KinematicProxy->GetHandle_LowLevel() ? ESyncState::HardDesync : ESyncState::SoftDesync);
		EXPECT_EQ(DesyncedParticles[1].MostDesynced, DesyncedParticles[1].Particle == KinematicProxy->GetHandle_LowLevel() ? ESyncState::HardDesync : ESyncState::SoftDesync);

		//no collision so just kept falling
		EXPECT_LT(Dynamic.X()[2], 10);
#endif

		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, DISABLED_RewindTest_SoftDesyncFromSameIslandThenBackToInSync_GeometryCollection_SingleFallingUnderGravity)
	{
		//TODO: disabled because at the moment GC particles are always marked as dirty - this messes up with transient dirty during rewind
		//Should probably rethink why GC has its own dirty view
		for (int Optimization = 0; Optimization < 3; ++Optimization)
		{
			FGeometryCollectionWrapper* Collection = TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init()->As<FGeometryCollectionWrapper>();

			FFramework UnitTest;
			EnableRewindCaptureHelper(UnitTest.Solver, 100, Optimization);
			UnitTest.AddSimulationObject(Collection);
			UnitTest.Initialize();

			TArray<FReal> Xs;
			const int32 LastStep = 10;
			for (int Step = 0; Step <= LastStep; ++Step)
			{
				UnitTest.Advance();
				Xs.Add(Collection->DynamicCollection->GetTransform(0).GetTranslation()[2]);
			}

			const int32 RewindStep = 3;

#if PHYSICS_THREAD_CONTEXT
			FPhysicsThreadContextScope Scope(true);
#endif
			FRewindData* RewindData = UnitTest.Solver->GetRewindData();
			//EXPECT_TRUE(RewindData->RewindToFrame(RewindStep));

			//GC doesn't marshal data from GT to PT so at the moment all we get is the GT data immediately after rewind, but it doesn't make it over to PT or collection
			//Not sure if I can even access GT particle so can't verify that, but saw it in debugger at least

			for (int Step = RewindStep; Step <= LastStep; ++Step)
			{
				UnitTest.Advance();

				//TODO: turn this on when we find a way to marshal data from GT to PT
				//EXPECT_EQ(Collection->DynamicCollection->Transform[0].GetTranslation()[2],Xs[Step]);
			}
		}
	}

	//Helps compare multiple runs for determinism
	//Also helps comparing runs across different compilers and delta times
	class FSimComparisonHelper
	{
	public:

		void SaveFrame(const TParticleView<TPBDRigidParticles<FReal, 3>>& NonDisabledDyanmic)
		{
			FEntry Frame;
			Frame.X.Reserve(NonDisabledDyanmic.Num());
			Frame.R.Reserve(NonDisabledDyanmic.Num());

			for (const auto& Dynamic : NonDisabledDyanmic)
			{
				Frame.X.Add(Dynamic.GetX());
				Frame.R.Add(Dynamic.GetR());
			}
			History.Add(MoveTemp(Frame));
		}

		static void ComputeMaxErrors(const FSimComparisonHelper& A, const FSimComparisonHelper& B, FReal& OutMaxLinearError,
			FReal& OutMaxAngularError, int32 HistoryMultiple = 1, const TArray<int32>* BMapping = nullptr)
		{
			ensure(B.History.Num() == (A.History.Num() * HistoryMultiple));

			FReal MaxLinearError2 = 0;
			FReal MaxAngularError2 = 0;

			for (int32 Idx = 0; Idx < A.History.Num(); ++Idx)
			{
				const int32 OtherIdx = Idx * HistoryMultiple + (HistoryMultiple - 1);
				const FEntry& Entry = A.History[Idx];
				const FEntry& OtherEntry = B.History[OtherIdx];

				FReal MaxLinearError, MaxAngularError;
				FEntry::CompareEntry(Entry, OtherEntry, MaxLinearError, MaxAngularError, BMapping);

				MaxLinearError2 = FMath::Max(MaxLinearError2, MaxLinearError * MaxLinearError);
				MaxAngularError2 = FMath::Max(MaxAngularError2, MaxAngularError * MaxAngularError);
			}

			OutMaxLinearError = FMath::Sqrt(MaxLinearError2);
			OutMaxAngularError = FMath::Sqrt(MaxAngularError2);
		}

	private:
		struct FEntry
		{
			TArray<FVec3> X;
			TArray<FRotation3> R;

			static void CompareEntry(const FEntry& A, const FEntry& B, FReal& OutMaxLinearError, FReal& OutMaxAngularError, const TArray<int32>* BMapping = nullptr)
			{
				FReal MaxLinearError2 = 0;
				FReal MaxAngularError2 = 0;
				
				auto BMappingHelper = [BMapping](const int32 Idx)
				{
					return BMapping ? (*BMapping)[Idx] : Idx;
				};

				check(A.X.Num() == A.R.Num());
				check(A.X.Num() == B.X.Num());
				for (int32 Idx = 0; Idx < A.X.Num(); ++Idx)
				{
					const FReal LinearError2 = (A.X[Idx] - B.X[BMappingHelper(Idx)]).SizeSquared();
					MaxLinearError2 = FMath::Max(LinearError2, MaxLinearError2);

					//if exactly the same we want 0 for testing purposes, inverse does not get that so just skip it
					if (B.R[BMappingHelper(Idx)] != A.R[Idx])
					{
						//For angular error we look at the rotation needed to go from B to A
						const FRotation3 Delta = B.R[BMappingHelper(Idx)] * A.R[Idx].Inverse();

						FVec3 Axis;
						FReal Angle;
						Delta.ToAxisAndAngleSafe(Axis, Angle, FVec3(0, 0, 1));
						const FReal Angle2 = Angle * Angle;
						MaxAngularError2 = FMath::Max(Angle2, MaxAngularError2);
					}
				}

				OutMaxLinearError = FMath::Sqrt(MaxLinearError2);
				OutMaxAngularError = FMath::Sqrt(MaxAngularError2);
			}
		};

		TArray<FEntry> History;
	};

	template <typename InitLambda>
	void RunHelper(FSimComparisonHelper& SimComparison, int32 NumSteps, FReal Dt, const InitLambda& InitFunc, const TArray<int32>* Mapping = nullptr)
	{
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();

		// Make a solver
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
		InitSolverSettings(Solver);
		Solver->SetIsDeterministic(true);

		TArray<FPhysicsActorHandle> Storage = InitFunc(Solver, Mapping);

		for (int32 Step = 0; Step < NumSteps; ++Step)
		{
			TickSolverHelper(Solver, Dt);
			SimComparison.SaveFrame(Solver->GetParticles().GetNonDisabledDynamicView());
		}

		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, DeterministicSim_SimpleFallingBox)
	{
		auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-10, -10, -10), FVec3(10, 10, 10)));

		const auto InitLambda = [&Box](auto& Solver, auto)
		{
			TArray<FPhysicsActorHandle> Storage;
			auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto& Dynamic = DynamicProxy->GetGameThreadAPI();

			Dynamic.SetGeometry(Box);
			Dynamic.SetGravityEnabled(true);
			Solver->RegisterObject(DynamicProxy);
			Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -1), 0);
			Dynamic.SetObjectState(EObjectStateType::Dynamic);

			Storage.Add(DynamicProxy);
			return Storage;
		};

		FSimComparisonHelper FirstRun;
		RunHelper(FirstRun, 100, 1 / 30.f, InitLambda);

		FSimComparisonHelper SecondRun;
		RunHelper(SecondRun, 100, 1 / 30.f, InitLambda);

		FReal MaxLinearError, MaxAngularError;
		FSimComparisonHelper::ComputeMaxErrors(FirstRun, SecondRun, MaxLinearError, MaxAngularError);
		EXPECT_EQ(MaxLinearError, 0);
		EXPECT_EQ(MaxAngularError, 0);
	}

	GTEST_TEST(AllTraits, DeterministicSim_ThresholdTest)
	{
		auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-10, -10, -10), FVec3(10, 10, 10)));

		FVec3 StartPos(0);
		FRotation3 StartRotation = FRotation3::FromIdentity();

		const auto InitLambda = [&Box, &StartPos, &StartRotation](auto& Solver, auto)
		{
			TArray<FPhysicsActorHandle> Storage;
			auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto& Dynamic = DynamicProxy->GetGameThreadAPI();

			Dynamic.SetGeometry(Box);
			Dynamic.SetGravityEnabled(true);
			Solver->RegisterObject(DynamicProxy);
			Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -1), 0);
			Dynamic.SetObjectState(EObjectStateType::Dynamic);
			Dynamic.SetX(StartPos);
			Dynamic.SetR(StartRotation);

			Storage.Add(DynamicProxy);
			return Storage;
		};

		FSimComparisonHelper FirstRun;
		RunHelper(FirstRun, 10, 1 / 30.f, InitLambda);

		//move X within threshold
		StartPos = FVec3(0, 0, 1);

		FSimComparisonHelper SecondRun;
		RunHelper(SecondRun, 10, 1 / 30.f, InitLambda);

		FReal MaxLinearError, MaxAngularError;
		FSimComparisonHelper::ComputeMaxErrors(FirstRun, SecondRun, MaxLinearError, MaxAngularError);
		EXPECT_EQ(MaxAngularError, 0);
		EXPECT_LT(MaxLinearError, 1.01);
		EXPECT_GT(MaxLinearError, 0.99);

		//move R within threshold
		StartPos = FVec3(0, 0, 0);
		StartRotation = FRotation3::FromAxisAngle(FVec3(1, 1, 0).GetSafeNormal(), 1);

		FSimComparisonHelper ThirdRun;
		RunHelper(ThirdRun, 10, 1 / 30.f, InitLambda);

		FSimComparisonHelper::ComputeMaxErrors(FirstRun, ThirdRun, MaxLinearError, MaxAngularError);
		EXPECT_EQ(MaxLinearError, 0);
		EXPECT_LT(MaxAngularError, 1.01);
		EXPECT_GT(MaxAngularError, 0.99);
	}

	GTEST_TEST(AllTraits, DeterministicSim_DoubleTick)
	{
		auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-10, -10, -10), FVec3(10, 10, 10)));

		const auto InitLambda = [&Box](auto& Solver, auto)
		{
			TArray<FPhysicsActorHandle> Storage;
			auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto& Dynamic = DynamicProxy->GetGameThreadAPI();

			Dynamic.SetGeometry(Box);
			Dynamic.SetGravityEnabled(false);
			Solver->RegisterObject(DynamicProxy);
			Dynamic.SetObjectState(EObjectStateType::Dynamic);
			Dynamic.SetV(FVec3(1, 0, 0));

			Storage.Add(DynamicProxy);
			return Storage;
		};

		FSimComparisonHelper FirstRun;
		RunHelper(FirstRun, 100, 1 / 30.f, InitLambda);

		//tick twice as often

		FSimComparisonHelper SecondRun;
		RunHelper(SecondRun, 200, 1 / 60.f, InitLambda);

		FReal MaxLinearError, MaxAngularError;
		FSimComparisonHelper::ComputeMaxErrors(FirstRun, SecondRun, MaxLinearError, MaxAngularError, 2);
		EXPECT_NEAR(MaxLinearError, 0, 1e-4);
		EXPECT_NEAR(MaxAngularError, 0, 1e-4);
	}

	GTEST_TEST(AllTraits, DeterministicSim_DoubleTickGravity)
	{
		auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-10, -10, -10), FVec3(10, 10, 10)));
		const FReal Gravity = -980;

		const auto InitLambda = [&Box, Gravity](auto& Solver, auto)
		{
			TArray<FPhysicsActorHandle> Storage;
			auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto& Dynamic = DynamicProxy->GetGameThreadAPI();

			Dynamic.SetGeometry(Box);
			Dynamic.SetGravityEnabled(true);
			Solver->RegisterObject(DynamicProxy);
			Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, Gravity), 0);
			Dynamic.SetObjectState(EObjectStateType::Dynamic);

			Storage.Add(DynamicProxy);
			return Storage;
		};

		const int32 NumSteps = 7;
		FSimComparisonHelper FirstRun;
		RunHelper(FirstRun, NumSteps, 1 / 30.f, InitLambda);

		//tick twice as often

		FSimComparisonHelper SecondRun;
		RunHelper(SecondRun, NumSteps * 2, 1 / 60.f, InitLambda);

		//expected integration gravity error
		const auto EulerIntegrationHelper = [Gravity](int32 Steps, FReal Dt)
		{
			FReal Z = 0;
			FReal V = 0;
			for (int32 Step = 0; Step < Steps; ++Step)
			{
				V += Gravity * Dt;
				Z += V * Dt;
			}

			return Z;
		};

		const FReal ExpectedZ30 = EulerIntegrationHelper(NumSteps, 1 / 30.f);
		const FReal ExpectedZ60 = EulerIntegrationHelper(NumSteps * 2, 1 / 60.f);
		EXPECT_LT(ExpectedZ30, ExpectedZ60);	//30 gains speed faster (we use the end velocity to integrate so the bigger dt, the more added energy)
		const FReal ExpectedError = ExpectedZ60 - ExpectedZ30;

		FReal MaxLinearError, MaxAngularError;
		FSimComparisonHelper::ComputeMaxErrors(FirstRun, SecondRun, MaxLinearError, MaxAngularError, 2);
		EXPECT_LT(MaxLinearError, ExpectedError + 1e-4);
		EXPECT_EQ(MaxAngularError, 0);
	}

	GTEST_TEST(AllTraits, DeterministicSim_DoubleTickCollide)
	{
		auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 50));

		const auto InitLambda = [&Sphere](auto& Solver, auto)
		{
			TArray<FPhysicsActorHandle> Storage;
			auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto& Dynamic = DynamicProxy->GetGameThreadAPI();

			Dynamic.SetGeometry(Sphere);
			Solver->RegisterObject(DynamicProxy);
			Dynamic.SetObjectState(EObjectStateType::Dynamic);
			Dynamic.SetGravityEnabled(false);
			Dynamic.SetV(FVec3(0, 0, -25));


			auto DynamicProxy2 = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto& Dynamic2 = DynamicProxy2->GetGameThreadAPI();

			Dynamic2.SetGeometry(Sphere);
			Solver->RegisterObject(DynamicProxy2);
			Dynamic2.SetX(FVec3(0, 0, -100 - 25 / 60.f - 0.1));	//make it so it overlaps for 30fps but not 60
			Dynamic2.SetGravityEnabled(false);

			ChaosTest::SetParticleSimDataToCollide({ DynamicProxy->GetParticle_LowLevel(),DynamicProxy2->GetParticle_LowLevel() });

			Storage.Add(DynamicProxy);
			Storage.Add(DynamicProxy2);

			return Storage;
		};

		const int32 NumSteps = 7;
		FSimComparisonHelper FirstRun;
		RunHelper(FirstRun, NumSteps, 1 / 30.f, InitLambda);

		//tick twice as often

		FSimComparisonHelper SecondRun;
		RunHelper(SecondRun, NumSteps * 2, 1 / 60.f, InitLambda);

		FReal MaxLinearError, MaxAngularError;
		FSimComparisonHelper::ComputeMaxErrors(FirstRun, SecondRun, MaxLinearError, MaxAngularError, 2);
	}

	GTEST_TEST(AllTraits, DeterministicSim_DoubleTickStackCollide)
	{
		auto SmallBox = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-1000, -1000, -1000), FVec3(1000, 1000, 0)));

		const auto InitLambda = [&SmallBox, &Box](auto& Solver, auto)
		{
			Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -980), 0);
			TArray<FPhysicsActorHandle> Storage;
			for (int Idx = 0; Idx < 5; ++Idx)
			{
				auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
				auto& Dynamic = DynamicProxy->GetGameThreadAPI();

				Dynamic.SetGeometry(SmallBox);
				Solver->RegisterObject(DynamicProxy);
				Dynamic.SetObjectState(EObjectStateType::Dynamic);
				Dynamic.SetGravityEnabled(true);
				Dynamic.SetX(FVec3(0, 20 * Idx, 100 * Idx));	//slightly offset

				Storage.Add(DynamicProxy);
			}

			auto KinematicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FKinematicGeometryParticle::CreateParticle());
			auto& Kinematic = KinematicProxy->GetGameThreadAPI();

			Kinematic.SetGeometry(Box);
			Solver->RegisterObject(KinematicProxy);
			Kinematic.SetX(FVec3(0, 0, -50));

			Storage.Add(KinematicProxy);

			for (int i = 0; i < Storage.Num(); ++i)
			{
				for (int j = i + 1; j < Storage.Num(); ++j)
				{
					ChaosTest::SetParticleSimDataToCollide({ Storage[i]->GetParticle_LowLevel(),Storage[j]->GetParticle_LowLevel() });
				}
			}

			return Storage;
		};

		const int32 NumSteps = 20;
		FSimComparisonHelper FirstRun;
		RunHelper(FirstRun, NumSteps, 1 / 30.f, InitLambda);

		//tick twice as often

		FSimComparisonHelper SecondRun;
		RunHelper(SecondRun, NumSteps, 1 / 30.f, InitLambda);

		//make sure deterministic
		FReal MaxLinearError, MaxAngularError;
		FSimComparisonHelper::ComputeMaxErrors(FirstRun, SecondRun, MaxLinearError, MaxAngularError, 1);
		EXPECT_EQ(MaxLinearError, 0);
		EXPECT_EQ(MaxAngularError, 0);

		//try with 60fps
		FSimComparisonHelper ThirdRun;
		RunHelper(ThirdRun, NumSteps * 2, 1 / 60.f, InitLambda);

		FSimComparisonHelper::ComputeMaxErrors(FirstRun, ThirdRun, MaxLinearError, MaxAngularError, 2);
	}
	
	GTEST_TEST(AllTraits, DeterministicSim_DifferentCreationOrder)
	{
		auto SmallBox = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		auto Box = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-1000, -1000, -1000), FVec3(1000, 1000, 0)));

		const int32 NumParticles = 50;
		const auto InitLambda = [&SmallBox, &Box, NumParticles](auto& Solver, const TArray<int32>* Mapping)
		{
			auto MappingHelper = [Mapping](const int32 Idx) { return Mapping ? (*Mapping)[Idx] : Idx; };
			Solver->GetEvolution()->GetGravityForces().SetAcceleration(FVec3(0, 0, -980), 0);
			TArray<FPhysicsActorHandle> Storage;
			for (int Idx = 0; Idx < NumParticles; ++Idx)
			{
				auto DynamicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
				auto& Dynamic = DynamicProxy->GetGameThreadAPI();

				Dynamic.SetGeometry(SmallBox);
				Solver->RegisterObject(DynamicProxy);
				Dynamic.SetObjectState(EObjectStateType::Dynamic);
				Dynamic.SetGravityEnabled(true);
				Dynamic.SetParticleID(FParticleID{ MappingHelper(Idx), INDEX_NONE });
				Dynamic.SetX(FVec3(0, 5 * MappingHelper(Idx), 100 * MappingHelper(Idx) + 50));	//slightly offset
				Dynamic.SetI(FVec3(1000, 1000, 1000));
				Dynamic.SetInvI(FVec3(1/1000.0, 1/1000.0, 1/1000.0));

				Storage.Add(DynamicProxy);
			}

			auto KinematicProxy = FSingleParticlePhysicsProxy::Create(Chaos::FKinematicGeometryParticle::CreateParticle());
			auto& Kinematic = KinematicProxy->GetGameThreadAPI();

			Kinematic.SetGeometry(Box);
			Solver->RegisterObject(KinematicProxy);
			Kinematic.SetX(FVec3(0, 0, 0));

			Storage.Add(KinematicProxy);

			for (int i = 0; i < Storage.Num(); ++i)
			{
				for (int j = i + 1; j < Storage.Num(); ++j)
				{
					ChaosTest::SetParticleSimDataToCollide({ Storage[i]->GetParticle_LowLevel(),Storage[j]->GetParticle_LowLevel() });
				}
			}

			return Storage;
		};

		const int32 NumSteps = 20;
		FSimComparisonHelper FirstRun;

		RunHelper(FirstRun, NumSteps, 1 / 30.f, InitLambda);

		//tick twice as often

		TArray<int32> Mapping;
		for (int32 Idx = 0; Idx < NumParticles; ++Idx)
		{
			Mapping.Add(NumParticles - Idx - 1);
			//Mapping.Add(Idx);
		}

		FSimComparisonHelper SecondRun;
		RunHelper(SecondRun, NumSteps, 1 / 30.f, InitLambda, &Mapping);

		//make sure deterministic
		FReal MaxLinearError, MaxAngularError;
		FSimComparisonHelper::ComputeMaxErrors(FirstRun, SecondRun, MaxLinearError, MaxAngularError, 1, &Mapping);
		EXPECT_EQ(MaxLinearError, 0);
		EXPECT_EQ(MaxAngularError, 0);
	}


	GTEST_TEST(AllTraits, RewindTest_InterpolatedTwoChannels)
	{
		Chaos::AsyncInterpolationMultiplier = 3.0f;
		int32 PrevNumActiveChannels = Chaos::DefaultNumActiveChannels;
		Chaos::DefaultNumActiveChannels = 2;
		//Have two moving particles, one in each channel to see that there's a delay in time on second channel
		TRewindHelper::TestDynamicSphere([](auto* Solver, FReal SimDt, int32 Optimization, auto Proxy, auto Sphere)
		{
			if (!Solver->IsUsingAsyncResults()) { return; }
			auto& Particle = Proxy->GetGameThreadAPI();
			Particle.SetV(FVec3(0, 0, 1));
			Particle.SetGravityEnabled(false);

			auto Proxy2 = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			auto& Particle2 = Proxy2->GetGameThreadAPI();

			auto Sphere2 = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
			Particle2.SetGeometry(Sphere2);
			Particle2.SetV(FVec3(0, 0, 1));
			Particle2.SetGravityEnabled(false);
			if (FProxyInterpolationBase* InterpolationData = Proxy2->GetInterpolationData())
			{
				InterpolationData->SetInterpChannel_External(1);
			}
			Solver->RegisterObject(Proxy2);

			FReal Time = 0;
			const FReal GtDt = 1;
			for (int Step = 0; Step < 32; ++Step)
			{
				TickSolverHelper(Solver);

				Time += GtDt;
				const FReal InterpolatedTime0 = Time - SimDt * Solver->GetAsyncInterpolationMultiplier();
				const FReal InterpolatedTime1 = InterpolatedTime0 - Chaos::SecondChannelDelay;

				if (InterpolatedTime0 < 0)
				{
					//No movement yet
					EXPECT_NEAR(Particle.X()[2], 0, 1e-2);
				}
				else
				{
					EXPECT_NEAR(Particle.X()[2], InterpolatedTime0, 1e-2);
				}

				if (InterpolatedTime1 < 0)
				{
					//No movement yet
					EXPECT_NEAR(Particle2.X()[2], 0, 1e-2);
				}
				else
				{
					EXPECT_NEAR(Particle2.X()[2], InterpolatedTime1, 1e-2);
				}
			}
		});

		Chaos::DefaultNumActiveChannels = PrevNumActiveChannels;
	}


}
