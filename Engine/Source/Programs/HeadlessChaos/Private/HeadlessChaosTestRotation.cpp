// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/Rotation.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PerParticlePBDEulerStep.h"
#include "Chaos/PerParticlePBDUpdateFromDeltaPosition.h"

namespace ChaosTest {
	
	using namespace Chaos;

	/**
	 * Check that the angular velocity calculated from change in Rotation
	 * matches the results of angular velocity integration.
	 */
	void RotationAngularVelocity()
	{
		FMath::RandInit(8926599);

		const FReal Dt = (FReal)1 / 30;
		const FReal MaxAngVel = FMath::DegreesToRadians(15 /* per tick */) / Dt;

		FVec3 Dims = { 100, 200, 300 };
		TArray<FVec3> AngularVelocities =
		{
			{1, 0, 0},
			{0, 1, 0},
			{0, 0, 1},
			{MaxAngVel, 0, 0},
			{0, MaxAngVel, 0},
			{0, 0, MaxAngVel},
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
			RandAxis() * FMath::FRandRange((FReal)0.01, MaxAngVel),
		};

		FPBDRigidParticles Particles;
		FPerParticlePBDEulerStep StepRule;

		for (int32 ParticleIndex = 0; ParticleIndex < AngularVelocities.Num(); ++ParticleIndex)
		{
			AppendAnalyticBox(Particles, Dims);

			// Particles.SetR(ParticleIndex, FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange(-PI, PI)).GetNormalized());
			Particles.SetR(ParticleIndex, FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange(-PI, PI)).GetNormalized());
			Particles.SetQ(ParticleIndex, Particles.GetR(ParticleIndex));
			Particles.SetV(ParticleIndex, FVec3::ZeroVector);
			Particles.SetW(ParticleIndex, AngularVelocities[ParticleIndex]);
			Particles.SetPreV(ParticleIndex, Particles.GetV(ParticleIndex));
			Particles.SetPreW(ParticleIndex, Particles.GetW(ParticleIndex));
			Particles.CenterOfMass(ParticleIndex) = FVec3(0);
			Particles.RotationOfMass(ParticleIndex) = FRotation3::FromIdentity();
		}

		for (int32 ParticleIndex = 0; ParticleIndex < (int32)Particles.Size(); ++ParticleIndex)
		{
			FRotation3 R0 = Particles.GetQ(ParticleIndex);

			StepRule.Apply(Particles, Dt, ParticleIndex);

			FRotation3 R1 = Particles.GetQ(ParticleIndex);
			FVec3 AngularVelocity = Particles.GetW(ParticleIndex);

			FVec3 CalculatedAngularVelocity1 = FRotation3::CalculateAngularVelocity1(R0, R1, Dt);

			// Verify that we calculated the same angular velocity that was used to integrate the rotation
			const FReal ExpectedAccuracy = AngularVelocity.Size() * (FReal)0.01;
			FVec3 Error = CalculatedAngularVelocity1 - AngularVelocity;
			EXPECT_NEAR(Error[0], (FReal)0, ExpectedAccuracy);
			EXPECT_NEAR(Error[1], (FReal)0, ExpectedAccuracy);
			EXPECT_NEAR(Error[2], (FReal)0, ExpectedAccuracy);

			// Verify that the two ang vel algorithm give roughly the same result
			FVec3 CalculatedAngularVelocity2 = FRotation3::CalculateAngularVelocity2(R0, R1, Dt);
			EXPECT_NEAR(CalculatedAngularVelocity1.X, CalculatedAngularVelocity2.X, (FReal)0.1);
			EXPECT_NEAR(CalculatedAngularVelocity1.Y, CalculatedAngularVelocity2.Y, (FReal)0.1);
			EXPECT_NEAR(CalculatedAngularVelocity1.Z, CalculatedAngularVelocity2.Z, (FReal)0.1);

			// Verify that the rotation's integration function agrees with the EulerStep rule
			FRotation3 R12 = FRotation3::IntegrateRotationWithAngularVelocity(R0, AngularVelocity, Dt);
			EXPECT_NEAR(R12.X, R1.X, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(R12.Y, R1.Y, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(R12.Z, R1.Z, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(R12.W, R1.W, KINDA_SMALL_NUMBER);
		}
	}


	TEST(RotationTests, AngularVelocityIsCorrect)
	{
		ChaosTest::RotationAngularVelocity();

		SUCCEED();
	}

	//
	//
	//

	/**
	 * Check that the angular velocity calculated from the change in quaternion agrees
	 * with the angular velocity calculation.
	 */
	void QuaternionIntegrateIsCorrect()
	{
		FMath::RandInit(8926599);
		const FReal Dt = (FReal)1 / 30;
		const FReal MaxAngVel = FMath::DegreesToRadians(15 /* per tick */) / Dt;
		FVec3 Dims = { 100, 200, 300 };

		FPBDRigidParticles Particles;
		FPerParticlePBDUpdateFromDeltaPosition UpdateRule;

		TArray<FRotation3> InitialRotations =
		{
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
		};
		TArray<FRotation3> FinalRotations =
		{
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
			FRotation3::FromAxisAngle(RandAxis(), FMath::RandRange((FReal)0, (FReal)2 * PI)),
		};

		for (int32 ParticleIndex = 0; ParticleIndex < InitialRotations.Num(); ++ParticleIndex)
		{
			FinalRotations[ParticleIndex].EnforceShortestArcWith(InitialRotations[ParticleIndex]);

			AppendAnalyticBox(Particles, Dims);

			Particles.SetX(ParticleIndex, FVec3::ZeroVector);
			Particles.SetP(ParticleIndex, FVec3::ZeroVector);
			Particles.SetR(ParticleIndex, InitialRotations[ParticleIndex]);
			Particles.SetQ(ParticleIndex, FinalRotations[ParticleIndex]);
			Particles.SetV(ParticleIndex, FVec3::ZeroVector);
			Particles.SetW(ParticleIndex, FVec3::ZeroVector);
			Particles.SetPreV(ParticleIndex, Particles.GetV(ParticleIndex));
			Particles.SetPreW(ParticleIndex, Particles.GetW(ParticleIndex));
			Particles.CenterOfMass(ParticleIndex) = FVec3(0);
			Particles.RotationOfMass(ParticleIndex) = FRotation3::FromIdentity();
		}

		for (int32 ParticleIndex = 0; ParticleIndex < (int32)Particles.Size(); ++ParticleIndex)
		{
			FRotation3 R0 = Particles.GetR(ParticleIndex);
			FRotation3 R1 = Particles.GetQ(ParticleIndex);

			UpdateRule.Apply(Particles, Dt, ParticleIndex);

			FVec3 ExpectedAngVel = Particles.GetW(ParticleIndex);
			FVec3 CalculatedAngVel1 = FRotation3::CalculateAngularVelocity1(R0, R1, Dt);

			EXPECT_NEAR(ExpectedAngVel.X, CalculatedAngVel1.X, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(ExpectedAngVel.Y, CalculatedAngVel1.Y, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(ExpectedAngVel.Z, CalculatedAngVel1.Z, KINDA_SMALL_NUMBER);
		}

	}

	TEST(RotationTests, QuaternionIntegrateIsCorrect)
	{
		ChaosTest::QuaternionIntegrateIsCorrect();

		SUCCEED();
	}

	//
	//
	//

	void QuaternionMatrixConcatenation()
	{
		FRotation3 QZ = FRotation3::FromAxisAngle(FVec3(0, 0, 1), FMath::DegreesToRadians(30));
		FRotation3 QX = FRotation3::FromAxisAngle(FVec3(1, 0, 0), FMath::DegreesToRadians(30));
		FMatrix33 MZ = QZ.ToMatrix();
		FMatrix33 MX = QX.ToMatrix();

		FRotation3 QZX = (QZ * QX);
		FMatrix33 MZX = Utilities::Multiply(MZ, MX);

		FVec3 V1 = FVec3(1, 1, 1);
		FVec3 V1ZX_Q1 = QZ * (QX * V1);
		FVec3 V1ZX_Q2 = (QZ * QX) * V1;
		FVec3 V1ZX_Q3 = QZX * V1;
		EXPECT_NEAR(V1ZX_Q1.X, V1ZX_Q2.X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1ZX_Q1.Y, V1ZX_Q2.Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1ZX_Q1.Z, V1ZX_Q2.Z, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1ZX_Q1.X, V1ZX_Q3.X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1ZX_Q1.Y, V1ZX_Q3.Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1ZX_Q1.Z, V1ZX_Q3.Z, KINDA_SMALL_NUMBER);

		FVec3 V1X_Q = QX * V1;
		FVec3 V1Z_Q = QZ * V1;
		FVec3 V1X_M = Utilities::Multiply(MX, V1);
		FVec3 V1Z_M = Utilities::Multiply(MZ, V1);
		EXPECT_NEAR(V1X_Q.X, V1X_M.X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1X_Q.Y, V1X_M.Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1X_Q.Z, V1X_M.Z, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1Z_Q.X, V1Z_M.X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1Z_Q.Y, V1Z_M.Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1Z_Q.Z, V1Z_M.Z, KINDA_SMALL_NUMBER);

		FVec3 V1ZX_M1 = Utilities::Multiply(MZ, Utilities::Multiply(MX, V1));
		FVec3 V1ZX_M2 = Utilities::Multiply(MZX, V1);
		EXPECT_NEAR(V1ZX_Q1.X, V1ZX_M1.X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1ZX_Q1.Y, V1ZX_M1.Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1ZX_Q1.Z, V1ZX_M1.Z, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1ZX_Q1.X, V1ZX_M2.X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1ZX_Q1.Y, V1ZX_M2.Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(V1ZX_Q1.Z, V1ZX_M2.Z, KINDA_SMALL_NUMBER);
	}

	TEST(RotationTests, QuaternionMatrixConcatenation)
	{
		ChaosTest::QuaternionMatrixConcatenation();

		SUCCEED();
	}

	//
	//
	//

	// Check CrossProductMatrix has correctly signed results for unit axes
	//		X x Y = Z;		Y x Z = X;		Z x X = Y;
	//		Y x X = -Z;		Z x Y = -X;		X x Z = -Y;
	void TestCrossProductMatrix_Axes()
	{
		const FReal Tolerance = KINDA_SMALL_NUMBER;

		TArray<FVec3> Axes =
		{
			FVec3(1, 0, 0),
			FVec3(0, 1, 0),
			FVec3(0, 0, 1),
		};

		for (int AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
		{
			int Axis1Index = AxisIndex;
			int Axis2Index = (AxisIndex + 1) % 3;
			int Axis3Index = (AxisIndex + 2) % 3;

			FVec3 V1 = Axes[Axis1Index];
			FVec3 V2 = Axes[Axis2Index];
			FVec3 V3 = Axes[Axis3Index];

			FVec3 VectorCrossP = FVec3::CrossProduct(V1, V2);
			EXPECT_NEAR(VectorCrossP.X, V3.X, Tolerance);
			EXPECT_NEAR(VectorCrossP.Y, V3.Y, Tolerance);
			EXPECT_NEAR(VectorCrossP.Z, V3.Z, Tolerance);

			FVec3 VectorCrossN = FVec3::CrossProduct(V2, V1);
			EXPECT_NEAR(VectorCrossN.X, -V3.X, Tolerance);
			EXPECT_NEAR(VectorCrossN.Y, -V3.Y, Tolerance);
			EXPECT_NEAR(VectorCrossN.Z, -V3.Z, Tolerance);

			FMatrix33 V1M = Utilities::CrossProductMatrix(V1);
			FVec3 MatrixCrossP = Utilities::Multiply(V1M, V2);
			EXPECT_NEAR(MatrixCrossP.X, V3.X, Tolerance);
			EXPECT_NEAR(MatrixCrossP.Y, V3.Y, Tolerance);
			EXPECT_NEAR(MatrixCrossP.Z, V3.Z, Tolerance);

			FVec3 MatrixCrossN = Utilities::Multiply(FMatrix33(V1M.GetTransposed()), V2);
			EXPECT_NEAR(MatrixCrossN.X, -V3.X, Tolerance);
			EXPECT_NEAR(MatrixCrossN.Y, -V3.Y, Tolerance);
			EXPECT_NEAR(MatrixCrossN.Z, -V3.Z, Tolerance);
		}
	}

	TEST(MathTests, TestCrossProductMatrix_Axes)
	{
		TestCrossProductMatrix_Axes();
	}

	//
	//
	//

	// Check that CrossProductMatrix agrees with UE cross product
	void TestCrossProductMatrix()
	{
		FMath::RandInit(10695676);

		const FReal MinValue = -100;
		const FReal MaxValue = 100;
		const FReal Tolerance = KINDA_SMALL_NUMBER;

		for (int RandIndex = 0; RandIndex < 100; ++RandIndex)
		{
			FVec3 V1 = RandomVector(MinValue, MaxValue);
			FVec3 V2 = RandomVector(MinValue, MaxValue);
			FVec3 CrossExpected = FVec3::CrossProduct(V1, V2);

			FMatrix33 V1M = Utilities::CrossProductMatrix(V1);
			FVec3 CrossActual = Utilities::Multiply(V1M, V2);

			EXPECT_NEAR(CrossActual.X, CrossExpected.X, Tolerance);
			EXPECT_NEAR(CrossActual.Y, CrossExpected.Y, Tolerance);
			EXPECT_NEAR(CrossActual.Z, CrossExpected.Z, Tolerance);
		}
	}

	TEST(MathTests, TestCrossProductMatrix)
	{
		TestCrossProductMatrix();
	}

	//
	//
	//

	void TestRotationHandedness()
	{
		// UE uses a left-handed coordinates (Z up) and left-handed rotations
		//
		//          Z
		//          |
		//          |
		//          *----X
		//        /
		//      Y 
		//
		//	+ve rotations about X will rotate Y towards Z
		//	+ve rotations about Y will rotate Z towards X
		//	+ve rotations about Z will rotate X towards Y
		//
		{
			// Rotate about X: 
			FVec3 Axis = FVec3(1, 0, 0);
			FReal AngleDeg = 45;
			FReal Angle = FMath::DegreesToRadians(AngleDeg);
			FVec3 Y = FVec3(0, 1, 0);
			FVec3 Z = FVec3(0, 0, 1);
			FRotation3 R = FRotation3::FromAxisAngle(Axis, Angle);
			FVec3 RY = R * Y;
			FVec3 RZ = R * Z;

			EXPECT_GT(RY.Z, 0);
			EXPECT_LT(RZ.Y, 0);
		}
		{
			// Rotate about Y: 
			FVec3 Axis = FVec3(0, 1, 0);
			FReal AngleDeg = 45;
			FReal Angle = FMath::DegreesToRadians(AngleDeg);
			FVec3 X = FVec3(1, 0, 0);
			FVec3 Z = FVec3(0, 0, 1);
			FRotation3 R = FRotation3::FromAxisAngle(Axis, Angle);
			FVec3 RX = R * X;
			FVec3 RZ = R * Z;

			EXPECT_LT(RX.Z, 0);
			EXPECT_GT(RZ.X, 0);
		}
		{
			// Rotate about Z: 
			FVec3 Axis = FVec3(0, 0, 1);
			FReal AngleDeg = 45;
			FReal Angle = FMath::DegreesToRadians(AngleDeg);
			FVec3 X = FVec3(1, 0, 0);
			FVec3 Y = FVec3(0, 1, 0);
			FRotation3 R = FRotation3::FromAxisAngle(Axis, Angle);
			FVec3 RX = R * X;
			FVec3 RY = R * Y;

			EXPECT_GT(RX.Y, 0);
			EXPECT_LT(RY.X, 0);
		}
	}

	TEST(MathTests, TestRotationHandedness)
	{
		TestRotationHandedness();
	}

	//
	//
	//

	void TestQuaternionAngVel_WorldSpace()
	{
		// Check that a world-space angular velocity has the expected effect on the axes of
		// two objects, one is upside down.

		// Set up two rotations about the Z axis, 180 degrees apart. One will have X axis pointing up, one down.
		FRotation3 Qa = FRotation3::FromAxisAngle(FVec3(0, 0, 1), FMath::DegreesToRadians(0));
		FRotation3 Qb = FRotation3::FromAxisAngle(FVec3(0, 0, 1), FMath::DegreesToRadians(180));
		FMatrix33 PreAxesa = Qa.ToMatrix();
		FMatrix33 PreAxesb = Qb.ToMatrix();

		// Relative axes of rotations...
		// X axes should be opposing
		// Y axes should be opposing
		// Z axes should be aligned
		EXPECT_NEAR((FVec3::DotProduct(PreAxesa.GetAxis(0), PreAxesb.GetAxis(0))), (FReal)-1, KINDA_SMALL_NUMBER);
		EXPECT_NEAR((FVec3::DotProduct(PreAxesa.GetAxis(1), PreAxesb.GetAxis(1))), (FReal)-1, KINDA_SMALL_NUMBER);
		EXPECT_NEAR((FVec3::DotProduct(PreAxesa.GetAxis(2), PreAxesb.GetAxis(2))), (FReal)1, KINDA_SMALL_NUMBER);

		// Angular delta about X
		FVec3 W = FVec3(FMath::DegreesToRadians(20), 0, 0);
		FRotation3 Wq = FRotation3::FromElements(W, 0);

		FRotation3 DQa = (Wq * Qa) * (FReal)0.5;
		FRotation3 DQb = (Wq * Qb) * (FReal)0.5;

		FRotation3 Qa2 = (Qa + DQa).GetNormalized();
		FRotation3 Qb2 = (Qb + DQb).GetNormalized();
		FMatrix33 PostAxesa = Qa2.ToMatrix();
		FMatrix33 PostAxesb = Qb2.ToMatrix();

		// Both were rotated in world space by the same amount - they should still be relatively-aligned in the same way
		EXPECT_NEAR((FVec3::DotProduct(PostAxesa.GetAxis(0), PostAxesb.GetAxis(0))), (FReal)-1, KINDA_SMALL_NUMBER);
		EXPECT_NEAR((FVec3::DotProduct(PostAxesa.GetAxis(1), PostAxesb.GetAxis(1))), (FReal)-1, KINDA_SMALL_NUMBER);
		EXPECT_NEAR((FVec3::DotProduct(PostAxesa.GetAxis(2), PostAxesb.GetAxis(2))), (FReal)1, KINDA_SMALL_NUMBER);
	}

	TEST(MathTests, TestQuaternionAngVel_WorldSpace)
	{
		TestQuaternionAngVel_WorldSpace();
	}
}
