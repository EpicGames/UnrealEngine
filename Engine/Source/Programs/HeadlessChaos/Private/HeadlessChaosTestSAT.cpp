// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/Core.h"
#include "Chaos/SAT.h"
#include "Chaos/Convex.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Particles.h"

namespace ChaosTest
{
	using namespace Chaos;

	// Two boxes of same size separated along the Z axis with no rotation.
	// Check that we get the correct separation
	GTEST_TEST(SATTests, TestSATBox)
	{
		const FVec3 BoxSize = FVec3(100, 100, 100);
		const FReal BoxMargin = 0;
		const FReal CullDistance = 100;
		FSATSettings SATSettings;

		const FRigidTransform3 Convex1Transform = FRigidTransform3(FVec3(0, 0, 0), FRotation3::FromAxisAngle(FVec3(1, 0, 0), 0.0f));
		const FRigidTransform3 Convex2Transform = FRigidTransform3(FVec3(0, 0, 150), FRotation3::FromAxisAngle(FVec3(1, 0, 0), 0.0f));
		const FImplicitBox3 Convex1 = FImplicitBox3(-0.5f * BoxSize, 0.5f * BoxSize, BoxMargin);
		const FImplicitBox3 Convex2 = FImplicitBox3(-0.5f * BoxSize, 0.5f * BoxSize, BoxMargin);

		FSATResult SATResult = SATPenetration(Convex1, Convex1Transform, Convex2, Convex2Transform, CullDistance, SATSettings);

		EXPECT_TRUE(SATResult.IsValid());

		const FReal DistanceTolerance = 1.e-3f;
		const FReal ExpectedSeparation = 50;
		EXPECT_NEAR(SATResult.SignedDistance, ExpectedSeparation, DistanceTolerance);
	}


	// Two convex boxes of same size separated along the Z axis with no rotation.
	// Check that we get the correct separation
	GTEST_TEST(SATTests, TestSATConvexBox)
	{
		const FVec3 BoxSize = FVec3(100, 100, 100);
		const FReal BoxMargin = 0;
		const FReal CullDistance = 100;
		FSATSettings SATSettings;

		const FRigidTransform3 Convex1Transform = FRigidTransform3(FVec3(0, 0, 0), FRotation3::FromAxisAngle(FVec3(1, 0, 0), 0.0f));
		const FRigidTransform3 Convex2Transform = FRigidTransform3(FVec3(0, 0, 150), FRotation3::FromAxisAngle(FVec3(1, 0, 0), 0.0f));
		const FImplicitConvex3 Convex1 = CreateConvexBox(BoxSize, BoxMargin);
		const FImplicitConvex3 Convex2 = CreateConvexBox(BoxSize, BoxMargin);

		FSATResult SATResult = SATPenetration(Convex1, Convex1Transform, Convex2, Convex2Transform, CullDistance, SATSettings);

		EXPECT_TRUE(SATResult.IsValid());

		const FReal DistanceTolerance = 1.e-3f;
		const FReal ExpectedSeparation = 50;
		EXPECT_NEAR(SATResult.SignedDistance, ExpectedSeparation, DistanceTolerance);
	}

	// Two scaled convex boxes of same size separated along the Z axis with no rotation.
	// Check that we get the correct separation
	GTEST_TEST(SATTests, TestSATScaledConvexBox)
	{
		const FVec3 BoxSize = FVec3(50, 100, 50);
		const FVec3 BoxScale = FVec3(2, 1, 2);
		const FReal BoxMargin = 0;
		const FReal CullDistance = 100;
		FSATSettings SATSettings;

		const FRigidTransform3 Convex1Transform = FRigidTransform3(FVec3(0, 0, 0), FRotation3::FromAxisAngle(FVec3(1, 0, 0), 0.0f));
		const FRigidTransform3 Convex2Transform = FRigidTransform3(FVec3(0, 0, 150), FRotation3::FromAxisAngle(FVec3(1, 0, 0), 0.0f));
		const TImplicitObjectScaled<FImplicitConvex3> Convex1 = CreateScaledConvexBox(BoxSize, BoxScale, BoxMargin);
		const TImplicitObjectScaled<FImplicitConvex3> Convex2 = CreateScaledConvexBox(BoxSize, BoxScale, BoxMargin);

		FSATResult SATResult = SATPenetration(Convex1, Convex1Transform, Convex2, Convex2Transform, CullDistance, SATSettings);

		EXPECT_TRUE(SATResult.IsValid());

		const FReal DistanceTolerance = 1.e-3f;
		const FReal ExpectedSeparation = 50;
		EXPECT_NEAR(SATResult.SignedDistance, ExpectedSeparation, DistanceTolerance);
	}

}