// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestSerialization.h"

//PRAGMA_DISABLE_OPTIMIZATION

#include "HeadlessChaos.h"
#include "Chaos/ChaosArchive.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/RigidParticles.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/SerializationTestUtility.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/ChaosPerfTest.h"
#include "Chaos/HeightField.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Chaos/BoundingVolumeHierarchy.h"
namespace
{
	FString GetSerializedBinaryPath();
}

namespace ChaosTest
{
	using namespace Chaos;

	FString GetSerializedBinaryPath()
	{
		return FPaths::EngineDir() / TEXT("Source/Programs/HeadlessChaos/SerializedBinaries");
	}

	void SimpleTypesSerialization()
	{
		FReal Real = 12345.6;
		FVec2 Vec2 { 12.3, 45.6 };
		FVec3 Vec3 { 12.3, 45.6, 78.9 };
		FVec4 Vec4 { 12.3, 45.6, 78.9, 32.1 };
		FRotation3 Rot3( FQuat{ 0, 0, 0, 1 });
		FMatrix33 Mat3 = RandomMatrix(-10, 10);
		FVector FVec{ 12.3, 45.6, 78.9 };

		TArray<uint8> Data;
		{
			FMemoryWriter Ar(Data);
			FChaosArchive Writer(Ar);

			Writer << Real << Vec2 << Vec3 << Vec4 << Rot3 << Mat3 << FVec;
		}

		{
			FReal SerializedReal;
			FVec2 SerializedVec2;
			FVec3 SerializedVec3;
			FVec4 SerializedVec4;
			FRotation3 SerializedRot3;
			FMatrix33 SerializedMat3;
			FVector SerializedFVec;
			{
				FMemoryReader Ar(Data);
				FChaosArchive Reader(Ar);

				Reader << SerializedReal << SerializedVec2 << SerializedVec3 << SerializedVec4 << SerializedRot3 << SerializedMat3 << SerializedFVec;
			}
			EXPECT_EQ(Real, SerializedReal);
			EXPECT_EQ(Vec2, SerializedVec2);
			EXPECT_EQ(Vec3, SerializedVec3);
			EXPECT_EQ(Vec4, SerializedVec4);
			EXPECT_EQ(Rot3, SerializedRot3);
			EXPECT_EQ(Mat3, SerializedMat3);
			EXPECT_EQ(FVec, SerializedFVec);
		}
	}

	void SimpleObjectsSerialization()
	{

		TArray<FSpherePtr> OriginalSpheres;
		OriginalSpheres.Add(FSpherePtr(new Chaos::FSphere(FVec3(), 1)));
		OriginalSpheres.Add(FSpherePtr(new Chaos::FSphere(FVec3(), 2)));
		OriginalSpheres.Add(FSpherePtr(new Chaos::FSphere(FVec3(), 3)));

		TArray<uint8> Data;
		{
			FMemoryWriter Ar(Data);
			FChaosArchive Writer(Ar);

			Writer << OriginalSpheres;
		}

		{
			FMemoryReader Ar(Data);
			FChaosArchive Reader(Ar);
			TArray<FSpherePtr> SerializedSpheres;

			Reader << SerializedSpheres;

			EXPECT_TRUE(SerializedSpheres.Num() == OriginalSpheres.Num());

			for (int32 Idx = 0; Idx < SerializedSpheres.Num(); ++Idx)
			{
				EXPECT_TRUE(SerializedSpheres[Idx]->GetRadiusf() == OriginalSpheres[Idx]->GetRadiusf());
			}
		}
	}

	void SharedObjectsSerialization()
	{
		TArray<FSpherePtr> OriginalSpheres;
		FSpherePtr Sphere(new Chaos::FSphere(FVec3(0), 1));
		OriginalSpheres.Add(Sphere);
		OriginalSpheres.Add(Sphere);
		TSerializablePtr<Chaos::FSphere> SerializableSphere(Sphere);

		TArray<uint8> Data;
		{
			FMemoryWriter Ar(Data);
			FChaosArchive Writer(Ar);

			Writer << OriginalSpheres;
			Writer << SerializableSphere;
		}
		
		{
			TArray<FSpherePtr> SerializedSpheres;
			TSerializablePtr<Chaos::FSphere> SerializedSphere;
			{
				FMemoryReader Ar(Data);
				FChaosArchive Reader(Ar);

				Reader << SerializedSpheres;
				Reader << SerializedSphere;

				EXPECT_TRUE(SerializedSpheres.Num() == OriginalSpheres.Num());
				EXPECT_EQ(SerializedSphere.GetReference(), SerializedSpheres[0].GetReference());

				for (int32 Idx = 0; Idx < SerializedSpheres.Num(); ++Idx)
				{
					EXPECT_TRUE(SerializedSpheres[Idx]->GetRadiusf() == OriginalSpheres[Idx]->GetRadiusf());
				}

				EXPECT_EQ(SerializedSpheres[0].GetReference(), SerializedSpheres[1].GetReference());
				EXPECT_EQ(SerializedSpheres[0].GetRefCount(), 3);
			}
			EXPECT_EQ(SerializedSpheres[0].GetRefCount(), 2);	//archive is gone so ref count went down
		}
	}

	void GraphSerialization()
	{
		TArray<Chaos::FImplicitObjectPtr> OriginalSpheres;
		OriginalSpheres.Emplace(new Chaos::FSphere{ FVec3(1,2,3), 1 });
		OriginalSpheres.Emplace(new Chaos::FSphere{ FVec3(1,2,3), 2 });

		TArray<Chaos::FImplicitObjectPtr> OriginalChildren;
		OriginalChildren.Emplace(new TImplicitObjectTransformed<FReal, 3>(OriginalSpheres[0], FRigidTransform3::Identity));
		OriginalChildren.Emplace(new TImplicitObjectTransformed<FReal, 3>(OriginalSpheres[1], FRigidTransform3::Identity));
		OriginalChildren.Emplace(new TImplicitObjectTransformed<FReal, 3>(OriginalSpheres[0], FRigidTransform3::Identity));

		TRefCountPtr<TImplicitObjectTransformed<FReal, 3>> Root(new TImplicitObjectTransformed<FReal, 3>(OriginalChildren[1], FRigidTransform3::Identity));

		TArray<uint8> Data;
		{
			FMemoryWriter Ar(Data);
			FChaosArchive Writer(Ar);

			Writer << OriginalSpheres;
			Writer << OriginalChildren;
			Writer << Root;
		}

		{
			FMemoryReader Ar(Data);
			FChaosArchive Reader(Ar);

			TArray <FSpherePtr> SerializedSpheres;
			TArray<TRefCountPtr<TImplicitObjectTransformed<FReal, 3>>> SerializedChildren;
			TRefCountPtr<TImplicitObjectTransformed<FReal, 3>> SerializedRoot;

			Reader << SerializedSpheres;
			Reader << SerializedChildren;
			Reader << SerializedRoot;

			EXPECT_EQ(SerializedSpheres.Num(), OriginalSpheres.Num());
			EXPECT_EQ(SerializedChildren.Num(), OriginalChildren.Num());

			EXPECT_EQ(SerializedRoot->GetTransformedObject(), SerializedChildren[1].GetReference());
			EXPECT_EQ(SerializedChildren[0]->GetTransformedObject(), SerializedSpheres[0].GetReference());
			EXPECT_EQ(SerializedChildren[1]->GetTransformedObject(), SerializedSpheres[1].GetReference());
			EXPECT_EQ(SerializedChildren[2]->GetTransformedObject(), SerializedSpheres[0].GetReference());
		}
	}

	void ObjectUnionSerialization()
	{
		TArray<Chaos::FImplicitObjectPtr> OriginalSpheres;
		OriginalSpheres.Emplace(new Chaos::FSphere(FVec3(1, 2, 3), 1));
		OriginalSpheres.Emplace(new Chaos::FSphere(FVec3(1, 2, 3), 2));

		TArray<Chaos::FImplicitObjectPtr> OriginalChildren;
		OriginalChildren.Emplace(new TImplicitObjectTransformed<FReal, 3>(OriginalSpheres[0], FRigidTransform3::Identity));
		OriginalChildren.Emplace(new TImplicitObjectTransformed<FReal, 3>(OriginalSpheres[1], FRigidTransform3::Identity));
		OriginalChildren.Emplace(new TImplicitObjectTransformed<FReal, 3>(OriginalSpheres[0], FRigidTransform3::Identity));

		FImplicitObjectUnionPtr Root(new FImplicitObjectUnion(MoveTemp(OriginalChildren)));

		TArray<uint8> Data;
		{
			FMemoryWriter Ar(Data);
			FChaosArchive Writer(Ar);

			Writer << Root;
			Writer << OriginalSpheres;
			Writer << OriginalChildren;
		}

		{
			FMemoryReader Ar(Data);
			FChaosArchive Reader(Ar);

			TArray <FSpherePtr> SerializedSpheres;
			TArray<TRefCountPtr<TImplicitObjectTransformed<FReal, 3>>> SerializedChildren;
			FImplicitObjectUnionPtr SerializedRoot;

			Reader << SerializedRoot;
			Reader << SerializedSpheres;
			Reader << SerializedChildren;

			EXPECT_EQ(SerializedSpheres.Num(), OriginalSpheres.Num());
			EXPECT_EQ(SerializedChildren.Num(), OriginalChildren.Num());
			EXPECT_EQ(SerializedChildren.Num(), 0);	//We did a move and then serialized, should be empty

			const TArray<Chaos::FImplicitObjectPtr>& UnionObjs = SerializedRoot->GetObjects();
			TImplicitObjectTransformed<FReal, 3>* FirstChild = static_cast<TImplicitObjectTransformed<FReal, 3>*>(UnionObjs[0].GetReference());
			TImplicitObjectTransformed<FReal, 3>* SecondChild = static_cast<TImplicitObjectTransformed<FReal, 3>*>(UnionObjs[1].GetReference());
			TImplicitObjectTransformed<FReal, 3>* ThirdChild = static_cast<TImplicitObjectTransformed<FReal, 3>*>(UnionObjs[2].GetReference());

			EXPECT_EQ(FirstChild->GetTransformedObject(), SerializedSpheres[0].GetReference());
			EXPECT_EQ(SecondChild->GetTransformedObject(), SerializedSpheres[1].GetReference());
			EXPECT_EQ(ThirdChild->GetTransformedObject(), SerializedSpheres[0].GetReference());
			EXPECT_TRUE(FirstChild != ThirdChild);	//First and third point to same sphere, but still unique children
		}
	}

	void ParticleSerialization()
	{
		TArray<Chaos::FImplicitObjectPtr> OriginalSpheres;
		OriginalSpheres.Emplace(new Chaos::FSphere(FVec3(1, 2, 3), 1));
		OriginalSpheres.Emplace(new Chaos::FSphere(FVec3(1, 2, 3), 2));

		{
			FGeometryParticles OriginalParticles;
			OriginalParticles.AddParticles(2);
			OriginalParticles.SetX(0, FVec3::ZeroVector);
			OriginalParticles.SetX(1, FVec3::ZeroVector);
			OriginalParticles.SetR(0, FRotation3::Identity);
			OriginalParticles.SetR(1, FRotation3::Identity);
			OriginalParticles.SetGeometry(0, OriginalSpheres[0]);
			OriginalParticles.SetGeometry(1, OriginalSpheres[1]);

			TArray<uint8> Data;
			{
				FMemoryWriter Ar(Data);
				FChaosArchive Writer(Ar);

				Writer << OriginalParticles;
				Writer << OriginalSpheres;
			}

			{
				FMemoryReader Ar(Data);
				FChaosArchive Reader(Ar);

				TArray <FSpherePtr> SerializedSpheres;
				FGeometryParticles SerializedParticles;

				Reader << SerializedParticles;
				Reader << SerializedSpheres;

				EXPECT_EQ(SerializedSpheres.Num(), OriginalSpheres.Num());
				EXPECT_EQ(SerializedParticles.Size(), OriginalParticles.Size());

				EXPECT_EQ(SerializedParticles.GetGeometry(0).GetReference(), SerializedSpheres[0].GetReference());
				EXPECT_EQ(SerializedParticles.GetGeometry(1).GetReference(), SerializedSpheres[1].GetReference());
			}
		}

		//ptr
		{
			auto OriginalParticles = MakeUnique<FGeometryParticles>();
			OriginalParticles->AddParticles(2);
			OriginalParticles->SetX(0, FVec3::ZeroVector);
			OriginalParticles->SetX(1, FVec3::ZeroVector);
			OriginalParticles->SetR(0, FRotation3::Identity);
			OriginalParticles->SetR(1, FRotation3::Identity);
			OriginalParticles->SetGeometry(0, OriginalSpheres[0]);
			OriginalParticles->SetGeometry(1, OriginalSpheres[1]);

			TArray<uint8> Data;
			{
				FMemoryWriter Ar(Data);
				FChaosArchive Writer(Ar);

				Writer << OriginalParticles;
				Writer << OriginalSpheres;
			}

			{
				FMemoryReader Ar(Data);
				FChaosArchive Reader(Ar);

				TArray <FSpherePtr> SerializedSpheres;
				TUniquePtr<FGeometryParticles> SerializedParticles;

				Reader << SerializedParticles;
				Reader << SerializedSpheres;

				EXPECT_EQ(SerializedSpheres.Num(), OriginalSpheres.Num());
				EXPECT_EQ(SerializedParticles->Size(), OriginalParticles->Size());

				EXPECT_EQ(SerializedParticles->GetGeometry(0).GetReference(), SerializedSpheres[0].GetReference());
				EXPECT_EQ(SerializedParticles->GetGeometry(1).GetReference(), SerializedSpheres[1].GetReference());
			}
		}
	}

	void BVHSerialization()
	{
		TArray<uint8> Data;
		{
			TArray<FImplicitObjectPtr> OriginalSpheres;
			OriginalSpheres.Emplace(new Chaos::FSphere(FVec3(0, 0, 0), 1));
			OriginalSpheres.Emplace(new Chaos::FSphere(FVec3(0, 0, 0), 2));

			FGeometryParticles OriginalParticles;
			OriginalParticles.AddParticles(2);
			OriginalParticles.SetX(0, FVec3(100, 1, 2));
			OriginalParticles.SetX(1, FVec3(0, 1, 2));
			OriginalParticles.SetR(0, FRotation3::Identity);
			OriginalParticles.SetR(1, FRotation3::Identity);
			OriginalParticles.SetGeometry(0, OriginalSpheres[0]);
			OriginalParticles.SetGeometry(1, OriginalSpheres[1]);
			
			TBoundingVolumeHierarchy<FGeometryParticles, TArray<int32>> OriginalBVH(OriginalParticles);

			FMemoryWriter Ar(Data);
			FChaosArchive Writer(Ar);

			Writer << OriginalBVH;
			Writer << OriginalSpheres;
			Writer << OriginalParticles;
		}

		{
			TArray <FSpherePtr> SerializedSpheres;
			FGeometryParticles SerializedParticles;
			TBoundingVolumeHierarchy<FGeometryParticles, TArray<int32>> SerializedBVH(SerializedParticles);
			FMemoryReader Ar(Data);
			FChaosArchive Reader(Ar);

			Reader << SerializedBVH;
			Reader << SerializedSpheres;
			Reader << SerializedParticles;

			const FAABB3 QueryBox{ {-1,0,0}, {1,10,20} };
			const TArray<int32>& PotentialIntersections = SerializedBVH.FindAllIntersections(QueryBox);
			TArray<int32> FinalIntersections;
			for (int32 Potential : PotentialIntersections)
			{
				FRigidTransform3 TM(SerializedParticles.GetX(Potential), SerializedParticles.GetR(Potential));
				const FAABB3 Bounds = SerializedParticles.GetGeometry(Potential)->BoundingBox().TransformedAABB(TM);
				if (Bounds.Intersects(QueryBox))
				{
					FinalIntersections.Add(Potential);
				}
			}

			EXPECT_EQ(FinalIntersections.Num(), 1);
			EXPECT_EQ(FinalIntersections[0], 1);
		}
	}

	void RigidParticlesSerialization()
	{
		TArray<FVec3> F;
		F.Emplace(FVec3(1, 2, 3));
		F.Emplace(FVec3(3, 2, 1));

		TArray<FVec3> X;
		X.Emplace(FVec3(0, 2, 1));
		X.Emplace(FVec3(100, 15, 0));

		TRigidParticles<FReal, 3> Particles;
		Particles.AddParticles(2);
		Particles.SetR(0, FRotation3::Identity);
		Particles.SetR(1, FRotation3::Identity);

		Particles.Acceleration(0) = F[0];
		Particles.Acceleration(1) = F[1];
		Particles.SetX(0, X[0]);
		Particles.SetX(1, X[1]);
		Particles.RotationOfMass(0) = FRotation3::FromIdentity();
		Particles.RotationOfMass(1) = FRotation3::FromIdentity();

		TCHAR const * BinaryFolderName = TEXT("RigidParticles");
		bool bSaveBinaryToDisk = false; // Flip to true and run to save current binary to disk for future tests.
		TArray<TRigidParticles<FReal, 3>> ObjectsToTest;
		bool bResult = SaveLoadUtility<FReal, TRigidParticles<FReal, 3>>(Particles, *GetSerializedBinaryPath(), BinaryFolderName, bSaveBinaryToDisk, ObjectsToTest);
		EXPECT_TRUE(bResult);

		for (TRigidParticles<FReal, 3> const &TestParticles : ObjectsToTest)
		{
			EXPECT_EQ(TestParticles.Size(), Particles.Size());
			EXPECT_EQ(TestParticles.Acceleration(0), Particles.Acceleration(0));
			EXPECT_EQ(TestParticles.Acceleration(1), Particles.Acceleration(1));
			EXPECT_EQ(TestParticles.GetX(0), Particles.GetX(0));
			EXPECT_EQ(TestParticles.GetX(1), Particles.GetX(1));
		}
	}

	void BVHParticlesSerialization()
	{
		TArray<uint8> Data;
		TArray<FImplicitObjectPtr> Spheres;
		Spheres.Emplace(new Chaos::FSphere(FVec3(0, 0, 0), 1));
		Spheres.Emplace(new Chaos::FSphere(FVec3(0, 0, 0), 1));
		Spheres.Emplace(new Chaos::FSphere(FVec3(0, 0, 0), 1));

		FGeometryParticles Particles;
		Particles.AddParticles(3);
		Particles.SetR(0, FRotation3::Identity);
		Particles.SetR(1, FRotation3::Identity);
		Particles.SetX(0, FVec3(15, 1, 2));
		Particles.SetX(1, FVec3(0, 2, 2));
		Particles.SetX(2, FVec3(0, 2, 2));
		Particles.SetR(0, FRotation3::Identity);
		Particles.SetR(1, FRotation3::Identity);
		Particles.SetR(2, FRotation3::Identity);
		Particles.SetGeometry(0, Spheres[0]);
		Particles.SetGeometry(1, Spheres[1]);
		Particles.SetGeometry(2, Spheres[2]);

		FBVHParticles BVHParticles(MoveTemp(Particles));

		TCHAR const *BinaryFolderName = TEXT("BVHParticles");
		bool bSaveBinaryToDisk = false; // Flip to true and run to save current binary to disk for future tests.
		TArray<FBVHParticles> ObjectsToTest;
		bool bResult = SaveLoadUtility<FReal, FBVHParticles>(BVHParticles, *GetSerializedBinaryPath(), BinaryFolderName, bSaveBinaryToDisk, ObjectsToTest);
		EXPECT_TRUE(bResult);

		for (FBVHParticles const &TestBVHP: ObjectsToTest)
		{
			const FAABB3 Box{ {-1,-1,-1}, {1,3,3} };
			TArray<int32> PotentialIntersections = BVHParticles.FindAllIntersections(Box);

			EXPECT_EQ(TestBVHP.Size(), BVHParticles.Size());
			EXPECT_EQ(PotentialIntersections.Num(), 2);
			EXPECT_EQ(PotentialIntersections[0], 1);
			EXPECT_EQ(PotentialIntersections[1], 2);
		}
	}

	void EvolutionPerfHelper(const FString& FilePath)
	{
		CHAOS_PERF_TEST(EvolutionPerf, EChaosPerfUnits::Us);

		for (int i = 0; i < 1000; ++i)
		{
			TUniquePtr<FArchive> File(IFileManager::Get().CreateFileReader(*FilePath));
			if (File)
			{
				Chaos::FChaosArchive ChaosAr(*File);
				FParticleUniqueIndicesMultithreaded UniqueIndices;
				FPBDRigidsSOAs Particles(UniqueIndices);

				THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
				FPBDRigidsEvolution Evolution(Particles, PhysicalMaterials);

				Evolution.Serialize(ChaosAr);
				Evolution.AdvanceOneTimeStep(1 / 60.f);
				Evolution.EndFrame(1 / 60.0f);
			}
		}
	}

	void EvolutionPerfHarness()
	{
		//Load evolutions and step them over and over (with rewind) to measure perf of different components in the system
		//EvolutionPerfHelper(FPaths::EngineDir() / TEXT("Restricted/NotForLicensees/Source/Programs/HeadlessPhysicsSQ/Captures/ChaosEvolution_76.bin"));
	}

	void HeightFieldSerialization()
	{
		const int32 Cols = 10;
		const int32 Rows = 20;
		TArray<uint16> Heights;
		Heights.SetNum(Cols * Rows);
		for (int32 Row = 0; Row < Rows; ++Row)
		{
			for (int32 Col= 0; Col< Cols; ++Col)
			{
				int32 Index = (Row * Cols) + Col;
				Heights[Index] = Index; // set the Index as the height 
			}
		}

		TArray<uint8> MaterialIndices;
		MaterialIndices.SetNum(1);
		MaterialIndices[0] = 0;

		FHeightFieldPtr OriginalHeightField(new FHeightField(Heights, MaterialIndices, Rows, Cols, { (FReal)20000., (FReal)30000., (FReal)10000. }));

		TArray<uint8> Data;
		{
			FMemoryWriter Ar(Data);
			FChaosArchive Writer(Ar);

			Writer << OriginalHeightField;
		}

		{
			FMemoryReader Ar(Data);
			FChaosArchive Reader(Ar);
			FHeightFieldPtr SerializedHeightField;

			Reader << SerializedHeightField;

			const FHeightField::FDataType& OriginalGeomData = OriginalHeightField->GeomData;
			const FHeightField::FDataType& SerializedGeomData = SerializedHeightField->GeomData;

			EXPECT_EQ(SerializedGeomData.MinValue, OriginalGeomData.MinValue);
			EXPECT_EQ(SerializedGeomData.MaxValue, OriginalGeomData.MaxValue);
			EXPECT_EQ(SerializedGeomData.Scale, OriginalGeomData.Scale);
			EXPECT_EQ(SerializedGeomData.NumRows, OriginalGeomData.NumRows);
			EXPECT_EQ(SerializedGeomData.NumCols, OriginalGeomData.NumCols);
#if 0 
			EXPECT_EQ(SerializedGeomData.Range, OriginalGeomData.Range);
			EXPECT_EQ(SerializedGeomData.HeightPerUnit, OriginalGeomData.HeightPerUnit);
#else
			// LWC-TODO : this is required for now as LWC mode serialize in floats causing some slight difference when reading back 
			EXPECT_TRUE(FMath::Abs(SerializedGeomData.Range - OriginalGeomData.Range) < SMALL_NUMBER);
			EXPECT_TRUE(FMath::Abs(SerializedGeomData.HeightPerUnit - OriginalGeomData.HeightPerUnit) < SMALL_NUMBER);
#endif
			EXPECT_EQ(SerializedGeomData.Heights.Num(), OriginalGeomData.Heights.Num());
			EXPECT_EQ(SerializedGeomData.MaterialIndices.Num(), OriginalGeomData.MaterialIndices.Num());

			for (int32 i = 0; i < SerializedGeomData.Heights.Num(); ++i)
			{
				EXPECT_EQ(SerializedGeomData.Heights[i], OriginalGeomData.Heights[i]);
			}
			

			for (int32 i = 0; i < SerializedGeomData.MaterialIndices.Num(); ++i)
			{
				EXPECT_EQ(SerializedGeomData.MaterialIndices[i], OriginalGeomData.MaterialIndices[i]);
			}
		}
	}

}