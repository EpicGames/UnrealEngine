// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestCollisions.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosCollisionConstraints.h"
#include "Chaos/Box.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Collision/PBDCollisionContainerSolver.h"
#include "Chaos/Convex.h"
#include "Chaos/Sphere.h"
#include "Chaos/GJK.h"
#include "Chaos/Pair.h"
#include "Chaos/PhysicalMaterials.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/Utilities.h"
#include "Modules/ModuleManager.h"


namespace ChaosTest {

	using namespace Chaos;

	// Two boxes that use a margin around a core AABB.
	// Test that collision detection treats the margin as part of the shape.
	void TestBoxBoxCollisionMargin(
		const FReal Margin0,
		const FReal Margin1,
		const FVec3& Size,
		const FVec3& Delta,
		const FReal ExpectedPhi,
		const FVec3& ExpectedNormal)
	{
		TArrayCollectionArray<bool> Collided;
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->Friction = 0;
		PhysicsMaterial->Restitution = 0;
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;

		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		Particles.GetParticleHandles().AddArray(&Collided);
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);

		auto Box0 = AppendDynamicParticleBoxMargin(Particles, Size, Margin0);
		Box0->SetX(FVec3(0, 0, 0));
		Box0->SetR(FRotation3(FQuat::Identity));
		Box0->SetV(FVec3(0));
		Box0->SetPreV(Box0->GetV());
		Box0->SetP(Box0->GetX());
		Box0->SetQ(Box0->GetR());
		Box0->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);

		auto Box1 = AppendDynamicParticleBoxMargin(Particles, Size, Margin1);
		Box1->SetX(Delta);
		Box1->SetR(FRotation3(FQuat::Identity));
		Box1->SetV(FVec3(0));
		Box1->SetPreV(Box1->GetV());
		Box1->SetP(Box1->GetX());
		Box1->SetQ(Box1->GetR());
		Box1->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);

		const FImplicitBox3* BoxImplicit0 = Box0->GetGeometry()->template GetObject<FImplicitBox3>();
		const FImplicitBox3* BoxImplicit1 = Box1->GetGeometry()->template GetObject<FImplicitBox3>();

		const FReal Tolerance = 2.0f * KINDA_SMALL_NUMBER;

		// Boxes should have a margin, limited by the minimum box dimension
		float ExpectedMargin0 = BoxImplicit0->ClampedMargin(Margin0);
		float ExpectedMargin1 = BoxImplicit0->ClampedMargin(Margin1);
		EXPECT_NEAR(BoxImplicit0->GetMarginf(), ExpectedMargin0, Tolerance);
		EXPECT_NEAR(BoxImplicit1->GetMarginf(), ExpectedMargin1, Tolerance);

		// Box Bounds should include margin
		const FAABB3 BoxBounds0 = BoxImplicit0->BoundingBox();
		const FAABB3 BoxBounds1 = BoxImplicit1->BoundingBox();
		EXPECT_NEAR(BoxBounds0.Extents().X, Size.X, Tolerance);
		EXPECT_NEAR(BoxBounds0.Extents().Y, Size.Y, Tolerance);
		EXPECT_NEAR(BoxBounds0.Extents().Z, Size.Z, Tolerance);
		EXPECT_NEAR(BoxBounds1.Extents().X, Size.X, Tolerance);
		EXPECT_NEAR(BoxBounds1.Extents().Y, Size.Y, Tolerance);
		EXPECT_NEAR(BoxBounds1.Extents().Z, Size.Z, Tolerance);

		Private::FCollisionConstraintAllocator CollisionAllocator;
		CollisionAllocator.SetMaxContexts(1);

		FPBDCollisionConstraintPtr Constraint = CollisionAllocator.GetContextAllocator(0)->CreateConstraint(
			Box0,
			Box0->GetGeometry(),
			Box0->ShapesArray()[0].Get(),
			nullptr,
			FRigidTransform3(),
			Box1,
			Box1->GetGeometry(),
			Box1->ShapesArray()[0].Get(),
			nullptr,
			FRigidTransform3(),
			FLT_MAX,
			true,
			EContactShapesType::BoxBox);

		FRigidTransform3 ShapeWorldTransform0 = Constraint->GetShapeRelativeTransform0() * Collisions::GetTransform(Constraint->GetParticle0());
		FRigidTransform3 ShapeWorldTransform1 = Constraint->GetShapeRelativeTransform1() * Collisions::GetTransform(Constraint->GetParticle1());
		Constraint->SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);

		// Detect collisions
		Constraint->ResetPhi(Constraint->GetCullDistance());
		Collisions::UpdateConstraint(*Constraint, ShapeWorldTransform0, ShapeWorldTransform1, 1 / 30.0f);

		EXPECT_NEAR(Constraint->GetPhi(), ExpectedPhi, Tolerance);
		EXPECT_NEAR(Constraint->CalculateWorldContactNormal().X, ExpectedNormal.X, Tolerance);
		EXPECT_NEAR(Constraint->CalculateWorldContactNormal().Y, ExpectedNormal.Y, Tolerance);
		EXPECT_NEAR(Constraint->CalculateWorldContactNormal().Z, ExpectedNormal.Z, Tolerance);
	}

	TEST(CollisionTests, TestBoxBoxCollisionMargin)
	{
		// Zero-phi tests
		TestBoxBoxCollisionMargin(0, 0, FVec3(20, 100, 50), FVec3(0, -100, 0), 0.0f, FVec3(0, 1, 0));
		TestBoxBoxCollisionMargin(1, 1, FVec3(20, 100, 50), FVec3(0, -100, 0), 0.0f, FVec3(0, 1, 0));
		TestBoxBoxCollisionMargin(5, 10, FVec3(20, 100, 50), FVec3(0, -100, 0), 0.0f, FVec3(0, 1, 0));
		TestBoxBoxCollisionMargin(10, 5, FVec3(20, 100, 50), FVec3(0, -100, 0), 0.0f, FVec3(0, 1, 0));

		// Positive-phi test
		TestBoxBoxCollisionMargin(0, 0, FVec3(20, 100, 50), FVec3(0, -110, 0), 10.0f, FVec3(0, 1, 0));
		TestBoxBoxCollisionMargin(1, 1, FVec3(20, 100, 50), FVec3(0, -110, 0), 10.0f, FVec3(0, 1, 0));
		TestBoxBoxCollisionMargin(5, 10, FVec3(20, 100, 50), FVec3(0, -110, 0), 10.0f, FVec3(0, 1, 0));
		TestBoxBoxCollisionMargin(10, 5, FVec3(20, 100, 50), FVec3(0, -110, 0), 10.0f, FVec3(0, 1, 0));

		// Negative-phi test
		TestBoxBoxCollisionMargin(0, 0, FVec3(20, 100, 50), FVec3(0, -90, 0), -10.0f, FVec3(0, 1, 0));
		TestBoxBoxCollisionMargin(1, 1, FVec3(20, 100, 50), FVec3(0, -90, 0), -10.0f, FVec3(0, 1, 0));
		TestBoxBoxCollisionMargin(5, 10, FVec3(20, 100, 50), FVec3(0, -90, 0), -10.0f, FVec3(0, 1, 0));
		TestBoxBoxCollisionMargin(10, 5, FVec3(20, 100, 50), FVec3(0, -90, 0), -10.0f, FVec3(0, 1, 0));

		// If the specified margin is too large the margin will get reduced and it should all still work
		TestBoxBoxCollisionMargin(15, 15, FVec3(20, 100, 100), FVec3(0, -100, 0), 0.0f, FVec3(0, 1, 0));
		TestBoxBoxCollisionMargin(15, 15, FVec3(20, 100, 100), FVec3(20, 0, 0), 0.0f, FVec3(-1, 0, 0));
	}

	// Two boxes that use a margin around a core AABB.
	// Test that collision detection treats the margin as part of the shape.
	void TestConvexConvexCollisionMargin(
		const FReal Margin0,
		const FReal Margin1,
		const FVec3& Size,
		const FVec3& Delta,
		const FReal ExpectedPhi,
		const FVec3& ExpectedNormal)
	{
		TArrayCollectionArray<bool> Collided;
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->Friction = 0;
		PhysicsMaterial->Restitution = 0;
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;

		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		Particles.GetParticleHandles().AddArray(&Collided);
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);

		auto Box0 = AppendDynamicParticleConvexBoxMargin(Particles, 0.5f * Size, Margin0);
		Box0->SetX(FVec3(0, 0, 0));
		Box0->SetR(FRotation3(FQuat::Identity));
		Box0->SetV(FVec3(0));
		Box0->SetPreV(Box0->GetV());
		Box0->SetP(Box0->GetX());
		Box0->SetQ(Box0->GetR());
		Box0->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);

		auto Box1 = AppendDynamicParticleConvexBoxMargin(Particles, 0.5f * Size, Margin1);
		Box1->SetX(Delta);
		Box1->SetR(FRotation3(FQuat::Identity));
		Box1->SetV(FVec3(0));
		Box1->SetPreV(Box1->GetV());
		Box1->SetP(Box1->GetX());
		Box1->SetQ(Box1->GetR());
		Box1->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);

		const FImplicitConvex3* ConvexImplicit0 = Box0->GetGeometry()->template GetObject<FImplicitConvex3>();
		const FImplicitConvex3* ConvexImplicit1 = Box1->GetGeometry()->template GetObject<FImplicitConvex3>();

		const FReal Tolerance = 2.0f * KINDA_SMALL_NUMBER;

		// Should have a margin
		EXPECT_NEAR(ConvexImplicit0->GetMargin(), Margin0, Tolerance);
		EXPECT_NEAR(ConvexImplicit1->GetMargin(), Margin1, Tolerance);

		// Bounds should include margin
		const FAABB3 BoxBounds0 = ConvexImplicit0->BoundingBox();
		const FAABB3 BoxBounds1 = ConvexImplicit1->BoundingBox();
		EXPECT_NEAR(BoxBounds0.Extents().X, Size.X, Tolerance);
		EXPECT_NEAR(BoxBounds0.Extents().Y, Size.Y, Tolerance);
		EXPECT_NEAR(BoxBounds0.Extents().Z, Size.Z, Tolerance);
		EXPECT_NEAR(BoxBounds1.Extents().X, Size.X, Tolerance);
		EXPECT_NEAR(BoxBounds1.Extents().Y, Size.Y, Tolerance);
		EXPECT_NEAR(BoxBounds1.Extents().Z, Size.Z, Tolerance);

		Private::FCollisionConstraintAllocator CollisionAllocator;
		CollisionAllocator.SetMaxContexts(1);

		FPBDCollisionConstraintPtr Constraint = CollisionAllocator.GetContextAllocator(0)->CreateConstraint(
			Box0,
			Box0->GetGeometry(),
			Box0->ShapesArray()[0].Get(),
			nullptr,
			FRigidTransform3(),
			Box1,
			Box1->GetGeometry(),
			Box1->ShapesArray()[0].Get(),
			nullptr,
			FRigidTransform3(),
			FLT_MAX,
			true,
			EContactShapesType::GenericConvexConvex);

		FRigidTransform3 ShapeWorldTransform0 = Constraint->GetShapeRelativeTransform0() * Collisions::GetTransform(Constraint->GetParticle0());
		FRigidTransform3 ShapeWorldTransform1 = Constraint->GetShapeRelativeTransform1() * Collisions::GetTransform(Constraint->GetParticle1());
		Constraint->SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);

		// Detect collisions
		Collisions::UpdateConstraint(*Constraint, ShapeWorldTransform0, ShapeWorldTransform1, 1 / 30.0f);

		EXPECT_NEAR(Constraint->GetPhi(), ExpectedPhi, Tolerance);
		EXPECT_NEAR(Constraint->CalculateWorldContactNormal().X, ExpectedNormal.X, Tolerance);
		EXPECT_NEAR(Constraint->CalculateWorldContactNormal().Y, ExpectedNormal.Y, Tolerance);
		EXPECT_NEAR(Constraint->CalculateWorldContactNormal().Z, ExpectedNormal.Z, Tolerance);
	}


	TEST(CollisionTests, TestConvexConvexCollisionMargin)
	{
		// Zero-phi tests
		TestConvexConvexCollisionMargin(0, 0, FVec3(20, 100, 50), FVec3(0, -100, 0), 0.0f, FVec3(0, 1, 0));
		TestConvexConvexCollisionMargin(1, 1, FVec3(20, 100, 50), FVec3(0, -100, 0), 0.0f, FVec3(0, 1, 0));
		TestConvexConvexCollisionMargin(5, 10, FVec3(20, 100, 50), FVec3(0, -100, 0), 0.0f, FVec3(0, 1, 0));
		TestConvexConvexCollisionMargin(10, 5, FVec3(20, 100, 50), FVec3(0, -100, 0), 0.0f, FVec3(0, 1, 0));

		// Positive-phi test
		TestConvexConvexCollisionMargin(0, 0, FVec3(20, 100, 50), FVec3(0, -110, 0), 10.0f, FVec3(0, 1, 0));
		TestConvexConvexCollisionMargin(1, 1, FVec3(20, 100, 50), FVec3(0, -110, 0), 10.0f, FVec3(0, 1, 0));
		TestConvexConvexCollisionMargin(5, 10, FVec3(20, 100, 50), FVec3(0, -110, 0), 10.0f, FVec3(0, 1, 0));
		TestConvexConvexCollisionMargin(10, 5, FVec3(20, 100, 50), FVec3(0, -110, 0), 10.0f, FVec3(0, 1, 0));

		// Negative-phi test
		TestConvexConvexCollisionMargin(0, 0, FVec3(20, 100, 50), FVec3(0, -90, 0), -10.0f, FVec3(0, 1, 0));
		TestConvexConvexCollisionMargin(1, 1, FVec3(20, 100, 50), FVec3(0, -90, 0), -10.0f, FVec3(0, 1, 0));
		TestConvexConvexCollisionMargin(5, 10, FVec3(20, 100, 50), FVec3(0, -90, 0), -10.0f, FVec3(0, 1, 0));
		TestConvexConvexCollisionMargin(10, 5, FVec3(20, 100, 50), FVec3(0, -90, 0), -10.0f, FVec3(0, 1, 0));
	}

	TEST(CollisionTests, DISABLED_TestConvexConvexCollisionMarginTooLarge)
	{
		// If the margin is too large, the margin should be limited
		// @todo(chaos): fix this for convex - we do not have margin limits implemented
		TestConvexConvexCollisionMargin(15, 15, FVec3(20, 100, 100), FVec3(0, -100, 0), 0.0f, FVec3(0, 1, 0));
		TestConvexConvexCollisionMargin(15, 15, FVec3(20, 100, 100), FVec3(20, 0, 0), 0.0f, FVec3(-1, 0, 0));
	}


	// Check that the margin does not impact the box raycast functions
	void TestBoxRayCastsMargin(
		const FReal Margin0,
		const FVec3& Size,
		const FVec3& StartPos,
		const FVec3& Dir,
		const FReal Length,
		const bool bExpectedHit,
		const FReal ExpectedTime,
		const FVec3& ExpectedPosition,
		const FVec3& ExpectedNormal)
	{
		TArrayCollectionArray<bool> Collided;
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->Friction = 0;
		PhysicsMaterial->Restitution = 0;
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;

		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		Particles.GetParticleHandles().AddArray(&Collided);
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		Particles.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);

		auto Box0 = AppendDynamicParticleBoxMargin(Particles, Size, Margin0);
		Box0->SetX(FVec3(0, 0, 0));
		Box0->SetR(FRotation3(FQuat::Identity));
		Box0->SetV(FVec3(0));
		Box0->SetPreV(Box0->GetV());
		Box0->SetP(Box0->GetX());
		Box0->SetQ(Box0->GetR());
		Box0->AuxilaryValue(PhysicsMaterials) = MakeSerializable(PhysicsMaterial);

		const FImplicitBox3* BoxImplicit0 = Box0->GetGeometry()->template GetObject<FImplicitBox3>();

		const FReal Tolerance = KINDA_SMALL_NUMBER;

		{
			FReal Time;
			FVec3 Position, Normal;
			int32 FaceIndex;
			bool bHit = BoxImplicit0->Raycast(StartPos, Dir, Length, 0.0f, Time, Position, Normal, FaceIndex);

			EXPECT_EQ(bHit, bExpectedHit);
			if (bHit)
			{
				EXPECT_NEAR(Time, ExpectedTime, Tolerance);
				EXPECT_NEAR(Position.X, ExpectedPosition.X, Tolerance);
				EXPECT_NEAR(Position.Y, ExpectedPosition.Y, Tolerance);
				EXPECT_NEAR(Position.Z, ExpectedPosition.Z, Tolerance);
				EXPECT_NEAR(Normal.X, ExpectedNormal.X, Tolerance);
				EXPECT_NEAR(Normal.Y, ExpectedNormal.Y, Tolerance);
				EXPECT_NEAR(Normal.Z, ExpectedNormal.Z, Tolerance);
			}
		}

		{
			FReal Time;
			FVec3 Position;

			bool bParallel[3];
			FVec3 InvDir;
			for (int Axis = 0; Axis < 3; ++Axis)
			{
				bParallel[Axis] = FMath::IsNearlyZero(Dir[Axis], (FReal)1.e-8);
				InvDir[Axis] = bParallel[Axis] ? 0 : 1 / Dir[Axis];
			}

			bool bHit = BoxImplicit0->RaycastFast(BoxImplicit0->Min(), BoxImplicit0->Max(), StartPos, Dir, InvDir, bParallel, Length, 1.0f / Length, Time, Position);

			EXPECT_EQ(bHit, bExpectedHit);
			if (bHit)
			{
				EXPECT_NEAR(Time, ExpectedTime, Tolerance);
				EXPECT_NEAR(Position.X, ExpectedPosition.X, Tolerance);
				EXPECT_NEAR(Position.Y, ExpectedPosition.Y, Tolerance);
				EXPECT_NEAR(Position.Z, ExpectedPosition.Z, Tolerance);
			}
		}
	}

	TEST(CollisionTests, TestBoxRayCastsMargin)
	{
		TestBoxRayCastsMargin(0, FVec3(100, 100, 100), FVec3(-200, 0, 0), FVec3(1, 0, 0), 500.0f, true, 150.0f, FVec3(-50, 0, 0), FVec3(-1, 0, 0));		// No Margin
		TestBoxRayCastsMargin(1, FVec3(100, 100, 100), FVec3(-200, 0, 0), FVec3(1, 0, 0), 500.0f, true, 150.0f, FVec3(-50, 0, 0), FVec3(-1, 0, 0));		// Small Margin
		TestBoxRayCastsMargin(50, FVec3(100, 100, 100), FVec3(-200, 0, 0), FVec3(1, 0, 0), 500.0f, true, 150.0f, FVec3(-50, 0, 0), FVec3(-1, 0, 0));	// Max margin
		TestBoxRayCastsMargin(70, FVec3(100, 100, 100), FVec3(-200, 0, 0), FVec3(1, 0, 0), 500.0f, true, 150.0f, FVec3(-50, 0, 0), FVec3(-1, 0, 0));	// Too much margin
	}


	TEST(ShapeInstanceTests, TestSingleMaterial)
	{
		{
		FImplicitObjectPtr CollidingCubeGeom = MakeImplicitObjectPtr<TBox<FReal, 3>>(-FVec3(100.0), FVec3(100.0));
		TUniquePtr<FShapeInstance> ShapeInstance1 = FShapeInstance::Make(0, CollidingCubeGeom);
		EXPECT_EQ(ShapeInstance1->NumMaterials(), 0);

		FMaterialHandle MaterialHandle = FPhysicalMaterialManager::Get().Create();
		FMaterialData MaterialData;
		MaterialData.Materials.Add(MaterialHandle);
		ShapeInstance1->SetMaterialData(MaterialData); 

		EXPECT_EQ(ShapeInstance1->NumMaterials(), 1);
		EXPECT_TRUE(ShapeInstance1->GetMaterial(0).InnerHandle.IsValid());
	}
		{
			THandleArray<FChaosPhysicsMaterial> SimMaterials;
			FChaosMaterialHandle MaterialInnerHandle = SimMaterials.Create();
			FMaterialHandle MaterialHandle = FPhysicalMaterialManager::Get().Create();
			MaterialHandle.InnerHandle = MaterialInnerHandle;

			FImplicitObjectPtr CollidingCubeGeom = MakeImplicitObjectPtr<TBox<FReal, 3>>(-FVec3(100.0), FVec3(100.0));
			TUniquePtr<FShapeInstance> ShapeInstance1 = FShapeInstance::Make(0, CollidingCubeGeom);
			EXPECT_EQ(ShapeInstance1->NumMaterialsInternal(&SimMaterials), 0);

			
			FMaterialData MaterialData;
			MaterialData.Materials.Add(MaterialHandle);
			ShapeInstance1->SetMaterialData(MaterialData);

			EXPECT_EQ(ShapeInstance1->NumMaterialsInternal(&SimMaterials), 1);
			EXPECT_TRUE(ShapeInstance1->GetMaterialInternal(0, &SimMaterials).InnerHandle.IsValid());
		}
	}

}