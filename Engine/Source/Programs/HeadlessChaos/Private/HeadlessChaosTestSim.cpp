// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/Box.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"
#include "Modules/ModuleManager.h"
#include "ChaosSolversModule.h"
#include "PBDRigidsSolver.h"
#include "GeometryCollection/GeometryCollectionTestFramework.h"

namespace ChaosTest {

	using namespace Chaos;

	GTEST_TEST(AllTraits, SimTests_SphereSphereSimTest_StaticBoundsChange)
	{
		// This test spawns a dynamic and a static, then moves the static around a few times after initialization.
		// The goal is to make sure that the bounds are updated correctly and the dynamic rests on top of the static
		// in its final position.

		Chaos::FImplicitObjectPtr Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));

		// Create solver #TODO make FFramework a little more general instead of mostly geometry collection focused
		GeometryCollectionTest::FFramework Framework;

		// Make a particle
		auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& Particle = Proxy->GetGameThreadAPI();
		Particle.SetGeometry(Sphere);
		Particle.SetX(FVec3(1000, 1000, 200));
		Particle.SetGravityEnabled(true);
		Framework.Solver->RegisterObject(Proxy);

		auto StaticProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
		auto& Static = StaticProxy->GetGameThreadAPI();
		Static.SetGeometry(Sphere);
		Static.SetX(FVec3(0, 0, 0));
		Framework.Solver->RegisterObject(StaticProxy);

		Static.SetX(FVec3(2000, 1000, 0));
		Static.SetX(FVec3(3000, 1000, 0));

		::ChaosTest::SetParticleSimDataToCollide({ Proxy->GetParticle_LowLevel(), StaticProxy->GetParticle_LowLevel() });

		for (int32 Iter = 0; Iter < 200; ++Iter)
		{
			Framework.Advance();

			if (Iter == 0)
			{
				Static.SetX(FVec3(1000, 1000, 0));
			}
		}

		EXPECT_NEAR(Particle.X().Z, 20, 1);
	}

	GTEST_TEST(AllEvolutions, SimTests_SphereSphereSimTest)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		auto Static = Evolution.CreateStaticParticles(1)[0];
		auto Dynamic = Evolution.CreateDynamicParticles(1)[0];

		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->SleepCounterThreshold = 2;

		Chaos::FImplicitObjectPtr Sphere(new Chaos::FSphere(FVec3(0, 0, 0), 50));
		Static->SetGeometry(Sphere);
		Dynamic->SetGeometry(Sphere);

		Evolution.SetPhysicsMaterial(Dynamic, MakeSerializable(PhysicsMaterial));

		Static->SetX(FVec3(10, 10, 10));
		Dynamic->SetX(FVec3(10, 10, 150));
		Dynamic->I() = TVec3<FRealSingle>(100000.0f);
		Dynamic->InvI() = TVec3<FRealSingle>(1.0f / 100000.0f);

		// The position of the static has changed and statics don't automatically update bounds, so update explicitly
		Static->UpdateWorldSpaceState(FRigidTransform3(Static->GetX(), Static->GetR()), FVec3(0));

		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		// IMPORTANT : this is required to make sure the particles internal representation will reflect the sim data
		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		const FReal Dt = 1 / 60.f;
		for (int i = 0; i < 200; ++i)
		{
			Evolution.AdvanceOneTimeStep(1 / 60.f);
			Evolution.EndFrame(Dt);
		}

		EXPECT_NEAR(Dynamic->GetX().Z, 110, 1);
	}

	GTEST_TEST(AllEvolutions, SimTests_BoxBoxSimTest)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		auto Static = Evolution.CreateStaticParticles(1)[0];
		auto Dynamic = Evolution.CreateDynamicParticles(1)[0];

		Chaos::FImplicitObjectPtr StaticBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Chaos::FImplicitObjectPtr DynamicBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Static->SetGeometry(StaticBox);
		Dynamic->SetGeometry(DynamicBox);

		Static->SetX(FVec3(10, 10, 10));
		Static->UpdateWorldSpaceState(FRigidTransform3(Static->GetX(), Static->GetR()), FVec3(0));
		Dynamic->SetX(FVec3(10, 10, 120));
		Dynamic->I() = TVec3<FRealSingle>(100000.0f);
		Dynamic->InvI() = TVec3<FRealSingle>(1.0f / 100000.0f);

		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		// IMPORTANT : this is required to make sure the particles internal representation will reflect the sim data
		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		const FReal Dt = 1 / 60.f;
		for (int i = 0; i < 100; ++i)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		EXPECT_NEAR(Dynamic->GetX().Z, 110, 5);
	}

	// This test will fail because the inertia of the dynamic box is very low. The mass and inertia are both 1.0, 
	// but the box is 100x100x100. When we detect collisions, we get points around the edge of the box. The impulse
	// required to stop the velocity at that point is tiny because a tiny impulse can impart a large angular velocity
	// at that position. Therefore we would need a very large number of iterations to resolve it.
	// 
	// This will be fixed if/when we have a multi-contact manifold between particle pairs and we simultaneously
	// resolve contacts in that manifold.
	//
	GTEST_TEST(AllEvolutions, DISABLED_SimTests_VeryLowInertiaSimTest)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		auto Static = Evolution.CreateStaticParticles(1)[0];
		auto Dynamic = Evolution.CreateDynamicParticles(1)[0];

		Chaos::FImplicitObjectPtr StaticBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Chaos::FImplicitObjectPtr DynamicBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Static->SetGeometry(StaticBox);
		Dynamic->SetGeometry(DynamicBox);

		Static->SetX(FVec3(10, 10, 10));
		Static->UpdateWorldSpaceState(FRigidTransform3(Static->GetX(), Static->GetR()), FVec3(0));
		Dynamic->SetX(FVec3(10, 10, 300));
		Dynamic->I() = TVec3<FRealSingle>(1);
		Dynamic->InvI() = TVec3<FRealSingle>(1);

		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		for (int i = 0; i < 100; ++i)
		{
			Evolution.AdvanceOneTimeStep(1 / 60.f);
			Evolution.EndFrame(1 / 60.f);
		}

		EXPECT_NEAR(Dynamic->GetX().Z, 110, 10);
	}

	GTEST_TEST(AllEvolutions, SimTests_SleepAndWakeSimTest)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		auto Static = Evolution.CreateStaticParticles(1)[0];
		auto Dynamic1 = Evolution.CreateDynamicParticles(1)[0];
		auto Dynamic2 = Evolution.CreateDynamicParticles(1)[0];

		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;

		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);

		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->SleepingLinearThreshold = 20;
		PhysicsMaterial->SleepingAngularThreshold = 20;
		PhysicsMaterial->SleepCounterThreshold = 5;

		Chaos::FImplicitObjectPtr StaticBox(new TBox<FReal, 3>(FVec3(-500, -500, -50), FVec3(500, 500, 50)));
		Chaos::FImplicitObjectPtr DynamicBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Static->SetGeometry(StaticBox);
		Dynamic1->SetGeometry(DynamicBox);
		Dynamic2->SetGeometry(DynamicBox);

		Static->SetX(FVec3(10, 10, 10));
		Static->UpdateWorldSpaceState(FRigidTransform3(Static->GetX(), Static->GetR()), FVec3(0));
		Dynamic1->SetX(FVec3(10, 10, 120));
		Dynamic2->SetX(FVec3(10, 10, 400));

		Evolution.SetPhysicsMaterial(Dynamic1, MakeSerializable(PhysicsMaterial));
		Evolution.SetPhysicsMaterial(Dynamic2, MakeSerializable(PhysicsMaterial));

		Dynamic1->I() = TVec3<FRealSingle>(100000.0f);
		Dynamic1->InvI() = TVec3<FRealSingle>(1.0f / 100000.0f);
		Dynamic2->I() = TVec3<FRealSingle>(100000.0f);
		Dynamic2->InvI() = TVec3<FRealSingle>(1.0f / 100000.0f);

		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic1,Dynamic2 });

		// IMPORTANT : this is required to make sure the particles internal representation will reflect the sim data
		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic1);
		Evolution.EnableParticle(Dynamic2);

		bool Dynamic1WentToSleep = false;
		bool Dynamic1HasWokeAgain = false;
		for (int i = 0; i < 1000; ++i)
		{
			Evolution.AdvanceOneTimeStep(1 / 60.f);
			Evolution.EndFrame(1 / 60.f);

			// at some point Dynamic1 should come to rest and go to sleep on static particle
			if (Dynamic1WentToSleep == false && Dynamic1->ObjectState() == EObjectStateType::Sleeping)
			{
				Dynamic1WentToSleep = true;

				EXPECT_LT(Dynamic1->GetX().Z, 120);
				EXPECT_GT(Dynamic1->GetX().Z, 100);
			}

			// later the Dynamic2 collides with Dynamic1 waking it up again
			if (Dynamic1WentToSleep)
			{
				if (Dynamic1->ObjectState() == EObjectStateType::Dynamic)
				{
					Dynamic1HasWokeAgain = true;
				}
			}
		}

		EXPECT_TRUE(Dynamic1WentToSleep);
		EXPECT_TRUE(Dynamic1HasWokeAgain);
	}

	GTEST_TEST(AllTraits, DISABLED_SimTests_MidSubstepSleep)
	{
		Chaos::FImplicitObjectPtr Sphere{new Chaos::FSphere(FVec3(0), 10)};

		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		auto Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);

		FSingleParticlePhysicsProxy* Proxy = FSingleParticlePhysicsProxy::Create(Chaos::TPBDRigidParticle<FReal, 3>::CreateParticle());
		Chaos::FRigidBodyHandle_External& Particle = Proxy->GetGameThreadAPI();
		Particle.SetGeometry(Sphere);
		Particle.SetV(FVec3(0, 0, -1));

		Solver->RegisterObject(Proxy);

		Solver->SetMaxSubSteps_External(4);
		Solver->SetMaxDeltaTime_External(1.0f / 60.0f);
		Solver->DisableAsyncMode();

		const FVector InitialX = Proxy->GetGameThreadAPI().X();

		struct FCallback : TSimCallbackObject<>
		{
			virtual void OnPreSimulate_Internal() override
			{
				check(Proxy);
				PTX = Proxy->GetPhysicsThreadAPI()->X();
				if(Hits == 1)
				{
					Proxy->GetPhysicsThreadAPI()->SetObjectState(EObjectStateType::Sleeping);
				}
				++Hits;
			}

			FSingleParticlePhysicsProxy* Proxy = nullptr;
			FVector PTX = FVector::ZeroVector;
			int32 Hits = 0;
		};

		FCallback* Callback = Solver->CreateAndRegisterSimCallbackObject_External<FCallback>();
		Callback->Proxy = Proxy;

		// This should ensure 4 steps take place.
		Solver->AdvanceAndDispatch_External(1.0f);
		Solver->UpdateGameThreadStructures();

		EXPECT_EQ(Callback->Hits, 4);

		const FVector GTX = Proxy->GetGameThreadAPI().X();

		EXPECT_NE(GTX, InitialX);
		EXPECT_EQ(GTX, Callback->PTX);

		Solver->UnregisterAndFreeSimCallbackObject_External(Callback);

		Module->DestroySolver(Solver);
	}

	// Test that isolated stationary particles without gravity sleep according to their sleep settings.
	GTEST_TEST(AllEvolutions, SimTests_SleepAndWakeSimTest3)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		// This lambda assumes two freshly awakened dynamics, ticked over 100 frames.
		// The sleepy one has a material sleep type and will fall asleep after 5 frames.
		const auto AdvanceSleepStates = [&](auto& SleepyDynamic, auto& AwakeDynamic)
		{
			for (int i = 0; i < 10; ++i)
			{
				Evolution.AdvanceOneTimeStep(1 / 60.f);
				Evolution.EndFrame(1 / 60.f);

				// Dynamic1 should fall asleep after 5 frames of sitting still
				if (i < 5)
				{
					EXPECT_EQ(SleepyDynamic->ObjectState(), EObjectStateType::Dynamic);
				}
				else
				{
					EXPECT_EQ(SleepyDynamic->ObjectState(), EObjectStateType::Sleeping);
				}

				// Dynamic2 should never fall asleep
				EXPECT_EQ(AwakeDynamic->ObjectState(), EObjectStateType::Dynamic);
			}
		};

		auto Dynamic1 = Evolution.CreateDynamicParticles(1)[0];
		auto Dynamic2 = Evolution.CreateDynamicParticles(1)[0];
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		Particles.GetParticleHandles().AddArray(&PhysicsMaterials);
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->SleepingLinearThreshold = 20;
		PhysicsMaterial->SleepingAngularThreshold = 20;
		PhysicsMaterial->SleepCounterThreshold = 5;

		Dynamic1->SetX(FVec3(-200, 0, 0));
		Dynamic2->SetX(FVec3(200, 0, 0));

		Chaos::FImplicitObjectPtr DynamicBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Dynamic1->SetGeometry(DynamicBox);
		Dynamic2->SetGeometry(DynamicBox);

		Dynamic1->SetGravityEnabled(false);
		Dynamic2->SetGravityEnabled(false);

		Dynamic1->SetSleepType(ESleepType::MaterialSleep);
		Dynamic2->SetSleepType(ESleepType::NeverSleep);

		Evolution.SetPhysicsMaterial(Dynamic1, MakeSerializable(PhysicsMaterial));
		Evolution.SetPhysicsMaterial(Dynamic2, MakeSerializable(PhysicsMaterial));

		// IMPORTANT : this is required to make sure the particles internal representation will reflect the sim data
		Evolution.EnableParticle(Dynamic1);
		Evolution.EnableParticle(Dynamic2);

		const auto& SleepData = Evolution.GetParticles().GetDynamicParticles().GetSleepData();
		EXPECT_EQ(SleepData.Num(), 0);

		AdvanceSleepStates(Dynamic1, Dynamic2);

		// Particle 1 should have fallen asleep
		EXPECT_EQ(SleepData.Num(), 1);
		if (SleepData.Num() >= 1)
		{
			EXPECT_EQ(SleepData[0].Particle, Dynamic1);
			EXPECT_TRUE(SleepData[0].Sleeping);
		}

		// Switch the sleep types and observe state changes and sleep events
		Dynamic1->SetSleepType(ESleepType::NeverSleep);
		Dynamic2->SetSleepType(ESleepType::MaterialSleep);

		// Particle 1 should have woken up due to the sleep type change
		EXPECT_EQ(SleepData.Num(), 2);
		if (SleepData.Num() >= 2)
		{
			EXPECT_EQ(SleepData[1].Particle, Dynamic1);
			EXPECT_FALSE(SleepData[1].Sleeping);
		}

		AdvanceSleepStates(Dynamic2, Dynamic1);

		// Particle 2 should have fallen asleep
		EXPECT_EQ(SleepData.Num(), 3);
		if (SleepData.Num() >= 3)
		{
			EXPECT_EQ(SleepData[2].Particle, Dynamic2);
			EXPECT_TRUE(SleepData[2].Sleeping);
		}
	}

	GTEST_TEST(AllEvolutions, SimTests_BoxBoxSimTest_DisableParticles)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		auto Static = Evolution.CreateStaticParticles(1)[0];
		auto Dynamic = Evolution.CreateDynamicParticles(1)[0];

		Chaos::FImplicitObjectPtr StaticBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Chaos::FImplicitObjectPtr DynamicBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Static->SetGeometry(StaticBox);
		Dynamic->SetGeometry(DynamicBox);

		Static->SetX(FVec3(10, 10, 10));
		Static->UpdateWorldSpaceState(FRigidTransform3(Static->GetX(), Static->GetR()), FVec3(0));
		Dynamic->SetX(FVec3(10, 10, 120));
		Dynamic->I() = TVec3<FRealSingle>(100000.0f);
		Dynamic->InvI() = TVec3<FRealSingle>(1.0f / 100000.0f);

		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		// IMPORTANT : this is required to make sure the particles internal representation will reflect the sim data
		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		// Step only a few times not to trigger sleeping
		const FReal Dt = 1 / 60.f;
		for (int i = 0; i < 10; ++i)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		// Make sure there is at least one collisions
		EXPECT_GT(Dynamic->ParticleCollisions().Num(), 0);

		// Make sure there is at least one active collision in the constraint graph
		int32 CountActiveCollisions = 0;
		for (int32 Index = 0; Index < Dynamic->ParticleCollisions().Num(); Index++)
		{
			FParticlePairMidPhase* MidPhase = Dynamic->ParticleCollisions().GetMidPhase(Index);
			MidPhase->VisitCollisions([&CountActiveCollisions](FPBDCollisionConstraint& Constraint)
				{
					if (!Constraint.GetDisabled() && Constraint.IsCurrent() && Constraint.IsInConstraintGraph())
					{
						++CountActiveCollisions;
					}
					return ECollisionVisitorResult::Continue;
				});
		}
		EXPECT_GT(CountActiveCollisions, 0);

		Evolution.DisableParticle(Dynamic);
		Evolution.DisableParticle(Static);

		// When particles are disabled, all collisions and other constraints should also be disabled.
		// Collisions and particles should also be removed from the constraint graph.
		EXPECT_EQ(Dynamic->ParticleCollisions().Num(), 0);
		EXPECT_EQ(Evolution.GetIslandManager().GetNumConstraints(), 0);
		EXPECT_EQ(Evolution.GetIslandManager().GetNumParticles(), 0);
	}

	GTEST_TEST(AllEvolutions, SimTests_BoxBoxSimTest_DestroyParticles)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		auto Static = Evolution.CreateStaticParticles(1)[0];
		auto Dynamic = Evolution.CreateDynamicParticles(1)[0];

		Chaos::FImplicitObjectPtr StaticBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Chaos::FImplicitObjectPtr DynamicBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Static->SetGeometry(StaticBox);
		Dynamic->SetGeometry(DynamicBox);

		Static->SetX(FVec3(10, 10, 10));
		Static->UpdateWorldSpaceState(FRigidTransform3(Static->GetX(), Static->GetR()), FVec3(0));
		Dynamic->SetX(FVec3(10, 10, 120));
		Dynamic->I() = TVec3<FRealSingle>(100000.0f);
		Dynamic->InvI() = TVec3<FRealSingle>(1.0f / 100000.0f);

		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		// IMPORTANT : this is required to make sure the particles internal representation will reflect the sim data
		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		// Step only a few times not to trigger sleeping
		const FReal Dt = 1 / 60.f;
		for (int i = 0; i < 10; ++i)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		// Make sure there is at least one collisions
		EXPECT_GT(Dynamic->ParticleCollisions().Num(), 0);

		// Make sure there is at least one active collision in the constraint graph
		int32 CountActiveCollisions = 0;
		for (int32 Index = 0; Index < Dynamic->ParticleCollisions().Num(); Index++)
		{
			FParticlePairMidPhase* MidPhase = Dynamic->ParticleCollisions().GetMidPhase(Index);
			MidPhase->VisitCollisions([&CountActiveCollisions](FPBDCollisionConstraint& Constraint)
				{
					if (!Constraint.GetDisabled() && Constraint.IsCurrent() && Constraint.IsInConstraintGraph())
					{
						++CountActiveCollisions;
					}
					return ECollisionVisitorResult::Continue;
				});
		}
		EXPECT_GT(CountActiveCollisions, 0);

		Evolution.DestroyParticle(Dynamic);
		Evolution.DestroyParticle(Static);

		// When particles are destroyed, all collisions and other constraints should also be destroyed.
		// Collisions and particles should also be removed from the constraint graph.
		//EXPECT_EQ(Dynamic->ParticleCollisions().Num(), 0);
		EXPECT_EQ(Evolution.GetIslandManager().GetNumConstraints(), 0);
		EXPECT_EQ(Evolution.GetIslandManager().GetNumParticles(), 0);
	}

	GTEST_TEST(AllEvolutions, SimTests_BoxBoxSimTest_DisableConstraints)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		auto Static = Evolution.CreateStaticParticles(1)[0];
		auto Dynamic = Evolution.CreateDynamicParticles(1)[0];

		Chaos::FImplicitObjectPtr StaticBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Chaos::FImplicitObjectPtr DynamicBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Static->SetGeometry(StaticBox);
		Dynamic->SetGeometry(DynamicBox);

		Static->SetX(FVec3(10, 10, 10));
		Static->UpdateWorldSpaceState(FRigidTransform3(Static->GetX(), Static->GetR()), FVec3(0));
		Dynamic->SetX(FVec3(10, 10, 120));
		Dynamic->I() = TVec3<FRealSingle>(100000.0f);
		Dynamic->InvI() = TVec3<FRealSingle>(1.0f / 100000.0f);

		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		// IMPORTANT : this is required to make sure the particles internal representation will reflect the sim data
		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		// Step only a few times not to trigger sleeping
		const FReal Dt = 1 / 60.f;
		for (int i = 0; i < 10; ++i)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		// Make sure there is at least one collisions
		EXPECT_GT(Dynamic->ParticleCollisions().Num(), 0);

		// Make sure there is at least one active collision in the constraint graph
		int32 CountActiveCollisions = 0;
		for (int32 Index = 0; Index < Dynamic->ParticleCollisions().Num(); Index++)
		{
			FParticlePairMidPhase* MidPhase = Dynamic->ParticleCollisions().GetMidPhase(Index);
			MidPhase->VisitCollisions([&CountActiveCollisions](FPBDCollisionConstraint& Constraint)
				{
					if (!Constraint.GetDisabled() && Constraint.IsCurrent() && Constraint.IsInConstraintGraph())
					{
						++CountActiveCollisions;
					}
					return ECollisionVisitorResult::Continue;
				});
		}
		EXPECT_GT(CountActiveCollisions, 0);

		Evolution.DisableConstraints(Dynamic);
		Evolution.DisableConstraints(Static);

		// Only non-collision constraints should be disabled
		// Collisions and particles should still be active and in the constraint graph.
		EXPECT_GT(Dynamic->ParticleCollisions().Num(), 0);
		EXPECT_GT(Evolution.GetIslandManager().GetNumConstraints(), 0);
		EXPECT_GT(Evolution.GetIslandManager().GetNumParticles(), 0);
		CountActiveCollisions = 0;
		for (int32 Index = 0; Index < Dynamic->ParticleCollisions().Num(); Index++)
		{
			FParticlePairMidPhase* MidPhase = Dynamic->ParticleCollisions().GetMidPhase(Index);
			MidPhase->VisitCollisions([&CountActiveCollisions](FPBDCollisionConstraint& Constraint)
				{
					if (!Constraint.GetDisabled() && Constraint.IsCurrent() && Constraint.IsInConstraintGraph())
					{
						++CountActiveCollisions;
					}
					return ECollisionVisitorResult::Continue;
				});
		}
		EXPECT_GT(CountActiveCollisions, 0);
	}

	GTEST_TEST(AllEvolutions, SimTests_BoxBoxSimTest_DestroyTransientConstraints)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		auto Static = Evolution.CreateStaticParticles(1)[0];
		auto Dynamic = Evolution.CreateDynamicParticles(1)[0];

		Chaos::FImplicitObjectPtr StaticBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Chaos::FImplicitObjectPtr DynamicBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Static->SetGeometry(StaticBox);
		Dynamic->SetGeometry(DynamicBox);

		Static->SetX(FVec3(10, 10, 10));
		Static->UpdateWorldSpaceState(FRigidTransform3(Static->GetX(), Static->GetR()), FVec3(0));
		Dynamic->SetX(FVec3(10, 10, 120));
		Dynamic->I() = TVec3<FRealSingle>(100000.0f);
		Dynamic->InvI() = TVec3<FRealSingle>(1.0f / 100000.0f);

		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		// IMPORTANT : this is required to make sure the particles internal representation will reflect the sim data
		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		// Step only a few times not to trigger sleeping
		const FReal Dt = 1 / 60.f;
		for (int i = 0; i < 10; ++i)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		// Make sure there is at least one collisions
		EXPECT_GT(Dynamic->ParticleCollisions().Num(), 0);

		// Make sure there is at least one active collision in the constraint graph
		int32 CountActiveCollisions = 0;
		for (int32 Index = 0; Index < Dynamic->ParticleCollisions().Num(); Index++)
		{
			FParticlePairMidPhase* MidPhase = Dynamic->ParticleCollisions().GetMidPhase(Index);
			MidPhase->VisitCollisions([&CountActiveCollisions](FPBDCollisionConstraint& Constraint)
				{
					if (!Constraint.GetDisabled() && Constraint.IsCurrent() && Constraint.IsInConstraintGraph())
					{
						++CountActiveCollisions;
					}
					return ECollisionVisitorResult::Continue;
				});
		}
		EXPECT_GT(CountActiveCollisions, 0);

		Evolution.DestroyTransientConstraints(Dynamic);
		Evolution.DestroyTransientConstraints(Static);

		// Collision constraints should be destroyed
		// Collisions should not be active and in the constraint graph.
		// Particles would be removed from constraint graph in the next tick
		EXPECT_EQ(Dynamic->ParticleCollisions().Num(), 0);
		EXPECT_EQ(Evolution.GetIslandManager().GetNumConstraints(), 0);
		EXPECT_GT(Evolution.GetIslandManager().GetNumParticles(), 0);
		CountActiveCollisions = 0;
		for (int32 Index = 0; Index < Dynamic->ParticleCollisions().Num(); Index++)
		{
			FParticlePairMidPhase* MidPhase = Dynamic->ParticleCollisions().GetMidPhase(Index);
			MidPhase->VisitCollisions([&CountActiveCollisions](FPBDCollisionConstraint& Constraint)
				{
					if (!Constraint.GetDisabled() && Constraint.IsCurrent() && Constraint.IsInConstraintGraph())
					{
						++CountActiveCollisions;
					}
					return ECollisionVisitorResult::Continue;
				});
		}
		EXPECT_EQ(CountActiveCollisions, 0);
	}

	GTEST_TEST(AllEvolutions, SimTests_BoxBoxSimTest_DisconnectConstraints)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);
		InitEvolutionSettings(Evolution);

		auto Static = Evolution.CreateStaticParticles(1)[0];
		auto Dynamic = Evolution.CreateDynamicParticles(1)[0];

		Chaos::FImplicitObjectPtr StaticBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Chaos::FImplicitObjectPtr DynamicBox(new TBox<FReal, 3>(FVec3(-50, -50, -50), FVec3(50, 50, 50)));
		Static->SetGeometry(StaticBox);
		Dynamic->SetGeometry(DynamicBox);

		Static->SetX(FVec3(10, 10, 10));
		Static->UpdateWorldSpaceState(FRigidTransform3(Static->GetX(), Static->GetR()), FVec3(0));
		Dynamic->SetX(FVec3(10, 10, 120));
		Dynamic->I() = TVec3<FRealSingle>(100000.0f);
		Dynamic->InvI() = TVec3<FRealSingle>(1.0f / 100000.0f);

		::ChaosTest::SetParticleSimDataToCollide({ Static,Dynamic });

		// IMPORTANT : this is required to make sure the particles internal representation will reflect the sim data
		Evolution.EnableParticle(Static);
		Evolution.EnableParticle(Dynamic);

		// Step only a few times not to trigger sleeping
		const FReal Dt = 1 / 60.f;
		for (int i = 0; i < 10; ++i)
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		// Make sure there is at least one collisions
		EXPECT_GT(Dynamic->ParticleCollisions().Num(), 0);

		// Make sure there is at least one active collision in the constraint graph
		int32 CountActiveCollisions = 0;
		for (int32 Index = 0; Index < Dynamic->ParticleCollisions().Num(); Index++)
		{
			FParticlePairMidPhase* MidPhase = Dynamic->ParticleCollisions().GetMidPhase(Index);
			MidPhase->VisitCollisions([&CountActiveCollisions](FPBDCollisionConstraint& Constraint)
				{
					if (!Constraint.GetDisabled() && Constraint.IsCurrent() && Constraint.IsInConstraintGraph())
					{
						++CountActiveCollisions;
					}
					return ECollisionVisitorResult::Continue;
				});
		}
		EXPECT_GT(CountActiveCollisions, 0);

		Evolution.DisconnectPersistentConstraints(TSet<FGeometryParticleHandle*>({ Dynamic, Static }));

		// Only non-collision constraints should be disconnected
		// Collisions and particles should still be active and in the constraint graph.
		EXPECT_GT(Dynamic->ParticleCollisions().Num(), 0);
		EXPECT_GT(Evolution.GetIslandManager().GetNumConstraints(), 0);
		EXPECT_GT(Evolution.GetIslandManager().GetNumParticles(), 0);
		CountActiveCollisions = 0;
		for (int32 Index = 0; Index < Dynamic->ParticleCollisions().Num(); Index++)
		{
			FParticlePairMidPhase* MidPhase = Dynamic->ParticleCollisions().GetMidPhase(Index);
			MidPhase->VisitCollisions([&CountActiveCollisions](FPBDCollisionConstraint& Constraint)
				{
					if (!Constraint.GetDisabled() && Constraint.IsCurrent() && Constraint.IsInConstraintGraph())
					{
						++CountActiveCollisions;
					}
					return ECollisionVisitorResult::Continue;
				});
		}
		EXPECT_GT(CountActiveCollisions, 0);
	}
}

