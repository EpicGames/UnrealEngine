// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestEvolution.h"
#include "Chaos/Collision/SimRaycast.h"
#include "Chaos/Collision/SimSweep.h"

PRAGMA_DISABLE_INTERNAL_WARNINGS

namespace ChaosTest
{
	using namespace Chaos;

	GTEST_TEST(SimSweepTests, TestBoxBoxOverlap)
	{
		const FVec3 FloorSize = FVec3(10000, 10000, 100);
		const FVec3 BoxPos = FVec3(0, 0, 200);
		const FRotation3 BoxRot = FRotation3::FromIdentity();
		const FVec3 BoxSize = FVec3(100);
		const FReal BoxMass = 100;

		FEvolutionTest Test;

		FGeometryParticleHandle* Floor = Test.AddParticleFloor(FloorSize);
		FPBDRigidParticleHandle* Box = Test.AddParticleBox(BoxPos, BoxRot, BoxSize, BoxMass);

		Test.Tick();

		TArray<Private::FSimOverlapParticleShape> Overlaps;
		const FAABB3 QueryBounds = FAABB3(FVec3(-1000), FVec3(1000));
		Private::SimOverlapBoundsAll(Test.GetEvolution().GetSpatialAcceleration(), QueryBounds, Overlaps);

		EXPECT_EQ(Overlaps.Num(), 2);
		if (Overlaps.Num() == 2)
		{
			EXPECT_TRUE((Overlaps[0].HitParticle == Floor) || (Overlaps[1].HitParticle == Floor));
			EXPECT_TRUE((Overlaps[0].HitParticle == Box) || (Overlaps[1].HitParticle == Box));
		}
	}

	GTEST_TEST(SimSweepTests, TestBoxBoxSweep)
	{
		const FVec3 FloorSize = FVec3(10000, 10000, 100);
		const FVec3 BoxPos = FVec3(0, 0, 500);
		const FRotation3 BoxRot = FRotation3::FromIdentity();
		const FVec3 BoxSize = FVec3(100);
		const FReal BoxMass = 100;

		FEvolutionTest Test;

		FGeometryParticleHandle* Floor = Test.AddParticleFloor(FloorSize);
		FPBDRigidParticleHandle* Box = Test.AddParticleBox(BoxPos, BoxRot, BoxSize, BoxMass);

		Test.Tick();

		FVec3 SweepStartPos = FVec3(0,0, 500);
		FVec3 SweepDir = FVec3(0, 0, -1);
		FRotation3 SweepRot = FRotation3::FromIdentity();
		FIgnoreCollisionManager* IgnoreCollisionManager = nullptr;

		// 100cm length sweep from Z=500 should miss
		Private::FSimSweepParticleHit Hit100;
		bool bIsHit100 = Private::SimSweepParticleFirstHit(
			Test.GetEvolution().GetSpatialAcceleration(), IgnoreCollisionManager, 
			Box, 
			SweepStartPos, SweepRot, SweepDir, 100, 
			Hit100);
		EXPECT_FALSE(bIsHit100);
		EXPECT_FALSE(Hit100.IsHit());

		// 1000cm downward sweep from Z=500 should hit with Z=50, for a sweep distance of D=450
		Private::FSimSweepParticleHit Hit1000;
		bool bIsHit1000 = Private::SimSweepParticleFirstHit(
			Test.GetEvolution().GetSpatialAcceleration(), IgnoreCollisionManager,
			Box,
			SweepStartPos,SweepRot, SweepDir, 1000,
			Hit1000);
		EXPECT_TRUE(bIsHit1000);
		EXPECT_TRUE(Hit1000.IsHit());
		if (bIsHit1000)
		{
			EXPECT_NEAR(Hit1000.HitDistance, 450.0f, UE_KINDA_SMALL_NUMBER);
		}
	}

	// Raycast against the floor (a cube)
	GTEST_TEST(SimSweepTests, TestBoxRaycastSimple)
	{
		const FVec3 FloorSize = FVec3(10000, 10000, 100);

		FEvolutionTest Test;

		FGeometryParticleHandle* Floor = Test.AddParticleFloor(FloorSize);
		Test.GetEvolution().SetParticleTransform(Floor, FVec3(0,0,100), FRotation3::FromIdentity(), false, false);

		Test.Tick();

		Private::FSimRaycastRay Ray;
		Ray.StartPos = FVec3f(0,0,500);
		Ray.Dir = FVec3f(0,0,-1);
		Ray.Length = 1000;
		
		Private::FSimRaycastHit Hit;

		Filter::FQueryFilterData QueryFilter = Filter::FQueryObjectFilterBuilder().SetObjectTypes(0xFFFFFFFF).Build();

		bool bAnyHits = Private::SimRaycastBatchFirstHits(
			Test.GetEvolution().GetSpatialAcceleration(),
			MakeArrayView<Private::FSimRaycastRay>(&Ray, 1),
			QueryFilter,
			/*Simple*/true, /*Complex*/false,
			MakeArrayView<Private::FSimRaycastHit>(&Hit, 1));

		EXPECT_TRUE(bAnyHits);
		EXPECT_TRUE(Hit.IsHit());
		EXPECT_NEAR(Hit.HitDistance, 350.0f, UE_SMALL_NUMBER);
		EXPECT_NEAR(Hit.HitPosition.Z, 150.0f, UE_SMALL_NUMBER);
	}

	// Raycast against a cube and the floor, with filtering to ignore one or the other
	GTEST_TEST(SimSweepTests, TestBoxRaycastSimpleFiltered)
	{
		const FVec3 FloorSize = FVec3(10000, 10000, 100);
		const FVec3 BoxPos = FVec3(0, 0, 500);
		const FRotation3 BoxRot = FRotation3::FromIdentity();
		const FVec3 BoxSize = FVec3(100);
		const FReal BoxMass = 100;

		FEvolutionTest Test;

		FGeometryParticleHandle* Floor = Test.AddParticleFloor(FloorSize);
		FPBDRigidParticleHandle* Box = Test.AddParticleBox(BoxPos, BoxRot, BoxSize, BoxMass);
		Test.GetEvolution().SetParticleObjectState(Box, EObjectStateType::Kinematic);

		// Floor -> CollisionChannel 0
		Filter::FShapeFilterData FloorFilterData = Filter::FShapeFilterBuilder().SetCollisionChannelIndex(0).SetBlockChannelMask(0xFFFFFFFF).Build();
		Floor->ShapeInstances()[0]->SetShapeFilterData(FloorFilterData);

		// Box -> CollisionChannel 1
		Filter::FShapeFilterData BoxFilterData = Filter::FShapeFilterBuilder().SetCollisionChannelIndex(1).SetBlockChannelMask(0xFFFFFFFF).Build();
		Box->ShapeInstances()[0]->SetShapeFilterData(BoxFilterData);

		Test.Tick();

		Private::FSimRaycastRay Rays[2];
		// Downward Ray
		Rays[0].StartPos = FVec3f(0, 0, 1000);
		Rays[0].Dir = FVec3f(0, 0, -1);
		Rays[0].Length = 2000;
		// Upward Ray
		Rays[1].StartPos = FVec3f(0, 0, -1000);
		Rays[1].Dir = FVec3f(0, 0, 1);
		Rays[1].Length = 2000;

		// Detect Collision Channel 0
		{
			Private::FSimRaycastHit Hits[2];

			Filter::FQueryFilterData QueryFilter = Filter::FQueryObjectFilterBuilder().SetObjectTypes(1 << 0).Build();

			bool bAnyHits = Private::SimRaycastBatchFirstHits(
				Test.GetEvolution().GetSpatialAcceleration(),
				MakeArrayView<Private::FSimRaycastRay>(Rays, UE_ARRAY_COUNT(Rays)),
				QueryFilter,
				/*Simple*/true, /*Complex*/false,
				MakeArrayView<Private::FSimRaycastHit>(Hits, UE_ARRAY_COUNT(Hits)));

			// Downward Ray - Floor (filtered Box)
			EXPECT_TRUE(Hits[0].IsHit());
			EXPECT_EQ(Hits[0].HitParticle, Floor);

			// Upward Ray - Floor (First encountered)
			EXPECT_TRUE(Hits[1].IsHit());
			EXPECT_EQ(Hits[1].HitParticle, Floor);
		}

		// Detect Collision Channel 1
		{
			Private::FSimRaycastHit Hits[2];

			Filter::FQueryFilterData QueryFilter = Filter::FQueryObjectFilterBuilder().SetObjectTypes(1 << 1).Build();

			bool bAnyHits = Private::SimRaycastBatchFirstHits(
				Test.GetEvolution().GetSpatialAcceleration(),
				MakeArrayView<Private::FSimRaycastRay>(Rays, UE_ARRAY_COUNT(Rays)),
				QueryFilter,
				/*Simple*/true, /*Complex*/false,
				MakeArrayView<Private::FSimRaycastHit>(Hits, UE_ARRAY_COUNT(Hits)));

			// Downward Ray - Box (First encountered)
			EXPECT_TRUE(Hits[0].IsHit());
			EXPECT_EQ(Hits[0].HitParticle, Box);

			// Upward Ray - Box (filtered Floor)
			EXPECT_TRUE(Hits[1].IsHit());
			EXPECT_EQ(Hits[1].HitParticle, Box);
		}
	}

	// Raycast against complex (in a scene with no complex shapes)
	GTEST_TEST(SimSweepTests, TestBoxRaycastComplex)
	{
		const FVec3 FloorSize = FVec3(10000, 10000, 100);

		FEvolutionTest Test;

		FGeometryParticleHandle* Floor = Test.AddParticleFloor(FloorSize);
		Test.GetEvolution().SetParticleTransform(Floor, FVec3(0, 0, 100), FRotation3::FromIdentity(), false, false);

		Test.Tick();

		Private::FSimRaycastRay Ray;
		Ray.StartPos = FVec3f(0, 0, 500);
		Ray.Dir = FVec3f(0, 0, -1);
		Ray.Length = 1000;

		Private::FSimRaycastHit Hit;

		Filter::FQueryFilterData QueryFilter = Filter::FQueryObjectFilterBuilder().SetObjectTypes(0xFFFFFFFF).Build();

		bool bAnyHits = Private::SimRaycastBatchFirstHits(
			Test.GetEvolution().GetSpatialAcceleration(),
			MakeArrayView<Private::FSimRaycastRay>(&Ray, 1),
			QueryFilter,
			/*Simple*/false, /*Complex*/true,
			MakeArrayView<Private::FSimRaycastHit>(&Hit, 1));

		EXPECT_FALSE(bAnyHits);
		EXPECT_FALSE(Hit.IsHit());
	}

	// Multiple raycasts as a batch
	GTEST_TEST(SimSweepTests, TestRaycastFirst)
	{
		const FVec3 FloorSize = FVec3(10000, 10000, 100);
		const FVec3 FloorPos = FVec3(0, 0, 100);
		const FVec3 BoxPos = FVec3(0, 0, 500);
		const FRotation3 BoxRot = FRotation3::FromIdentity();
		const FVec3 BoxSize = FVec3(100);
		const FReal BoxMass = 100;

		FEvolutionTest Test;

		FPBDRigidParticleHandle* Box = Test.AddParticleBox(BoxPos, BoxRot, BoxSize, BoxMass);
		Test.GetEvolution().SetParticleObjectState(Box, EObjectStateType::Kinematic);

		FGeometryParticleHandle* Floor = Test.AddParticleFloor(FloorSize);
		Test.GetEvolution().SetParticleTransform(Floor, FVec3(0, 0, 100), FRotation3::FromIdentity(), false, false);

		Test.Tick();

		Private::FSimRaycastRay Rays[2];
		// Downward Ray
		Rays[0].StartPos = FVec3f(0, 0, 1000);
		Rays[0].Dir = FVec3f(0, 0, -1);
		Rays[0].Length = 2000;
		// Upward Ray
		Rays[1].StartPos = FVec3f(0, 0, -1000);
		Rays[1].Dir = FVec3f(0, 0, 1);
		Rays[1].Length = 2000;

		Private::FSimRaycastHit Hits[2];

		Filter::FQueryFilterData QueryFilter = Filter::FQueryObjectFilterBuilder().SetObjectTypes(0xFFFFFFFF).Build();

		bool bAnyHits = Private::SimRaycastBatchFirstHits(
			Test.GetEvolution().GetSpatialAcceleration(),
			MakeArrayView<Private::FSimRaycastRay>(Rays, UE_ARRAY_COUNT(Rays)),
			QueryFilter,
			/*Simple*/true, /*Complex*/false,
			MakeArrayView<Private::FSimRaycastHit>(Hits, UE_ARRAY_COUNT(Hits)));

		EXPECT_TRUE(bAnyHits);

		// Downward Ray
		EXPECT_TRUE(Hits[0].IsHit());
		EXPECT_EQ(Hits[0].HitParticle, Box);
		EXPECT_NEAR(Hits[0].HitDistance, 450.0f, UE_SMALL_NUMBER);
		EXPECT_NEAR(Hits[0].HitPosition.Z, 550.0f, UE_SMALL_NUMBER);

		// Upward Ray
		EXPECT_TRUE(Hits[1].IsHit());
		EXPECT_EQ(Hits[1].HitParticle, Floor);
		EXPECT_NEAR(Hits[1].HitDistance, 1050.0f, UE_SMALL_NUMBER);
		EXPECT_NEAR(Hits[1].HitPosition.Z, 50.0f, UE_SMALL_NUMBER);
	}

}

PRAGMA_ENABLE_INTERNAL_WARNINGS
