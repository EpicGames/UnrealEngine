// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSolversModule.h"
#include "HeadlessChaosTestUtility.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/JointConstraintProxy.h"
#include "Chaos/PBDJointConstraintData.h"
#include "HAL/ConsoleManager.h"
#include "PhysicsInterfaceDeclaresCore.h"

namespace ChaosTest
{
	using namespace Chaos;

	class SleepingTests : public ::testing::TestWithParam<bool>
	{
		void SetUp()
		{
			SetPartialIslandSleepUnitTestDefaults();
		}

		void TearDown()
		{
			RestorePartialIslandSleepEditorDefaults();
		}

		void SetPartialIslandSleepUnitTestDefaults()
		{
			CVarPostStepWake = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.PostStepWakeThreshold"), false);
			check(CVarPostStepWake && CVarPostStepWake->IsVariableFloat());
			PostStepWake_EditorDefault = CVarPostStepWake->GetFloat();
			CVarPostStepWake->Set(PostStepWake_UnitTestDefault);

			CVarMomentumPropagation = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.MomentumPropagation"), false);
			check(CVarMomentumPropagation && CVarMomentumPropagation->IsVariableFloat());
			MomentumPropagation_EditorDefault = CVarMomentumPropagation->GetFloat();
			CVarMomentumPropagation->Set(MomentumPropagation_UnitTestDefault);

			CVarPartialSleepCollisionConstraintsOnly = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.CollisionConstraintsOnly"), false);
			check(CVarPartialSleepCollisionConstraintsOnly && CVarPartialSleepCollisionConstraintsOnly->IsVariableInt());
			PartialSleepCollisionConstraintsOnly_EditorDefault = CVarPartialSleepCollisionConstraintsOnly->GetInt();
			CVarPartialSleepCollisionConstraintsOnly->Set(PartialSleepCollisionConstraintsOnly_UnitTestDefault);

			CVarPartialWakePreIntegration = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.PartialWakePreIntegration"), false);
			check(CVarPartialWakePreIntegration && CVarPartialWakePreIntegration->IsVariableBool());
			bPartialWakePreIntegration_EditorDefault = CVarPartialWakePreIntegration->GetBool();
			CVarPartialWakePreIntegration->Set(bPartialWakePreIntegration_UnitTestDefault);

			CVarMinMotionlessRatio = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.MinimumMotionlessRatio"), false);
			check(CVarMinMotionlessRatio && CVarMinMotionlessRatio->IsVariableFloat());
			MinimumMotionlessThreshold_EditorDefault = CVarMinMotionlessRatio->GetFloat();
			CVarMinMotionlessRatio->Set(MinimumMotionlessThreshold_UnitTestDefault);

			CVarLinearWakeThresholdMultiplier = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.LinearWakeThresholdMultiplier"), false);
			check(CVarLinearWakeThresholdMultiplier && CVarLinearWakeThresholdMultiplier->IsVariableFloat());
			LinearWakeThresholdMultiplier_EditorDefault = CVarLinearWakeThresholdMultiplier->GetFloat();
			CVarLinearWakeThresholdMultiplier->Set(LinearWakeThresholdMultiplier_UnitTestDefault);

			CVarAngularWakeThresholdMultiplier = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.AngularWakeThresholdMultiplier"), false);
			check(CVarAngularWakeThresholdMultiplier && CVarAngularWakeThresholdMultiplier->IsVariableFloat());
			AngularWakeThresholdMultiplier_EditorDefault = CVarAngularWakeThresholdMultiplier->GetFloat();
			CVarAngularWakeThresholdMultiplier->Set(AngularWakeThresholdMultiplier_UnitTestDefault);
			
			CVarValidateConstraintSleepState = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep.ValidateConstraintSleepState"), false);
			check(CVarValidateConstraintSleepState && CVarValidateConstraintSleepState->IsVariableBool());
			bValidateConstraintSleepState_EditorDefault = CVarValidateConstraintSleepState->GetBool();
			CVarValidateConstraintSleepState->Set(bValidateConstraintSleepState_UnitTestDefault);
		}

		void RestorePartialIslandSleepEditorDefaults()
		{
			CVarPostStepWake->Set(PostStepWake_EditorDefault);
			CVarMomentumPropagation->Set(MomentumPropagation_EditorDefault);
			CVarPartialSleepCollisionConstraintsOnly->Set(PartialSleepCollisionConstraintsOnly_EditorDefault);
			CVarPartialWakePreIntegration->Set(bPartialWakePreIntegration_EditorDefault);
			CVarMinMotionlessRatio->Set(MinimumMotionlessThreshold_EditorDefault);
			CVarLinearWakeThresholdMultiplier->Set(LinearWakeThresholdMultiplier_EditorDefault);
			CVarAngularWakeThresholdMultiplier->Set(AngularWakeThresholdMultiplier_EditorDefault);
			CVarValidateConstraintSleepState->Set(bValidateConstraintSleepState_EditorDefault);
		}

		// Default values for unit testing
		FRealSingle PostStepWake_UnitTestDefault = 0.0f;
		FRealSingle MomentumPropagation_UnitTestDefault = 0.0f;
		int32 PartialSleepCollisionConstraintsOnly_UnitTestDefault = 0;
		bool bPartialWakePreIntegration_UnitTestDefault = false;
		FRealSingle MinimumMotionlessThreshold_UnitTestDefault = 0.0f;
		FRealSingle LinearWakeThresholdMultiplier_UnitTestDefault = 1.0f;
		FRealSingle AngularWakeThresholdMultiplier_UnitTestDefault = 1.0f;
		bool bValidateConstraintSleepState_UnitTestDefault = true;

		// Storage for default values editor hard-coded in IslandManager.cpp
		FRealSingle PostStepWake_EditorDefault;
		FRealSingle MomentumPropagation_EditorDefault;
		int32 PartialSleepCollisionConstraintsOnly_EditorDefault;
		bool bPartialWakePreIntegration_EditorDefault;
		FRealSingle MinimumMotionlessThreshold_EditorDefault;
		FRealSingle LinearWakeThresholdMultiplier_EditorDefault;
		FRealSingle AngularWakeThresholdMultiplier_EditorDefault;
		bool bValidateConstraintSleepState_EditorDefault;

	public:
		// Storage of console variable pointer
		IConsoleVariable* CVarPostStepWake;
		IConsoleVariable* CVarMomentumPropagation;
		IConsoleVariable* CVarPartialSleepCollisionConstraintsOnly;
		IConsoleVariable* CVarPartialWakePreIntegration;
		IConsoleVariable* CVarMinMotionlessRatio;
		IConsoleVariable* CVarLinearWakeThresholdMultiplier;
		IConsoleVariable* CVarAngularWakeThresholdMultiplier;
		IConsoleVariable* CVarValidateConstraintSleepState;
	};

	class FSleepingTest
	{
	public:
		FSleepingTest(const bool bPartialSleeping)
			: MaterialData()
			, NoSleepMaterialData()
			, FloorProxy()
			, ParticleProxies()
			, ParticleHandles()
			, JointHandles()
			, TickCount(0)
			, CVarPartialSleeping()
		{
			// Create materials before creating the solver so they are included in SimMaterials
			FMaterialHandle MaterialHandle = FPhysicalMaterialManager::Get().Create();
			FPhysicsMaterial* PhysicsMaterial = MaterialHandle.Get();
			PhysicsMaterial->Friction = 0.7;
			PhysicsMaterial->StaticFriction = 0.0;
			PhysicsMaterial->Restitution = 0.0;
			PhysicsMaterial->Density = 1.0;
			PhysicsMaterial->SleepingLinearThreshold = 1.0;
			PhysicsMaterial->SleepingAngularThreshold = 0.05;
			PhysicsMaterial->SleepCounterThreshold = 4;
			MaterialData.Materials.Add(MaterialHandle);

			FMaterialHandle NoSleepMaterialHandle = FPhysicalMaterialManager::Get().Create();
			FPhysicsMaterial* NoSleepPhysicsMaterial = NoSleepMaterialHandle.Get();
			NoSleepPhysicsMaterial->Friction = 0.7;
			NoSleepPhysicsMaterial->StaticFriction = 0.0;
			NoSleepPhysicsMaterial->Restitution = 0.0;
			NoSleepPhysicsMaterial->Density = 1.0;
			NoSleepPhysicsMaterial->SleepingLinearThreshold = 0.0;
			NoSleepPhysicsMaterial->SleepingAngularThreshold = 0.0;
			NoSleepPhysicsMaterial->SleepCounterThreshold = 4;
			NoSleepMaterialData.Materials.Add(NoSleepMaterialHandle);

			Module = FChaosSolversModule::GetModule();
			Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);
			Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);

			CVarPartialSleeping = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.Sleep.PartialIslandSleep"), false);
			check(CVarPartialSleeping && CVarPartialSleeping->IsVariableBool());
			PartialSleepDefault = CVarPartialSleeping->GetBool();
			CVarPartialSleeping->Set(bPartialSleeping);
		}

		~FSleepingTest()
		{
			CVarPartialSleeping->Set(PartialSleepDefault);

			for (FSingleParticlePhysicsProxy* ParticleProxy : ParticleProxies)
			{
				Solver->UnregisterObject(ParticleProxy);
			}
			for (FJointConstraint* Joint : JointHandles)
			{
				if (Joint)
					Solver->UnregisterObject(Joint);
			}
			if (FloorProxy)
				Solver->UnregisterObject(FloorProxy);
			Module->DestroySolver(Solver);
		}

		FRigidBodyHandle_External* MakeFloor(const FVec3& Position = FVec3(0))
		{
			FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
			FRigidBodyHandle_External* FloorParticle = &FloorProxy->GetGameThreadAPI();
			FImplicitObjectPtr FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-5000, -5000, -100), FVec3(5000, 5000, 0)));
			FloorParticle->SetGeometry(FloorGeom);
			FloorParticle->SetObjectState(EObjectStateType::Static);
			Solver->RegisterObject(FloorProxy);
			FloorParticle->SetX(Position);
			ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });
			for (const TUniquePtr<FPerShapeData>& Shape : FloorParticle->ShapesArray())
			{
				Shape->SetMaterialData(MaterialData);
			}
			return FloorParticle;
		}

		FRigidBodyHandle_External* MakeBox(const FVec3& Position, const FVec3& Dimensions = FVec3(200, 400, 100), const FReal Mass = 1)
		{
			FSingleParticlePhysicsProxy* BoxProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			FRigidBodyHandle_External* BoxParticle = &BoxProxy->GetGameThreadAPI();
			const FVec3 HalfSize = Dimensions / 2.0f;
			FImplicitObjectPtr CollidingCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(-HalfSize, HalfSize));
			BoxParticle->SetGeometry(CollidingCubeGeom);
			Solver->RegisterObject(BoxProxy);
			BoxParticle->SetGravityEnabled(true);
			BoxParticle->SetX(Position);
			SetBoxInertiaTensor(*BoxParticle, /*Dimensions=*/Dimensions, /*Mass=*/Mass);
			ChaosTest::SetParticleSimDataToCollide({ BoxProxy->GetParticle_LowLevel() });
			for (const TUniquePtr<FPerShapeData>& Shape : BoxParticle->ShapesArray())
			{
				Shape->SetMaterialData(MaterialData);
			}

			ParticleProxies.Add(BoxProxy);
			ParticleHandles.Add(BoxParticle);

			return BoxParticle;
		}

		FRigidBodyHandle_External* MakeCube(const FVec3& Position, const FReal Size = 200, const FReal Mass = 1, const FMaterialData* OverrideMaterialData = nullptr)
		{
			FSingleParticlePhysicsProxy* CubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			FRigidBodyHandle_External* CubeParticle = &CubeProxy->GetGameThreadAPI();
			const FReal HalfSize = Size / 2.0f;
			FImplicitObjectPtr CollidingCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-HalfSize), FVec3(HalfSize)));
			CubeParticle->SetGeometry(CollidingCubeGeom);
			Solver->RegisterObject(CubeProxy);
			CubeParticle->SetGravityEnabled(true);
			CubeParticle->SetX(Position);
			SetCubeInertiaTensor(*CubeParticle, /*Dimension=*/Size, /*Mass=*/Mass);
			ChaosTest::SetParticleSimDataToCollide({ CubeProxy->GetParticle_LowLevel() });
			const FMaterialData& ShapeMaterial = OverrideMaterialData ? *OverrideMaterialData : MaterialData;
			for (const TUniquePtr<FPerShapeData>& Shape : CubeParticle->ShapesArray())
			{
				Shape->SetMaterialData(ShapeMaterial);
			}

			ParticleProxies.Add(CubeProxy);
			ParticleHandles.Add(CubeParticle);

			return CubeParticle;
		}

		FRigidBodyHandle_External* MakeSphere(const FVec3& Position, const FReal Radius = 100, const FReal Mass = 1)
		{
			FSingleParticlePhysicsProxy* SphereProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			FRigidBodyHandle_External* SphereParticle = &SphereProxy->GetGameThreadAPI();
			FImplicitObjectPtr CollidingSphereGeom = Chaos::FImplicitObjectPtr(new TSphere<FReal, 3>(FVec3(0), Radius));
			SphereParticle->SetGeometry(CollidingSphereGeom);
			Solver->RegisterObject(SphereProxy);
			SphereParticle->SetGravityEnabled(true);
			SphereParticle->SetX(Position);
			SetSphereInertiaTensor(*SphereParticle, /*Radius=*/Radius, /*Mass=*/Mass);
			ChaosTest::SetParticleSimDataToCollide({ SphereProxy->GetParticle_LowLevel() });
			for (const TUniquePtr<FPerShapeData>& Shape : SphereParticle->ShapesArray())
			{
				Shape->SetMaterialData(MaterialData);
			}

			ParticleProxies.Add(SphereProxy);
			ParticleHandles.Add(SphereParticle);

			return SphereParticle;
		}

		FJointConstraint* AddJoint(const FProxyBasePair& InConstrainedParticles, const FTransformPair& InTransform)
		{
			FJointConstraint* Joint = new FJointConstraint();
			Joint->SetParticleProxies(InConstrainedParticles);

			FPBDJointSettings SettingsTemp = Joint->GetJointSettings();
			SettingsTemp.ConnectorTransforms = InTransform;
			Joint->SetJointSettings(SettingsTemp);

			Solver->RegisterObject(Joint);

			JointHandles.Add(Joint);
			return Joint;
		}

		void MakeStackOfCubes(const int32 Num, const FReal Size = 200, const FVec3 Pos = FVec3(0))
		{
			MakeFloor();
			const FReal InitialHeight = Size / 2.0f;
			for (int32 Id = 0; Id < Num; ++Id)
			{
				const FReal Z = InitialHeight + Size * Id;
				MakeCube(Pos + FVec3(0, 0, Z), Size);
			}
		}

		void MakeQueueOfCubes(const int32 Num, const FReal Size = 200)
		{
			MakeFloor();
			const FReal InitialHeight = Size / 2.0f;
			for (int32 Id = 0; Id < Num; ++Id)
			{
				const FReal Y = Size * Id;
				const FReal Z = InitialHeight;
				MakeCube(FVec3(0, Y, Z), Size);
			}
		}

		void MakeStackOfSpheres(const int32 Num, const FReal Radius = 100)
		{
			MakeFloor();
			const FReal InitialHeight = Radius;
			for (int32 Id = 0; Id < Num; ++Id)
			{
				const FReal Z = InitialHeight + 2 * Radius * Id;
				MakeSphere(FVec3(0, 0, Z), Radius);
			}
		}

		void MakeChainOfSpheres(const int32 Num, const FReal Radius = 100)
		{
			const int32 DistanceMultiplier = 3;
			for (int32 Id = 0; Id < Num; ++Id)
			{
				const FReal Z = -DistanceMultiplier * Radius * Id;
				FRigidBodyHandle_External* SphereParticle = MakeSphere(FVec3(0, 0, Z), Radius);
			}
			for (int32 Id = 0; Id < Num - 1; ++Id)
			{
				FRigidTransform3 Transform1 = FRigidTransform3(FVec3(0, 0, -DistanceMultiplier * Radius * 0.5), FRotation3::FromIdentity());
				FRigidTransform3 Transform2 = FRigidTransform3(FVec3(0, 0, DistanceMultiplier * Radius * 0.5), FRotation3::FromIdentity());
				AddJoint({ ParticleProxies[Id], ParticleProxies[Id + 1] }, { Transform1 , Transform2 });
			}

		}

		// Build a brick wall using the following pattern (NumY = 5, NumZ = 3)
		// 
		// if (bShiftUnevenRows)
		// P10 -P11 -P12 -P13 - P14
		//   \  / \  / \  / \  /  \
		//    P5 - P6 - P7 - P8 - P9
		//	  / \  / \  / \  / \ /
		//	P0 - P1 - P2 - P3 - P4
		//  |    |    |    |    |
		//          Floor
		// 
		// if (!bShiftUnevenRows)
		// P10 -P11 -P12 -P13 -P14
		//  |    |    |    |    |
		//  P5 - P6 - P7 - P8 - P9
		//  |    |    |    |    |
		//	P0 - P1 - P2 - P3 - P4
		//  |    |    |    |    |
		//          Floor
		// NOTE: The COM of P0 is located at (0, 0, 0) and the top of the floor at (0, 0, -HalfBrickSize).
		void MakeBrickWall(const int32 NumY, const int32 NumZ, const FVec3& BrickDimensions = FVec3(100, 200, 50), const FReal Mass = 2, const bool bShiftUnevenRows = true)
		{
			const FReal HalfWidth = BrickDimensions.Y * 0.5;
			const FReal HalfHeight = BrickDimensions.Z * 0.5;
			MakeFloor(FVec3(0, 0, -HalfHeight));
			for (int32 Z = 0; Z < NumZ; ++Z)
			{
				for (int32 Y = 0; Y < NumY; ++Y)
				{
					const bool bIsUneven = Z % 2 == 1;
					if (bShiftUnevenRows && bIsUneven) // uneven
					{
						//if (Y + 1 == NumY) continue; // skip the last brick
						// Shift every uneven row by HalfWidth
						MakeBox(FVec3(0, Y * BrickDimensions.Y + HalfWidth, Z * BrickDimensions.Z), BrickDimensions, Mass);
					}
					else // even
					{
						MakeBox(FVec3(0, Y * BrickDimensions.Y, Z * BrickDimensions.Z), BrickDimensions, Mass);
					}
				}
			}
		}

		void Advance()
		{
			Solver->AdvanceAndDispatch_External(1 / 60.0);
			Solver->UpdateGameThreadStructures();
			++TickCount;
		}

		void AdvanceUntilSleeping(const int32 MaxIterations = 200)
		{
			const int32 MaxTickCount = TickCount + MaxIterations;
			bool bIsSleeping = false;
			while (!bIsSleeping && (TickCount < MaxTickCount))
			{
				Advance();

				bIsSleeping = true;
				for (FRigidBodyHandle_External* ParticleHandle : ParticleHandles)
				{
					// Check that none of the particles is dynamic
					if (ParticleHandle->ObjectState() == EObjectStateType::Dynamic)
					{
						bIsSleeping = false;
					}
				}
			}

			EXPECT_TRUE(bIsSleeping);
			EXPECT_LT(TickCount, MaxTickCount);
		}

		FMaterialData MaterialData;
		FMaterialData NoSleepMaterialData;

		FSingleParticlePhysicsProxy* FloorProxy;
		TArray<FSingleParticlePhysicsProxy*> ParticleProxies;
		TArray<FRigidBodyHandle_External*> ParticleHandles;
		TArray< FJointConstraint*> JointHandles;

		FPBDRigidsSolver* Solver;
		FChaosSolversModule* Module;

		int32 TickCount;

		IConsoleVariable* CVarPartialSleeping;
		bool PartialSleepDefault;
	};

	// Dummy test to make sure a stack of cubes will fall asleep.
	TEST_P(SleepingTests, SleepChainOfSpheres)
	{
		FSleepingTest Test(GetParam());
		Test.MakeChainOfSpheres(5);
		Test.ParticleHandles[0]->SetObjectState(EObjectStateType::Static);

		Test.AdvanceUntilSleeping();

		for (int32 Id = 1; Id < Test.ParticleHandles.Num(); ++Id)
		{
			const FRigidBodyHandle_External* ParticleHandle = Test.ParticleHandles[Id];
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}
	}

	// Dummy test to make sure a stack of cubes will fall asleep.
	TEST_P(SleepingTests, SleepStackOfCubes)
	{
		FSleepingTest Test(GetParam());
		Test.MakeStackOfCubes(5);

		Test.AdvanceUntilSleeping();

		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}
	}

	// Dummy test to make sure a stack of perfectly aligned spheres will fall asleep.
	TEST_P(SleepingTests, SleepStackOfSpheres)
	{
		FSleepingTest Test(GetParam());
		Test.MakeStackOfSpheres(5);

		Test.AdvanceUntilSleeping();

		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}
	}

	TEST_P(SleepingTests, DropCubeOnStack)
	{
		const int32 NumOfStackedObjects = 5;
		const FReal Dimension = 200;
		const bool bPartialSleepEnabled = GetParam();

		// Turn off mommentum propagation to test level-based waking only
		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(0);

		// Create a stack of cubes
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeStackOfCubes(NumOfStackedObjects, /*Size=*/Dimension);

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping();

		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Add another cube on top
		// @todo (Chaos): This presently only triggers a wake-up event if the top cube is moving. 
		// This is not great and should be updated
		const FReal PosZ = 1100; // Right above the stack
		FRigidBodyHandle_External* TopCube = Test.MakeCube(FVec3(0, 0, PosZ), Dimension);
		TopCube->SetV(FVec3(0, 0, -10));

		Test.Advance();

		if (!bPartialSleepEnabled)
		{
			// The entire stack will wake up
			for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
			{
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else
		{
			// Only the top of the stack will wake up
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Dynamic);
		}

		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}

	TEST_P(SleepingTests, CollideWithStack)
	{
		const int32 NumOfStackedObjects = 5;
		const FReal Dimension = 200;
		const bool bPartialSleepEnabled = GetParam();

		// Turn off mommentum propagation to test level-based waking only
		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(0);

		// Create a stack of cubes
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeStackOfCubes(NumOfStackedObjects, /*Size=*/Dimension);

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping();

		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Make the stack collide with a particle 
		const FReal PosZ = 550; // slightly above the 3rd cube from below
		FRigidBodyHandle_External* CollidingCube = Test.MakeCube(FVec3(200, 0, PosZ), Dimension);
		CollidingCube->SetV(FVec3(-100, 0, 0), true);

		Test.Advance();

		if (!bPartialSleepEnabled)
		{
			// The entire stack will wake up
			for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
			{
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else
		{
			// Only the top of the stack will wake up
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Dynamic);
		}

		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}

	TEST_P(SleepingTests, PullCubeOutOfStack)
	{
		const int32 NumOfStackedObjects = 5;
		const FReal Dimension = 200;
		const bool bPartialSleepEnabled = GetParam();

		// Turn off mommentum propagation to test level-based waking only
		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(0);

		// Create a stack of cubes
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeStackOfCubes(NumOfStackedObjects, /*Size=*/Dimension);

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping();

		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Pull one of the particles out of the stack by applying an impulse
		Test.ParticleHandles[3]->SetLinearImpulse(FVec3(0, 100, 0), true);

		Test.Advance();

		if (!bPartialSleepEnabled)
		{
			// The entire stack will wake up
			for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
			{
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else
		{
			// The stack wakes up only above the level of where the impulse is applied.
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);
		}

		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}

	TEST_P(SleepingTests, UpdateMassOfCubeInStack)
	{
		const int32 NumOfStackedObjects = 5;
		const FReal Dimension = 200;
		const bool bPartialSleepEnabled = GetParam();

		// Turn off momentum propagation to test level-based waking only
		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(0);

		const bool bPartialWakePreIntegrationDefault = CVarPartialWakePreIntegration->GetBool();
		CVarPartialWakePreIntegration->Set(true);

		// Create a stack of cubes
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeStackOfCubes(NumOfStackedObjects, /*Size=*/Dimension);

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping();

		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Update the particle mass without waking the particle
		Test.ParticleHandles[3]->SetM(10);

		Test.Advance();

		// Updating the particle mass only will not wake the stack.
		// Neither in full nor in partial island sleep mode.
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Update the particle mass and wake up the particle
		Test.ParticleHandles[3]->SetM(20);
		Test.ParticleHandles[3]->SetObjectState(EObjectStateType::Dynamic);

		Test.Advance();

		if (!bPartialSleepEnabled)
		{
			// The entire stack wakes up.
			for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
			{
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else
		{
			// The stack wakes up only above the level of where the mass was changed.
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);
		}

		CVarMomentumPropagation->Set(MomentumPropagationDefault);
		CVarPartialWakePreIntegration->Set(bPartialWakePreIntegrationDefault);
	}

	TEST_P(SleepingTests, DeleteBottomCubeInStack)
	{
		const int32 NumOfStackedObjects = 5;
		const FReal Dimension = 200;
		const bool bPartialSleepEnabled = GetParam();

		// Turn off momentum propagation to test level-based waking only
		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(0);

		const bool bPartialWakePreIntegrationDefault = CVarPartialWakePreIntegration->GetBool();
		CVarPartialWakePreIntegration->Set(true);

		// Create a stack of cubes
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeStackOfCubes(NumOfStackedObjects, /*Size=*/Dimension);

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping();

		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Delete the bottom cube
		FSingleParticlePhysicsProxy* BottomBoxProxy = Test.ParticleProxies[0];
		Test.ParticleHandles.RemoveAt(0);
		Test.ParticleProxies.RemoveAt(0);
		Test.Solver->UnregisterObject(BottomBoxProxy);

		Test.Advance();

		// The entire stack will wake up
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
		}

		CVarMomentumPropagation->Set(MomentumPropagationDefault);
		CVarPartialWakePreIntegration->Set(bPartialWakePreIntegrationDefault);
	}

	TEST_P(SleepingTests, TeleportCubeOutOfStack)
	{
		const int32 NumOfStackedObjects = 5;
		const FReal Dimension = 200;
		const bool bPartialSleepEnabled = GetParam();

		const bool bPartialWakePreIntegrationDefault = CVarPartialWakePreIntegration->GetBool();
		CVarPartialWakePreIntegration->Set(true);

		// Create a stack of cubes
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeStackOfCubes(NumOfStackedObjects, /*Size=*/Dimension);

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping();

		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Teleport one of the cubes out of the stack
		Test.ParticleHandles[2]->SetX(FVec3(0, 500, 0), true);

		Test.Advance();

		if (!bPartialSleepEnabled)
		{
			// The entire stack will wake up
			for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
			{
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else
		{
			// With pre-integration partial waking, only one particle below the teleported one wakes up.
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);
		}

		CVarPartialWakePreIntegration->Set(bPartialWakePreIntegrationDefault);
	}

	TEST_P(SleepingTests, TeleportCubeFromOneStackToAnother)
	{
		const int32 NumOfStackedObjects = 5;
		const FReal Dimension = 200;
		const bool bPartialSleepEnabled = true;
		const FRealSingle MomentumPropagation = GetParam() ? 0.5f : 0.0f;

		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(MomentumPropagation);

		const bool bPartialWakePreIntegrationDefault = CVarPartialWakePreIntegration->GetBool();
		CVarPartialWakePreIntegration->Set(true);

		// Create two identical stacks of cubes which don't touch
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeStackOfCubes(NumOfStackedObjects, /*Size=*/Dimension);
		Test.MakeStackOfCubes(NumOfStackedObjects, /*Size=*/Dimension, /*Pos=*/FVec3(0, 500, 0));

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping();

		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Teleport one of the cubes out of the stack into the other stack
		const FVec3 NewX = Test.ParticleHandles[4]->GetX() + FVec3(0, 500, 0);
		Test.ParticleHandles[4]->SetX(NewX, true);

		Test.Advance();

		if (MomentumPropagation > 0.0f)
		{
			// Stack we teleport out of
			// Everything wakes up if momentum transfer is enabled
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);

			// Stack we teleport into
			// Everything wakes up if momentum transfer is enabled
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[6]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[7]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[8]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[9]->ObjectState() == EObjectStateType::Dynamic);
		}
		else // MomentumPropagation == 0.0f
		{
			// Stack we teleport out of
			// Only one particle above the one which is teleported will wake up (level-based waking).
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);

			// Stack we teleport into
			// Only one particle above the one which is teleported into the stack will wake up (level-based waking).
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[6]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[7]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[8]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[9]->ObjectState() == EObjectStateType::Dynamic);
		}

		CVarPartialWakePreIntegration->Set(bPartialWakePreIntegrationDefault);
		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}

	TEST_P(SleepingTests, TeleportCubeFromOneStackToAnotherWithPartialSleepDisallowed)
	{
		const int32 NumOfStackedObjects = 5;
		const FReal Dimension = 200;
		const bool bPartialSleepEnabled = true;
		const bool bPartialIslandSleepAllowed = GetParam();

		const bool bPartialWakePreIntegrationDefault = CVarPartialWakePreIntegration->GetBool();
		CVarPartialWakePreIntegration->Set(true);

		// Create two identical stacks of cubes which don't touch
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeStackOfCubes(NumOfStackedObjects, /*Size=*/Dimension);
		Test.MakeStackOfCubes(NumOfStackedObjects, /*Size=*/Dimension, /*Pos=*/FVec3(0, 500, 0));

		// Limitation: This flag needs to be set at least a step before teleporting the particle
		// since the island flag is only updated in CreateConstraintGraph and already needs to be read in UpdateExplicitSleep.
		Test.ParticleHandles[4]->SetPartialIslandSleepAllowed(bPartialIslandSleepAllowed);

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping();

		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Teleport one of the cubes out of the stack into the other stack
		const FVec3 NewX = Test.ParticleHandles[4]->GetX() + FVec3(0, 500, 0);
		Test.ParticleHandles[4]->SetX(NewX, true);

		Test.Advance();

		if (bPartialIslandSleepAllowed)
		{
			// Stack we teleport out of
			// Only one particle above the one which is teleported will wake up (level-based waking).
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);

			// Stack we teleport into
			// Only one particle above the one which is teleported into the stack will wake up (level-based waking).
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[6]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[7]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[8]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[9]->ObjectState() == EObjectStateType::Dynamic);
		}
		else
		{
			// Stack we teleport out of
			// Everything wakes up since partial island sleeping is deactivated
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);

			// Stack we teleport into
			// Everything wakes up since partial island sleeping is deactivated
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[6]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[7]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[8]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[9]->ObjectState() == EObjectStateType::Dynamic);
		}

		CVarPartialWakePreIntegration->Set(bPartialWakePreIntegrationDefault);
	}

	TEST_P(SleepingTests, TeleportCubeFromOneStackToAnotherWithKinematicObstacle)
	{
		const int32 NumOfStackedObjects = 6;
		const FReal Dimension = 200;
		const bool bPartialSleepEnabled = true;
		const FRealSingle MomentumPropagation = GetParam() ? 0.5f : 0.0f;

		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(MomentumPropagation);

		const bool bPartialWakePreIntegrationDefault = CVarPartialWakePreIntegration->GetBool();
		CVarPartialWakePreIntegration->Set(true);

		// Create two identical stacks of cubes which don't touch
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeStackOfCubes(NumOfStackedObjects, /*Size=*/Dimension);
		Test.MakeStackOfCubes(NumOfStackedObjects, /*Size=*/Dimension, /*Pos=*/FVec3(0, 500, 0));

		// Make the top of the static kinematic for levels not to correspond to horizontal height
		Test.ParticleHandles[5]->SetObjectState(EObjectStateType::Static);
		Test.ParticleHandles[11]->SetObjectState(EObjectStateType::Static);

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping();

		//// Check that everything is sleeping
		//for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		//{
		//	EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		//}

		// Teleport one of the cubes out of the stack into the other stack
		const FVec3 NewX = Test.ParticleHandles[2]->GetX() + FVec3(0, 500, 400);
		Test.ParticleHandles[2]->SetX(NewX, true);

		Test.Advance();

		if (MomentumPropagation > 0.0f)
		{
			// Stack we teleport out of
			// Everything wakes up if momentum transfer is enabled
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Static);

			// Stack we teleport into
			// Everything wakes up if momentum transfer is enabled
			EXPECT_TRUE(Test.ParticleHandles[6]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[7]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[8]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[9]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[10]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[11]->ObjectState() == EObjectStateType::Static);
		}
		else // MomentumPropagation == 0.0f
		{
			// Stack we teleport out of
			// Only one particle above the one which is teleported will wake up (level-based waking).
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Static);

			// Stack we teleport into
			// Only one particle above the one which is teleported into the stack will wake up (level-based waking).
			EXPECT_TRUE(Test.ParticleHandles[6]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[7]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[8]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[9]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[10]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[11]->ObjectState() == EObjectStateType::Static);
		}

		CVarPartialWakePreIntegration->Set(bPartialWakePreIntegrationDefault);
		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}

	// Uses a traditional brick wall where every 2nd row is shifted (see SleepingTests::MakeBrickWall)
	TEST_P(SleepingTests, CollideWithBrickWall)
	{
		const int32 NumOfStackedObjects = 5;
		const FVec3 Dimensions = FVec3(100, 200, 50);
		const bool bPartialSleepEnabled = GetParam();

		// Turn off mommentum propagation to test level-based waking only
		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(0);

		// Create a stack of cubes
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeBrickWall(/*NumY=*/ 5, /*NumZ=*/3, /*Dimensions=*/Dimensions);

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping(/*MaxIterations=*/500);

		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Make the stack collide with a particle located: 
		// - between the 2nd & 3rd row in height
		// - between the 2nd & 3rd column in width
		FRigidBodyHandle_External* CollidingCube = Test.MakeCube(FVec3(50, 300, 75), 50);
		CollidingCube->SetV(FVec3(-200, 0, 0), true);

		Test.Advance();

		//// Debug print of all particle positions
		//for (int32 Id = 0; Id < Test.ParticleHandles.Num(); ++Id)
		//{
		//	FRigidBodyHandle_External* ParticleHandle = Test.ParticleHandles[Id];
		//	printf("Particle Id: %d, State: %d, Position: %f, %f, %f\n", Id, 
		//		ParticleHandle->ObjectState(), ParticleHandle->GetX().X, ParticleHandle->GetX().Y, ParticleHandle->GetX().Z);
		//}

		if (!bPartialSleepEnabled)
		{
			// The entire brick wall will wake up
			for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
			{
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else
		{
			// Only the brick 6 colliding with the spawned cube and the bricks directly above wake up
			// 1st row
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Sleeping);
			// 2nd row
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[6]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[7]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[8]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[9]->ObjectState() == EObjectStateType::Sleeping);
			// 3rd row
			EXPECT_TRUE(Test.ParticleHandles[10]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[11]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[12]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[13]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[14]->ObjectState() == EObjectStateType::Sleeping);
		}

		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}

	// Uses a block of bricks without shifting (see SleepingTests::MakeBrickWall)
	TEST_P(SleepingTests, CollideWithBlockOfBricks)
	{
		const int32 NumOfStackedObjects = 5;
		const FVec3 Dimensions = FVec3(100, 200, 50);
		const bool bPartialSleepEnabled = GetParam();

		// Turn off mommentum propagation to test level-based waking only
		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(0);

		// Create a stack of cubes
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeBrickWall(/*NumY=*/ 5, /*NumZ=*/3, /*Dimensions=*/Dimensions, /*Mass=*/2, /*bShiftUnevenRows*/false);

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping(/*MaxIterations=*/500);

		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Make the stack collide with a particle located: 
		// - between the 2nd & 3rd row in height
		// - between the 2nd & 3rd column in width
		FRigidBodyHandle_External* CollidingCube = Test.MakeCube(FVec3(50, 300, 75), 50);
		CollidingCube->SetV(FVec3(-200, 0, 0), true);

		Test.Advance();

		//// Debug print of all particle positions
		//for (int32 Id = 0; Id < Test.ParticleHandles.Num(); ++Id)
		//{
		//	FRigidBodyHandle_External* ParticleHandle = Test.ParticleHandles[Id];
		//	printf("Particle Id: %d, State: %d, Position: %f, %f, %f\n", Id, 
		//		ParticleHandle->ObjectState(), ParticleHandle->GetX().X, ParticleHandle->GetX().Y, ParticleHandle->GetX().Z);
		//}

		if (!bPartialSleepEnabled)
		{
			// The entire brick wall will wake up
			for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
			{
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else
		{
			// Only the bricks 6 & 7 colliding with the spawned cube and the bricks directly above wake up
			// 1st row
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Sleeping);
			// 2nd row
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[6]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[7]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[8]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[9]->ObjectState() == EObjectStateType::Sleeping);
			// 3rd row
			EXPECT_TRUE(Test.ParticleHandles[10]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[11]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[12]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[13]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[14]->ObjectState() == EObjectStateType::Sleeping);
		}

		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}

	TEST_P(SleepingTests, CollideWithQueueOfCubes)
	{
		const int32 NumOfStackedObjects = 5;
		const FReal Dimension = 200;
		const bool bPartialSleepEnabled = true;
		const bool bAllowPartialSleep = GetParam();

		// Turn off mommentum propagation to test level-based waking only
		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(0);

		// Create a stack of cubes
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeQueueOfCubes(NumOfStackedObjects, /*Size=*/Dimension);

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping();

		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Create another particle which will collide with the queue
		FRigidBodyHandle_External* CollidingCube = Test.MakeCube(FVec3(0, -200, 100), 200);
		CollidingCube->SetV(FVec3(0, 200, 0), true);
		CollidingCube->SetPartialIslandSleepAllowed(bAllowPartialSleep);

		Test.Advance();

		//// Debug print of all particle positions
		//for (int32 Id = 0; Id < Test.ParticleHandles.Num(); ++Id)
		//{
		//	FRigidBodyHandle_External* ParticleHandle = Test.ParticleHandles[Id];
		//	printf("Particle Id: %d, State: %d, Position: %f, %f, %f\n", Id,
		//		ParticleHandle->ObjectState(), ParticleHandle->GetX().X, ParticleHandle->GetX().Y, ParticleHandle->GetX().Z);
		//}

		if (!bAllowPartialSleep)
		{
			// Check that everything is awake
			for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
			{
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else
		{
			// Only the colliding cube and first particle in the queue wake up
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Sleeping);
		}

		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}

	TEST_P(SleepingTests, PropagateMomentumThroughQueue)
	{
		const int32 NumOfStackedObjects = 5;
		const FReal Dimension = 200;
		const bool bPartialSleepEnabled = true;
		const FRealSingle MomentumPropagation = GetParam() ? 0.5f : 0.1f;
		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(MomentumPropagation);
		// Create a stack of cubes
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeQueueOfCubes(NumOfStackedObjects, /*Size=*/Dimension);
		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping();
		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}
		// Create another particle which will collide with the queue
		FRigidBodyHandle_External* CollidingCube = Test.MakeCube(FVec3(0, -200, 100), 200);
		// Note: We use the VSmooth for momentum propagation (0.3 * v_old + 0.7 * v_new)
		// Thus, only 30% of the velocity are actually transmitted.
		CollidingCube->SetLinearImpulse(FVec3(0, 200, 0), true);
		Test.Advance();
		//// Debug print of all particle positions
		//for (int32 Id = 0; Id < Test.ParticleHandles.Num(); ++Id)
		//{
		//	FRigidBodyHandle_External* ParticleHandle = Test.ParticleHandles[Id];
		//	printf("Particle Id: %d, State: %d, Position: %f, %f, %f\n", Id,
		//		ParticleHandle->ObjectState(), ParticleHandle->GetX().X, ParticleHandle->GetX().Y, ParticleHandle->GetX().Z);
		//}
		if (GetParam()) // 50% propagation
		{
			// Check that everything is awake
			for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
			{
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else // 10% propagation
		{
			// Only the colliding cube and first two particles in the queue wake up
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Sleeping);
		}
		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}
	// Uses a block of bricks without shifting (see SleepingTests::MakeBrickWall)
	TEST_P(SleepingTests, PropagateMomentumThroughBlockOfBricks)
	{
		const int32 NumOfStackedObjects = 5;
		const FVec3 Dimensions = FVec3(100, 200, 50);
		const bool bPartialSleepEnabled = true;
		const FRealSingle MomentumPropagation = GetParam() ? 0.5f : 0.1f;
		const FRealSingle MomentumPropagationDefault = CVarMomentumPropagation->GetFloat();
		CVarMomentumPropagation->Set(MomentumPropagation);
		// Create a stack of cubes
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeBrickWall(/*NumY=*/ 5, /*NumZ=*/3, /*Dimensions=*/Dimensions, /*Mass=*/2, /*bShiftUnevenRows*/false);
		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping(/*MaxIterations=*/500);
		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}
		// Cube spawned about particles 11 & 12 (i.e. the 2nd and 3rd of the top row)
		FRigidBodyHandle_External* CollidingCube = Test.MakeCube(FVec3(0, 300, 150), 50);
		// Note: We use the VSmooth for momentum propagation (0.3 * v_old + 0.7 * v_new)
		// Thus, only 30% of the velocity are actually transmitted.
		CollidingCube->SetLinearImpulse(FVec3(0, 0, -200), true);
		Test.Advance();
		//// Debug print of all particle positions
		//for (int32 Id = 0; Id < Test.ParticleHandles.Num(); ++Id)
		//{
		//	FRigidBodyHandle_External* ParticleHandle = Test.ParticleHandles[Id];
		//	printf("Particle Id: %d, State: %d, Position: %f, %f, %f\n", Id, 
		//		ParticleHandle->ObjectState(), ParticleHandle->GetX().X, ParticleHandle->GetX().Y, ParticleHandle->GetX().Z);
		//}
		if (GetParam()) // 50% propagation
		{
			// The entire brick wall will wake up
			for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
			{
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else // 10% propagation
		{
			// Colliding with particles 11 & 12
			// Only particles with constraint distance <= 2 wake up.
			// 1st row
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Sleeping);
			// 2nd row
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[6]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[7]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[8]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[9]->ObjectState() == EObjectStateType::Sleeping);
			// 3rd row
			EXPECT_TRUE(Test.ParticleHandles[10]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[11]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[12]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[13]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[14]->ObjectState() == EObjectStateType::Dynamic);
		}
		CVarMomentumPropagation->Set(MomentumPropagationDefault);
	}

	TEST_P(SleepingTests, PostStepWaking)
	{
		const int32 NumOfStackedObjects = 5;
		const FReal Dimension = 200;
		const bool bPartialSleepEnabled = true;
		const FRealSingle PostStepWakeThreshold = GetParam() ? 10.0f : 0.0f;

		const FRealSingle PostStepWakeThresholdDefault = CVarPostStepWake->GetFloat();
		CVarPostStepWake->Set(PostStepWakeThreshold);

		// Create a stack of cubes
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeQueueOfCubes(NumOfStackedObjects, /*Size=*/Dimension);

		// Simulate until all particles are sleeping
		Test.AdvanceUntilSleeping();

		// Check that everything is sleeping
		for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
		{
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}

		// Create another particle which will collide with the queue
		FRigidBodyHandle_External* CollidingCube = Test.MakeCube(FVec3(0, -200, 100), 200);

		for (int32 Step = 0; Step < 10; ++Step)
		{
			CollidingCube->SetLinearImpulse(FVec3(0, 1000, 0), true);
			Test.Advance();

			//// Debug print of all particle positions
			//for (int32 Id = 0; Id < Test.ParticleHandles.Num(); ++Id)
			//{
			//	FRigidBodyHandle_External* ParticleHandle = Test.ParticleHandles[Id];
			//	printf("Particle Id: %d, State: %d, Position: %f, %f, %f\n", Id,
			//		ParticleHandle->ObjectState(), ParticleHandle->GetX().X, ParticleHandle->GetX().Y, ParticleHandle->GetX().Z);
			//}
		}

		if (GetParam()) // post-step waking enabled
		{
			// Check that everything is awake
			for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
			{
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else // post-step waking disabled
		{
			// Only the colliding cube and first particle in the queue wake up
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Sleeping);
		}

		CVarPostStepWake->Set(PostStepWakeThresholdDefault);
	}

	// Sleep all particles except one which are connected by joint constraints.
	// Since joint constraints don't support partial sleeping, the entire island wakes up.
	TEST_P(SleepingTests, PartiallySleepingIslandWithJoints)
	{
		const int32 PartialSleepCollisionConstraintsOnly = GetParam() ? 1 : 0;
		const int32 PartialSleepCollisionConstraintsOnlyDefault = CVarPartialSleepCollisionConstraintsOnly->GetInt();
		CVarPartialSleepCollisionConstraintsOnly->Set(PartialSleepCollisionConstraintsOnly);

		const bool bPartialSleepEnabled = true;
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeChainOfSpheres(5);

		Test.ParticleHandles[0]->SetObjectState(EObjectStateType::Static);
		Test.ParticleHandles[1]->SetObjectState(EObjectStateType::Sleeping);
		Test.ParticleHandles[2]->SetObjectState(EObjectStateType::Sleeping);
		Test.ParticleHandles[3]->SetObjectState(EObjectStateType::Sleeping);
		Test.ParticleHandles[4]->SetObjectState(EObjectStateType::Dynamic);

		Test.Advance();

		if (PartialSleepCollisionConstraintsOnly == 1)
		{
			// All particles wake up since the connecting constraints don't support partial sleeping
			for (int32 Id = 1; Id < Test.ParticleHandles.Num(); ++Id)
			{
				const FRigidBodyHandle_External* ParticleHandle = Test.ParticleHandles[Id];
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else if (PartialSleepCollisionConstraintsOnly == 0)
		{
			// Partial sleep supported, no state change
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Static);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);
		}
		CVarPartialSleepCollisionConstraintsOnly->Set(PartialSleepCollisionConstraintsOnlyDefault);
	}

	// Same as PartiallySleepingIslandWithJoints, just assign a different particle to be awake.
	TEST_P(SleepingTests, PartiallySleepingIslandWithJoints_2)
	{
		const int32 PartialSleepCollisionConstraintsOnly = GetParam() ? 1 : 0;
		const int32 PartialSleepCollisionConstraintsOnlyDefault = CVarPartialSleepCollisionConstraintsOnly->GetInt();
		CVarPartialSleepCollisionConstraintsOnly->Set(PartialSleepCollisionConstraintsOnly);

		const bool bPartialSleepEnabled = true;
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeChainOfSpheres(5);

		Test.ParticleHandles[0]->SetObjectState(EObjectStateType::Static);
		Test.ParticleHandles[1]->SetObjectState(EObjectStateType::Sleeping);
		Test.ParticleHandles[2]->SetObjectState(EObjectStateType::Dynamic);
		Test.ParticleHandles[3]->SetObjectState(EObjectStateType::Sleeping);
		Test.ParticleHandles[4]->SetObjectState(EObjectStateType::Sleeping);

		Test.Advance();

		if (PartialSleepCollisionConstraintsOnly == 1)
		{
			// All particles wake up since the connecting constraints don't support partial sleeping
			for (int32 Id = 1; Id < Test.ParticleHandles.Num(); ++Id)
			{
				const FRigidBodyHandle_External* ParticleHandle = Test.ParticleHandles[Id];
				EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Dynamic);
			}
		}
		else if (PartialSleepCollisionConstraintsOnly == 0)
		{
			// Partial sleep supported, no state change
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Static);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Sleeping);
		}
		CVarPartialSleepCollisionConstraintsOnly->Set(PartialSleepCollisionConstraintsOnlyDefault);
	}

	// TODO: Test with CVars::bChaosSolverPartialSleepCollisionConstraintsOnly == true
	// Sleep all particles except one which are connected by joint constraints.
	// Since all constraints already sleep, the entire island remains asleep.
	TEST_P(SleepingTests, FullySleepingIslandWithJoints)
	{
		const int32 PartialSleepCollisionConstraintsOnly = GetParam() ? 1 : 0;
		const int32 PartialSleepCollisionConstraintsOnlyDefault = CVarPartialSleepCollisionConstraintsOnly->GetInt();
		CVarPartialSleepCollisionConstraintsOnly->Set(PartialSleepCollisionConstraintsOnly);

		const bool bPartialSleepEnabled = true;
		FSleepingTest Test(bPartialSleepEnabled);
		Test.MakeChainOfSpheres(5);

		Test.ParticleHandles[0]->SetObjectState(EObjectStateType::Static);
		Test.ParticleHandles[1]->SetObjectState(EObjectStateType::Sleeping);
		Test.ParticleHandles[2]->SetObjectState(EObjectStateType::Sleeping);
		Test.ParticleHandles[3]->SetObjectState(EObjectStateType::Sleeping);
		Test.ParticleHandles[4]->SetObjectState(EObjectStateType::Sleeping);

		Test.Advance();

		// All particles remain asleep since the entire island is asleep
		for (int32 Id = 1; Id < Test.ParticleHandles.Num(); ++Id)
		{
			const FRigidBodyHandle_External* ParticleHandle = Test.ParticleHandles[Id];
			EXPECT_TRUE(ParticleHandle->ObjectState() == EObjectStateType::Sleeping);
		}
		CVarPartialSleepCollisionConstraintsOnly->Set(PartialSleepCollisionConstraintsOnlyDefault);
	}

	TEST_P(SleepingTests, StackOfCubesNeverSleep)
	{
		const bool bPartialSleeping = GetParam();
		FSleepingTest Test(bPartialSleeping);

		// This test builds the stack top-down to reverse the processing order SleepNonMovingParticles() vs level information.
		// If we build the stack bottom-up, we always put particles to sleep bottom-up. 
		// This hides issues with sleeping particles on top of particles which cannot sleep.
		const int32 Num = 6;
		const FReal Size = 200;
		const FVec3 Pos = FVec3(0);
		const FReal InitialHeight = Size / 2.0f;
		for (int32 Id = Num - 1; Id >= 0; --Id)
		{
			const FReal Z = InitialHeight + Size * Id;
			Test.MakeCube(Pos + FVec3(0, 0, Z), Size);
		}
		Test.MakeFloor();

		// Tick once to create constraints
		Test.Advance();

		// The SingleParticlePhysicsProxy will automatically update sleep thresholds when calling SetSleepType.
		Test.ParticleHandles[2]->SetSleepType(ESleepType::NeverSleep);

		for (int32 Tick = 0; Tick < 200; ++Tick)
		{
			Test.Advance();
		}

		if (bPartialSleeping)
		{
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Sleeping);
		}
		else
		{
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Dynamic);
		}
	}

	TEST_P(SleepingTests, StackOfCubesZeroSleepThreshold)
	{
		const bool bPartialSleeping = GetParam();
		FSleepingTest Test(bPartialSleeping);

		// Build the stack from top to bottom so that the island manager processes sleeping from highest to lowest level.
		// Inject one cube into the stack that has zero sleep threshold.
		// All levels above should remain awake.
		const FReal Size = 200;
		const int32 Num = 6;
		Test.MakeFloor();
		const FReal InitialHeight = Size / 2.0f;
		for (int32 Id = Num - 1; Id >= 0; --Id)
		{
			// ParticleHandles[2] cannot sleep (3rd cube from bottom starting from 0)
			const FMaterialData* OverrideMaterialData = (Id == 3) ? &Test.NoSleepMaterialData : nullptr;
			const FReal Z = InitialHeight + Size * Id;
			Test.MakeCube(FVec3(0, 0, Z), Size, 1, OverrideMaterialData);
		}

		for (int32 Tick = 0; Tick < 200; ++Tick)
		{
			Test.Advance();
		}

		if (bPartialSleeping)
		{
			// ParticleHandles[2] keeps all cubes above and one cube below awake
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Sleeping);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Dynamic);
		}
		else
		{
			EXPECT_TRUE(Test.ParticleHandles[0]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[1]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[2]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[3]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[4]->ObjectState() == EObjectStateType::Dynamic);
			EXPECT_TRUE(Test.ParticleHandles[5]->ObjectState() == EObjectStateType::Dynamic);
		}
	}

	INSTANTIATE_TEST_SUITE_P(, SleepingTests, testing::Bool());
}