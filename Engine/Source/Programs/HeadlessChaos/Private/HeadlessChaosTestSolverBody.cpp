// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/GJK.h"
#include "Chaos/Pair.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"

namespace ChaosTest
{
	using namespace Chaos;

	// The implicit velocity of a body is calculated correctly
	TEST(FSolverBodyTests, TestImplicitVelocity)
	{
		const FReal Dt = 1.0f / 30.0f;
		const FVec3 X0 = FVec3(0, 0, 0);
		const FVec3 V0 = FVec3(100, 10, 30);
		const FReal M0 = 100.0f;
		const FReal I0 = M0 * 1000.0f;

		FSolverBody SolverBody0 = FSolverBody::MakeInitialized();
		SolverBody0.SetX(X0);
		SolverBody0.SetP(X0);
		SolverBody0.SetInvM(1.0f / M0);
		SolverBody0.SetInvI(1.0f / I0);

		SolverBody0.ApplyPositionDelta(V0 * Dt);

		SolverBody0.SetImplicitVelocity(Dt);

		EXPECT_NEAR(SolverBody0.V().X, V0.X, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(SolverBody0.V().Y, V0.Y, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(SolverBody0.V().Z, V0.Z, KINDA_SMALL_NUMBER);
	}

	// Test that correction does not affect the implicit velocity
	TEST(FSolverBodyTests, TestCorrection)
	{
		const FReal Dt = 0.1f;
		const FVec3 X0 = FVec3(10, 0, 0);
		const FVec3 V0 = FVec3(10, 20, 30);
		const FVec3 W0 = FVec3(10, 0, -10);
		const FRotation3 R0 = FRotation3::FromVector(FVec3(1.0f, -2.0f, 3.0f));
		const FReal M0 = 100.0f;
		const FReal I0 = M0 * 1000.0f;

		const FSolverVec3 PositionDelta = FSolverVec3(10, 0, 0);
		const FSolverVec3 PositionCorrectionDelta = FSolverVec3(5, 0, 5);

		const FSolverVec3 RotationDelta = FSolverVec3(0, 10, 0);
		const FSolverVec3 RotationCorrectionDelta = FSolverVec3(0, 5, 0);

		FSolverBody SolverBody = FSolverBody::MakeInitialized();
		SolverBody.SetX(X0);
		SolverBody.SetR(R0);
		SolverBody.SetV(V0);
		SolverBody.SetW(W0);
		SolverBody.SetP(X0);
		SolverBody.SetQ(R0);
		SolverBody.SetInvM(1.0f / M0);
		SolverBody.SetInvI(1.0f / I0);

		SolverBody.ApplyPositionDelta(PositionDelta);
		SolverBody.ApplyPositionCorrectionDelta(PositionCorrectionDelta);

		SolverBody.ApplyRotationDelta(RotationDelta);
		SolverBody.ApplyRotationCorrectionDelta(RotationCorrectionDelta);

		const FSolverVec3 ExpectedDP = PositionDelta + PositionCorrectionDelta;
		const FSolverVec3 ExpectedCP = PositionCorrectionDelta;
		const FSolverVec3 ExpectedDQ = RotationDelta + RotationCorrectionDelta;
		const FSolverVec3 ExpectedCQ = RotationCorrectionDelta;
		EXPECT_VECTOR_FLOAT_EQ(SolverBody.DP(), ExpectedDP);
		EXPECT_VECTOR_FLOAT_EQ(SolverBody.CP(), ExpectedCP);
		EXPECT_VECTOR_FLOAT_EQ(SolverBody.DQ(), ExpectedDQ);
		EXPECT_VECTOR_FLOAT_EQ(SolverBody.CQ(), ExpectedCQ);

		SolverBody.SetImplicitVelocity(Dt);

		const FVec3 ExpectedVelocity = V0 + FVec3(PositionDelta / Dt);
		const FVec3 ExpectedAngularVelocity = W0 + FVec3(RotationDelta / Dt);
		EXPECT_VECTOR_FLOAT_EQ(SolverBody.V(), ExpectedVelocity);
		EXPECT_VECTOR_FLOAT_EQ(SolverBody.W(), ExpectedAngularVelocity);

		SolverBody.ApplyCorrections();

		const FVec3 ExpectedPosition = X0 + FVec3(PositionDelta + PositionCorrectionDelta);
		const FRotation3 ExpectedRotation = FRotation3::IntegrateRotationWithAngularVelocity(R0, RotationDelta + RotationCorrectionDelta, 1.0);
		EXPECT_VECTOR_FLOAT_EQ(SolverBody.P(), ExpectedPosition);
		EXPECT_FLOAT_EQ(ExpectedRotation.W, SolverBody.Q().W);
		EXPECT_FLOAT_EQ(ExpectedRotation.X, SolverBody.Q().X);
		EXPECT_FLOAT_EQ(ExpectedRotation.Y, SolverBody.Q().Y);
		EXPECT_FLOAT_EQ(ExpectedRotation.Z, SolverBody.Q().Z);
	}

	class FRigidSOAsTest : public ::testing::Test
	{
	protected:
		FRigidSOAsTest()
			: RigidSOAs(UniqueIndices)
		{
			PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
			PhysicsMaterial->Friction = FReal(0);
			PhysicsMaterial->Restitution = FReal(0);

			RigidSOAs.GetParticleHandles().AddArray(&Collided);
			RigidSOAs.GetParticleHandles().AddArray(&PhysicsMaterials);
			RigidSOAs.GetParticleHandles().AddArray(&PerParticlePhysicsMaterials);

		}

		~FRigidSOAsTest()
		{
		}

		FPBDRigidParticleHandle* CreateDynamicBox(FReal Radius)
		{
			return AppendDynamicParticleSphere(RigidSOAs, FVec3(Radius));
		}

		TArrayCollectionArray<bool> Collided;
		TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> PhysicsMaterials;
		TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> PerParticlePhysicsMaterials;
		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial;
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs RigidSOAs;
	};

	class FSolverBodyTest : public FRigidSOAsTest
	{
	protected:
		FSolverBodyTest()
		{
		}

		~FSolverBodyTest()
		{
		}

		FSolverBodyContainer SolverBodyContainer;
	};

	TEST_F(FSolverBodyTest, TestAddOrFind)
	{
		TArray<FPBDRigidParticleHandle*> Particles =
		{
			CreateDynamicBox(50.0f)
		};

		SolverBodyContainer.Reset(1);

		SolverBodyContainer.FindOrAdd(Particles[0]);
	}
}
