// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Chaos/Island/IslandManager.h"
#include "Chaos/PBDRigidsSOAs.h"
#include "Chaos/Evolution/ConstraintGroupSolver.h"
#include "Chaos/Evolution/IterationSettings.h"

#include "ChaosSolversModule.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/JointConstraintProxy.h"
#include "Chaos/PBDJointConstraintData.h"
#include "HAL/ConsoleManager.h"
#include "PhysicsInterfaceDeclaresCore.h"

namespace ChaosTest
{
	using namespace Chaos;

	class SolverPartitionTest : public ::testing::Test
	{
	protected:
		SolverPartitionTest()
			: SOAs(UniqueIndices)
		{
			Solver = Container.CreateSceneSolver(0);
		}

		void SetUp()
		{
			CVarGraphColoring = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.PartitionManager.Partitioning"), false);
			check(CVarGraphColoring && CVarGraphColoring->IsVariableInt());
			GraphColoring_EditorDefault = CVarGraphColoring->GetInt();
			CVarGraphColoring->Set(GraphColoring_UnitTestDefault);

			CVarBatchSize = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.PartitionManager.MinBatchSize"), false);
			check(CVarBatchSize && CVarBatchSize->IsVariableInt());
			BatchSize_EditorDefault = CVarBatchSize->GetInt();
			CVarBatchSize->Set(BatchSize_UnitTestDefault);

			CVarMinConstraintsForColoring = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.PartitionManager.MinConstraintsForColoring"), false);
			check(CVarMinConstraintsForColoring && CVarMinConstraintsForColoring->IsVariableInt());
			MinConstraintsForColoring_EditorDefault = CVarMinConstraintsForColoring->GetInt();
			CVarMinConstraintsForColoring->Set(MinConstraintsForColoring_UnitTestDefault);

			// Make sure the hard-coded default for partitioning is off (0).
			EXPECT_EQ(GraphColoring_EditorDefault, 0);
		}

		void TearDown()
		{
			CVarGraphColoring->Set(GraphColoring_EditorDefault);
			CVarBatchSize->Set(BatchSize_EditorDefault);
			CVarMinConstraintsForColoring->Set(MinConstraintsForColoring_EditorDefault);
		}

		// Default values for unit testing
		int32 GraphColoring_UnitTestDefault = 1;
		int32 BatchSize_UnitTestDefault = 1;
		int32 MinConstraintsForColoring_UnitTestDefault = 1;

		// Storage for default values hard-coded in SolverPartitionManager.cpp
		int32 GraphColoring_EditorDefault;
		int32 BatchSize_EditorDefault;
		int32 MinConstraintsForColoring_EditorDefault;

		// Storage of console variable pointer
		IConsoleVariable* CVarGraphColoring;
		IConsoleVariable* CVarBatchSize;
		IConsoleVariable* CVarMinConstraintsForColoring;

		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs SOAs;
		FCharacterGroundConstraintContainer Container;
		TArray<FCharacterGroundConstraintSettings> Settings;
		TArray<FCharacterGroundConstraintDynamicData> Data;
		Private::FIterationSettings IterationSettings;
		TUniquePtr<FConstraintContainerSolver> Solver;
	};

	// Test coloring of constraint graph
	// Test 1
	//	Ak - Bd - Cd - Dd - Ed
	//	   0    1    0    1
	TEST_F(SolverPartitionTest, GraphColoring_Chain)
	{
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(5);
		Settings.SetNum(1);
		Data.SetNum(1);

		TArray<FCharacterGroundConstraintHandle*> Constraints;
		for (int32 ParticleIndex = 0; ParticleIndex < Particles.Num() - 1; ++ParticleIndex)
		{
			Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[ParticleIndex], Particles[ParticleIndex + 1]));
		}

		Solver->AddConstraints();
		EXPECT_EQ(Solver->GetNumConstraints(), Constraints.Num());

		PRAGMA_DISABLE_INTERNAL_WARNINGS
		const int32 MaxSerialBatches = Solver->PrepareSolverPartitions();
		const TArray<TArray<int32>>& SerialIndices = Solver->GetConstraintsBatchIndices();
		PRAGMA_ENABLE_INTERNAL_WARNINGS

		EXPECT_EQ(SerialIndices.Num(), 2);
		EXPECT_EQ(Solver->GetNumConstraints(), 4);

		if (SerialIndices.Num() > 0 && SerialIndices[0].Num() > 0)
			EXPECT_EQ(SerialIndices[0][0], 0);
		if (SerialIndices.Num() > 1 && SerialIndices[1].Num() > 0)
			EXPECT_EQ(SerialIndices[1][0], 2);
	}

	// Test coloring of constraint graph
	// Test 1
	//				    Gd
	//				 0/	  \3
	//		        Ed     Fd
	//            1/ \2	 1/ \2
	//            Bd   Cd   Dd
	//            0\   |0   /0
	//                 Ak
	TEST_F(SolverPartitionTest, GraphColoring_WallWithSpacing)
	{
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(7);
		Settings.SetNum(1);
		Data.SetNum(1);

		TArray<FCharacterGroundConstraintHandle*> Constraints;

		Particles[0]->SetObjectStateLowLevel(EObjectStateType::Kinematic);

		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[1]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[2]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[3]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[1], Particles[4]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[2], Particles[4]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[2], Particles[5]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[3], Particles[5]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[4], Particles[6]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[5], Particles[6]));

		Solver->AddConstraints();
		EXPECT_EQ(Solver->GetNumConstraints(), Constraints.Num());

		PRAGMA_DISABLE_INTERNAL_WARNINGS
		const int32 MaxSerialBatches = Solver->PrepareSolverPartitions();
		const TArray<TArray<int32>>& SerialIndices = Solver->GetConstraintsBatchIndices();
		PRAGMA_ENABLE_INTERNAL_WARNINGS

		EXPECT_EQ(SerialIndices.Num(), 4);
		EXPECT_EQ(Solver->GetNumConstraints(), 9);

		if (SerialIndices.Num() > 0 && SerialIndices[0].Num() > 0)
			EXPECT_EQ(SerialIndices[0][0], 0);
		if (SerialIndices.Num() > 1 && SerialIndices[1].Num() > 0)
			EXPECT_EQ(SerialIndices[1][0], 4);
		if (SerialIndices.Num() > 2 && SerialIndices[2].Num() > 0)
			EXPECT_EQ(SerialIndices[2][0], 6);
		if (SerialIndices.Num() > 3 && SerialIndices[3].Num() > 0)
			EXPECT_EQ(SerialIndices[3][0], 8);
	}

	// Test coloring of constraint graph
	// Test 1
	//				    Gd
	//				 1/	  \2
	//		        Ed -0  Fd
	//            2/  \3 4/  \1
	//            Bd -1 Cd -2 Dd
	//            0\   |0   /0
	//                 Ak
	TEST_F(SolverPartitionTest, GraphColoring_WallWithoutSpacing)
	{
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(7);
		Settings.SetNum(1);
		Data.SetNum(1);

		TArray<FCharacterGroundConstraintHandle*> Constraints;

		Particles[0]->SetObjectStateLowLevel(EObjectStateType::Kinematic);

		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[1]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[2]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[3]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[1], Particles[2]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[2], Particles[3]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[1], Particles[4]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[2], Particles[4]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[2], Particles[5]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[3], Particles[5]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[4], Particles[5]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[4], Particles[6]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[5], Particles[6]));

		Solver->AddConstraints();
		EXPECT_EQ(Solver->GetNumConstraints(), Constraints.Num());

		PRAGMA_DISABLE_INTERNAL_WARNINGS
		const int32 MaxSerialBatches = Solver->PrepareSolverPartitions();
		const TArray<TArray<int32>>& SerialIndices = Solver->GetConstraintsBatchIndices();
		PRAGMA_ENABLE_INTERNAL_WARNINGS

		EXPECT_EQ(SerialIndices.Num(), 5);
		EXPECT_EQ(Solver->GetNumConstraints(), 12);

		if (SerialIndices.Num() > 0 && SerialIndices[0].Num() > 0)
			EXPECT_EQ(SerialIndices[0][0], 0);
		if (SerialIndices.Num() > 1 && SerialIndices[1].Num() > 0)
			EXPECT_EQ(SerialIndices[1][0], 4);
		if (SerialIndices.Num() > 2 && SerialIndices[2].Num() > 0)
			EXPECT_EQ(SerialIndices[2][0], 7);
		if (SerialIndices.Num() > 3 && SerialIndices[3].Num() > 0)
			EXPECT_EQ(SerialIndices[3][0], 10);
		if (SerialIndices.Num() > 4 && SerialIndices[4].Num() > 0)
			EXPECT_EQ(SerialIndices[4][0], 11);
	}

	// Test coloring of constraint graph
	// Test 1
	//			 Hd 0- Kd 2- Ld
	//			  1|   1|   0|
	//		     Ed 0- Fd 2- Gd
	//            2|   3|   1|
	//           Bd 1- Cd 2- Dd
	//             0\  0|   /0
	//                 Ak
	//
	TEST_F(SolverPartitionTest, GraphColoring_BlockWithoutSpacing)
	{
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(10);
		Settings.SetNum(1);
		Data.SetNum(1);

		TArray<FCharacterGroundConstraintHandle*> Constraints;

		Particles[0]->SetObjectStateLowLevel(EObjectStateType::Kinematic);

		// Constraints with the ground
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[1]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[2]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[3]));
		// Horizontal constraints
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[1], Particles[2]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[2], Particles[3]));
		// Vertical constraints
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[1], Particles[4]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[2], Particles[5]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[3], Particles[6]));
		// Horizontal constraints
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[4], Particles[5]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[5], Particles[6]));
		// Vertical constraints
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[4], Particles[7]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[5], Particles[8]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[6], Particles[9]));
		// Horizontal constraints
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[7], Particles[8]));
		Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[8], Particles[9]));

		Solver->AddConstraints();
		EXPECT_EQ(Solver->GetNumConstraints(), Constraints.Num());

		PRAGMA_DISABLE_INTERNAL_WARNINGS
		const int32 MaxSerialBatches = Solver->PrepareSolverPartitions();
		const TArray<TArray<int32>> SerialIndices = Solver->GetConstraintsBatchIndices();
		PRAGMA_ENABLE_INTERNAL_WARNINGS

		EXPECT_EQ(SerialIndices.Num(), 4);
		EXPECT_EQ(Solver->GetNumConstraints(), 15);

		if (SerialIndices.Num() > 0 && SerialIndices[0].Num() > 0)
			EXPECT_EQ(SerialIndices[0][0], 0);
		if (SerialIndices.Num() > 1 && SerialIndices[1].Num() > 0)
			EXPECT_EQ(SerialIndices[1][0], 6);
		if (SerialIndices.Num() > 2 && SerialIndices[2].Num() > 0)
			EXPECT_EQ(SerialIndices[2][0], 10);
		if (SerialIndices.Num() > 3 && SerialIndices[3].Num() > 0)
			EXPECT_EQ(SerialIndices[3][0], 14);
	}

	TEST_F(SolverPartitionTest, GraphColoring_Exceed64ConstraintsPerParticle)
	{
		constexpr int32 NumParticles = 80;
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(NumParticles);
		Settings.SetNum(1);
		Data.SetNum(1);

		TArray<FCharacterGroundConstraintHandle*> Constraints;

		// Particles[0] is connected to all other particles
		for (int PId = 1; PId < NumParticles; ++PId)
		{
			Constraints.Add(Container.AddConstraint(Settings[0], Data[0], Particles[0], Particles[PId]));
		}

		Solver->AddConstraints();
		EXPECT_EQ(Solver->GetNumConstraints(), Constraints.Num());

		PRAGMA_DISABLE_INTERNAL_WARNINGS
		const int32 MaxSerialBatches = Solver->PrepareSolverPartitions();
		const TArray<TArray<int32>> SerialIndices = Solver->GetConstraintsBatchIndices();
		PRAGMA_ENABLE_INTERNAL_WARNINGS

		// We've added 79 constraints between Particles[0] and all other particles.
		// This exceeds the limit of parallel colors supported
		// There should 65 serial indices, the last one is the overflow group.
		EXPECT_EQ(SerialIndices.Num(), 65);
		EXPECT_EQ(Solver->GetNumConstraints(), 79);
		for (int ColorId = 0; ColorId < 65; ++ColorId)
		{
			EXPECT_EQ(SerialIndices[ColorId].Num(), 1);
			EXPECT_EQ(SerialIndices[ColorId][0], ColorId);
		}
	}
}

namespace ChaosTest
{
	using namespace Chaos;

	class PartitioningSolverTest : public ::testing::Test
	{
	public:
		void SetUp()
		{
			CVarParallelSolve = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.PartitionManager.ParallelSolve"), false);
			check(CVarParallelSolve && CVarParallelSolve->IsVariableInt());
			ParallelSolve_EditorDefault = CVarParallelSolve->GetInt();
			CVarParallelSolve->Set(ParallelSolve_UnitTestDefault);

			CVarGraphColoring = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.PartitionManager.Partitioning"), false);
			check(CVarGraphColoring && CVarGraphColoring->IsVariableInt());
			GraphColoring_EditorDefault = CVarGraphColoring->GetInt();
			CVarGraphColoring->Set(GraphColoring_UnitTestDefault);

			CVarBatchSize = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.PartitionManager.MinBatchSize"), false);
			check(CVarBatchSize && CVarBatchSize->IsVariableInt());
			BatchSize_EditorDefault = CVarBatchSize->GetInt();
			CVarBatchSize->Set(BatchSize_UnitTestDefault);

			CVarMinConstraintsForColoring = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Chaos.Solver.PartitionManager.MinConstraintsForColoring"), false);
			check(CVarMinConstraintsForColoring && CVarMinConstraintsForColoring->IsVariableInt());
			MinConstraintsForColoring_EditorDefault = CVarMinConstraintsForColoring->GetInt();
			CVarMinConstraintsForColoring->Set(MinConstraintsForColoring_UnitTestDefault);			
			
			// Make sure the hard-coded default for partitioning is off (0) and parallel solver tied to partitioning (-1).
			EXPECT_EQ(GraphColoring_EditorDefault, 0);
			EXPECT_EQ(ParallelSolve_EditorDefault, -1);
		}

		void TearDown()
		{
			CVarParallelSolve->Set(ParallelSolve_EditorDefault);
			CVarGraphColoring->Set(GraphColoring_EditorDefault);
			CVarBatchSize->Set(BatchSize_EditorDefault);
			CVarMinConstraintsForColoring->Set(MinConstraintsForColoring_EditorDefault);
		}

		// Default values for unit testing
		int32 ParallelSolve_UnitTestDefault = 1;
		int32 GraphColoring_UnitTestDefault = 1;
		int32 BatchSize_UnitTestDefault = 1;
		int32 MinConstraintsForColoring_UnitTestDefault = 1;

		// Storage for default values hard-coded in SolverPartitionManager.cpp
		int32 ParallelSolve_EditorDefault;
		int32 GraphColoring_EditorDefault;
		int32 BatchSize_EditorDefault;
		int32 MinConstraintsForColoring_EditorDefault;

		// Storage of console variable pointer
		IConsoleVariable* CVarParallelSolve;
		IConsoleVariable* CVarGraphColoring;
		IConsoleVariable* CVarBatchSize;
		IConsoleVariable* CVarMinConstraintsForColoring;


	};

	class FPartitioningSolverTest
	{
	public:
		FPartitioningSolverTest()
			: MaterialData()
			, FloorProxy()
			, ParticleProxies()
			, ParticleHandles()
			, JointHandles()
			, TickCount(0)
		{
			// Create a new material before creating the solver
			FMaterialHandle MaterialHandle = FPhysicalMaterialManager::Get().Create();
			FPhysicsMaterial* PhysicsMaterial = MaterialHandle.Get();
			PhysicsMaterial->Friction = 0.7;
			PhysicsMaterial->StaticFriction = 0.0;
			PhysicsMaterial->Restitution = 0.0;
			PhysicsMaterial->Density = 1.0;
			PhysicsMaterial->SleepingLinearThreshold = 1.0;
			PhysicsMaterial->SleepingAngularThreshold = 0.05;
			PhysicsMaterial->SleepCounterThreshold = 4;
			MaterialData.Materials.Add(MaterialHandle);

			Module = FChaosSolversModule::GetModule();
			Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);
			InitSolverSettings(Solver);
			Solver->SetThreadingMode_External(EThreadingModeTemp::SingleThread);
		}

		~FPartitioningSolverTest()
		{

			for (FSingleParticlePhysicsProxy* ParticleProxy : ParticleProxies)
			{
				Solver->UnregisterObject(ParticleProxy);
			}
			for (FJointConstraint* Joint : JointHandles)
			{
				if (Joint)
					Solver->UnregisterObject(Joint);
			}
			if (FloorProxy)
				Solver->UnregisterObject(FloorProxy);
			Module->DestroySolver(Solver);
		}

		FRigidBodyHandle_External* MakeFloor(const FVec3& Position = FVec3(0))
		{
			FloorProxy = FSingleParticlePhysicsProxy::Create(Chaos::FGeometryParticle::CreateParticle());
			FRigidBodyHandle_External* FloorParticle = &FloorProxy->GetGameThreadAPI();
			FImplicitObjectPtr FloorGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-5000, -5000, -100), FVec3(5000, 5000, 0)));
			FloorParticle->SetGeometry(FloorGeom);
			FloorParticle->SetObjectState(EObjectStateType::Static);
			Solver->RegisterObject(FloorProxy);
			FloorParticle->SetX(Position);
			ChaosTest::SetParticleSimDataToCollide({ FloorProxy->GetParticle_LowLevel() });
			for (const TUniquePtr<FPerShapeData>& Shape : FloorParticle->ShapesArray())
			{
				Shape->SetMaterialData(MaterialData);
			}
			return FloorParticle;
		}

		FRigidBodyHandle_External* MakeCube(const FVec3& Position, const FReal Size = 200, const FReal Mass = 1)
		{
			FSingleParticlePhysicsProxy* CubeProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			FRigidBodyHandle_External* CubeParticle = &CubeProxy->GetGameThreadAPI();
			const FReal HalfSize = Size / 2.0f;
			FImplicitObjectPtr CollidingCubeGeom = Chaos::FImplicitObjectPtr(new TBox<FReal, 3>(FVec3(-HalfSize), FVec3(HalfSize)));
			CubeParticle->SetGeometry(CollidingCubeGeom);
			Solver->RegisterObject(CubeProxy);
			CubeParticle->SetGravityEnabled(true);
			CubeParticle->SetX(Position);
			SetCubeInertiaTensor(*CubeParticle, /*Dimension=*/Size, /*Mass=*/Mass);
			ChaosTest::SetParticleSimDataToCollide({ CubeProxy->GetParticle_LowLevel() });
			for (const TUniquePtr<FPerShapeData>& Shape : CubeParticle->ShapesArray())
			{
				Shape->SetMaterialData(MaterialData);
			}

			ParticleProxies.Add(CubeProxy);
			ParticleHandles.Add(CubeParticle);

			return CubeParticle;
		}

		FRigidBodyHandle_External* MakeSphere(const FVec3& Position, const FReal Radius = 100, const FReal Mass = 1)
		{
			FSingleParticlePhysicsProxy* SphereProxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
			FRigidBodyHandle_External* SphereParticle = &SphereProxy->GetGameThreadAPI();
			FImplicitObjectPtr CollidingSphereGeom = Chaos::FImplicitObjectPtr(new TSphere<FReal, 3>(FVec3(0), Radius));
			SphereParticle->SetGeometry(CollidingSphereGeom);
			Solver->RegisterObject(SphereProxy);
			SphereParticle->SetGravityEnabled(true);
			SphereParticle->SetX(Position);
			SetSphereInertiaTensor(*SphereParticle, /*Radius=*/Radius, /*Mass=*/Mass);
			ChaosTest::SetParticleSimDataToCollide({ SphereProxy->GetParticle_LowLevel() });
			for (const TUniquePtr<FPerShapeData>& Shape : SphereParticle->ShapesArray())
			{
				Shape->SetMaterialData(MaterialData);
			}

			ParticleProxies.Add(SphereProxy);
			ParticleHandles.Add(SphereParticle);

			return SphereParticle;
		}

		FJointConstraint* AddJoint(const FProxyBasePair& InConstrainedParticles, const FTransformPair& InTransform)
		{
			FJointConstraint* Joint = new FJointConstraint();
			Joint->SetParticleProxies(InConstrainedParticles);

			FPBDJointSettings SettingsTemp = Joint->GetJointSettings();
			SettingsTemp.ConnectorTransforms = InTransform;
			Joint->SetJointSettings(SettingsTemp);

			Solver->RegisterObject(Joint);

			JointHandles.Add(Joint);
			return Joint;
		}

		void MakeStackOfCubes(const int32 Num, const FReal Size = 200, const FVec3 Pos = FVec3(0))
		{
			const FReal InitialHeight = Size / 2.0f;
			for (int32 Id = 0; Id < Num; ++Id)
			{
				const FReal Z = InitialHeight + Size * Id;
				MakeCube(Pos + FVec3(0, 0, Z), Size);
			}
		}

		void MakeChainOfSpheres(const int32 Num, const FReal Radius = 100, const FVec3 Pos = FVec3(0))
		{
			const int32 DistanceMultiplier = 3;
			TArray<FSingleParticlePhysicsProxy*> Proxies;
			for (int32 Id = 0; Id < Num; ++Id)
			{
				const FReal Z = -DistanceMultiplier * Radius * Id;
				FRigidBodyHandle_External* SphereParticle = MakeSphere(Pos + FVec3(0, 0, Z), Radius);
				Proxies.Add(SphereParticle->GetProxy());
			}
			for (int32 Id = 0; Id < Num - 1; ++Id)
			{
				FRigidTransform3 Transform1 = FRigidTransform3(FVec3(0, 0, -DistanceMultiplier * Radius * 0.5), FRotation3::FromIdentity());
				FRigidTransform3 Transform2 = FRigidTransform3(FVec3(0, 0, DistanceMultiplier * Radius * 0.5), FRotation3::FromIdentity());
				AddJoint({ Proxies[Id], Proxies[Id + 1] }, { Transform1 , Transform2 });
			}

		}

		void Advance()
		{
			Solver->AdvanceAndDispatch_External(1 / 60.0);
			Solver->UpdateGameThreadStructures();
			++TickCount;
		}

		void AdvanceUntilSleeping(const int32 MaxIterations = 200)
		{
			const int32 MaxTickCount = TickCount + MaxIterations;
			bool bIsSleeping = false;
			while (!bIsSleeping && (TickCount < MaxTickCount))
			{
				Advance();

				bIsSleeping = true;
				for (FRigidBodyHandle_External* ParticleHandle : ParticleHandles)
				{
					// Check that none of the particles is dynamic
					if (ParticleHandle->ObjectState() == EObjectStateType::Dynamic)
					{
						bIsSleeping = false;
					}
				}
			}

			EXPECT_TRUE(bIsSleeping);
			EXPECT_LT(TickCount, MaxTickCount);
		}

		FMaterialData MaterialData;

		FSingleParticlePhysicsProxy* FloorProxy;
		TArray<FSingleParticlePhysicsProxy*> ParticleProxies;
		TArray<FRigidBodyHandle_External*> ParticleHandles;
		TArray< FJointConstraint*> JointHandles;

		FPBDRigidsSolver* Solver;
		FChaosSolversModule* Module;

		int32 TickCount;
	};

	// Compares test results between serial and parallel solve.
	// Note that we do expect equivalence since the execution order is the same.
	TEST_F(PartitioningSolverTest, StackOfCubes_SerialVsParallelSolve)
	{
		const int32 NumOfCubes = 20;
		const int32 ParallelModes = 2;
		const int32 PartitioningTypes = 2;
		const int32 NumSteps = 10;
		const int32 NumIterations = 8;
		//const int32 NumShockPropagationIterations = 0;

		for (int32 PartitioningType = 0; PartitioningType < PartitioningTypes; ++PartitioningType)
		{
			TArray<TArray<FVec3>> FinalPosition;
			FinalPosition.SetNum(ParallelModes);
			for (int32 ParallelMode = 0; ParallelMode < ParallelModes; ++ParallelMode)
			{
				FinalPosition[ParallelMode].Reserve(NumOfCubes);

				CVarGraphColoring->Set(PartitioningType);
				CVarParallelSolve->Set(ParallelMode);

				FPartitioningSolverTest Test;
				Test.Solver->SetPositionIterations(NumIterations);
				Test.Solver->SetVelocityIterations(0);
				// Test.Solver->GetEvolution()->SetShockPropagationIterations(NumShockPropagationIterations, NumShockPropagationIterations);
				Test.MakeFloor();
				Test.MakeStackOfCubes(NumOfCubes);

				for (int32 Step = 0; Step < NumSteps; ++Step)
				{
					Test.Advance();
				}

				for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
				{
					FinalPosition[ParallelMode].Add(ParticleHandle->GetX());
				}
			}


			for (int32 ParallelModeId = 0; ParallelModeId < ParallelModes - 1; ++ParallelModeId)
			{
				//printf("ParallelModes: %d, %d\n", ParallelModeId, ParallelModeId + 1);
				for (int32 Index = 0; Index < FinalPosition[ParallelModeId].Num(); ++Index)
				{
					EXPECT_VECTOR_NEAR(FinalPosition[ParallelModeId][Index], FinalPosition[ParallelModeId + 1][Index], UE_SMALL_NUMBER);
				}
			}
		}
	}

	// Compares test results between serial and parallel solve.
	// Note that we do expect equivalence since the execution order is the same.
	TEST_F(PartitioningSolverTest, TwoStacksOfCubes_SerialVsParallelSolve)
	{
		const int32 NumOfCubes = 20;
		const int32 ParallelModes = 2;
		const int32 PartitioningTypes = 2;
		const int32 NumSteps = 1;
		const int32 NumIterations = 8;
		const int32 NumShockPropagationIterations = 0;

		for (int32 PartitioningType = 0; PartitioningType < PartitioningTypes; ++PartitioningType)
		{
			TArray<TArray<FVec3>> FinalPosition;
			FinalPosition.SetNum(ParallelModes);
			for (int32 ParallelMode = 0; ParallelMode < ParallelModes; ++ParallelMode)
			{
				FinalPosition[ParallelMode].Reserve(NumOfCubes);

				CVarGraphColoring->Set(PartitioningType);
				CVarParallelSolve->Set(ParallelMode);

				FPartitioningSolverTest Test;
				Test.Solver->SetPositionIterations(NumIterations);
				//Test.Solver->SetVelocityIterations(0);
				//Test.Solver->GetEvolution()->SetShockPropagationIterations(NumShockPropagationIterations, NumShockPropagationIterations);
				Test.MakeFloor();
				Test.MakeStackOfCubes(NumOfCubes, 200, FVec3(0, 0, 0));
				Test.MakeStackOfCubes(NumOfCubes, 200, FVec3(600, 0, 0));

				for (int32 Step = 0; Step < NumSteps; ++Step)
				{
					Test.Advance();
				}

				for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
				{
					FinalPosition[ParallelMode].Add(ParticleHandle->GetX());
				}
			}


			for (int32 ParallelModeId = 0; ParallelModeId < ParallelModes - 1; ++ParallelModeId)
			{
				//printf("ParallelModes: %d, %d\n", ParallelModeId, ParallelModeId + 1);
				for (int32 Index = 0; Index < FinalPosition[ParallelModeId].Num(); ++Index)
				{
					EXPECT_VECTOR_NEAR(FinalPosition[ParallelModeId][Index], FinalPosition[ParallelModeId + 1][Index], UE_SMALL_NUMBER);
				}
			}
		}
	}

	// Compares test results between serial and parallel solve.
	// Note that we do expect equivalence since the execution order is the same.
	TEST_F(PartitioningSolverTest, ChainOfSpheres_SerialVsParallelSolve)
	{
		const int32 NumOfCubes = 20;
		const int32 ParallelModes = 2;
		const int32 PartitioningTypes = 2;
		const int32 NumSteps = 10;
		const int32 NumIterations = 8;
		const int32 NumShockPropagationIterations = 0;

		for (int32 PartitioningType = 0; PartitioningType < PartitioningTypes; ++PartitioningType)
		{
			TArray<TArray<FVec3>> FinalPosition;
			FinalPosition.SetNum(ParallelModes);
			for (int32 ParallelMode = 0; ParallelMode < ParallelModes; ++ParallelMode)
			{
				FinalPosition[ParallelMode].Reserve(NumOfCubes * 2);

				CVarGraphColoring->Set(PartitioningType);
				CVarParallelSolve->Set(ParallelMode);

				FPartitioningSolverTest Test;
				Test.Solver->SetPositionIterations(NumIterations);
				//Test.Solver->SetVelocityIterations(0);
				//Test.Solver->GetEvolution()->SetShockPropagationIterations(NumShockPropagationIterations, NumShockPropagationIterations);
				Test.MakeChainOfSpheres(NumOfCubes, 200, FVec3(0, 0, 0));
				Test.MakeChainOfSpheres(NumOfCubes, 200, FVec3(600, 0, 0));

				for (int32 Step = 0; Step < NumSteps; ++Step)
				{
					Test.Advance();
				}

				for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
				{
					FinalPosition[ParallelMode].Add(ParticleHandle->GetX());
				}
			}


			for (int32 ParallelModeId = 0; ParallelModeId < ParallelModes - 1; ++ParallelModeId)
			{
				//printf("ParallelModes: %d, %d\n", ParallelModeId, ParallelModeId + 1);
				for (int32 Index = 0; Index < FinalPosition[ParallelModeId].Num(); ++Index)
				{
					EXPECT_VECTOR_NEAR(FinalPosition[ParallelModeId][Index], FinalPosition[ParallelModeId + 1][Index], UE_SMALL_NUMBER);
				}
			}
		}
	}

	// Compares test results between no partitioning and graph coloring.
	// Note that we do not expect equivalence since we change the execution order.
	TEST_F(PartitioningSolverTest, StackOfCubes_NoPartitioningVsColoring)
	{
		const int32 NumOfCubes = 5;
		const int32 ParallelModes = 2;
		const int32 PartitioningTypes = 2;
		const int32 NumSteps = 1;

		for (int32 ParallelMode = 0; ParallelMode < ParallelModes; ++ParallelMode)
		{
			TArray<TArray<FVec3>> FinalPosition;
			FinalPosition.SetNum(PartitioningTypes);

			for (int32 PartitioningType = 0; PartitioningType < PartitioningTypes; ++PartitioningType)
			{
				FinalPosition[PartitioningType].Reserve(NumOfCubes);

				CVarGraphColoring->Set(PartitioningType);
				CVarParallelSolve->Set(ParallelMode);

				FPartitioningSolverTest Test;
				Test.Solver->SetPositionIterations(500);
				Test.MakeFloor();
				Test.MakeStackOfCubes(NumOfCubes);

				for (int32 Step = 0; Step < NumSteps; ++Step)
				{
					Test.Advance();
				}

				for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
				{
					FinalPosition[PartitioningType].Add(ParticleHandle->GetX());
				}
			}

			// Coloring changes the execution order
			// Therefore, we expect the solver to take longer to converge, especially for high stacks.
			// Nevertheless, for a small stack and many solver iterations, the result should be nearly identical.
			for (int32 PartitioningType = 0; PartitioningType < PartitioningTypes - 1; ++PartitioningType)
			{
				//printf("PartitioningTypes: %d, %d\n", PartitioningType, PartitioningType + 1);
				for (int32 Index = 0; Index < FinalPosition[PartitioningType].Num(); ++Index)
				{
					EXPECT_VECTOR_NEAR(FinalPosition[PartitioningType][Index], FinalPosition[PartitioningType + 1][Index], UE_KINDA_SMALL_NUMBER);
				}
			}
		}
	}

	// Compares test results between no partitioning and graph coloring.
	// Note that we do not expect equivalence since we change the execution order.
	TEST_F(PartitioningSolverTest, TwoStacksOfCubes_NoPartitioningVsColoring)
	{
		const int32 NumOfCubes = 5;
		const int32 ParallelModes = 2;
		const int32 PartitioningTypes = 2;
		const int32 NumSteps = 1;
		const int32 NumIterations = 500;
		const int32 NumShockPropagationIterations = 0;

		for (int32 ParallelMode = 0; ParallelMode < ParallelModes; ++ParallelMode)
		{
			TArray<TArray<FVec3>> FinalPosition;
			FinalPosition.SetNum(PartitioningTypes);
			for (int32 PartitioningType = 0; PartitioningType < PartitioningTypes; ++PartitioningType)
			{
				FinalPosition[PartitioningType].Reserve(NumOfCubes * 2);

				CVarGraphColoring->Set(PartitioningType);
				CVarParallelSolve->Set(ParallelMode);

				FPartitioningSolverTest Test;
				Test.Solver->SetPositionIterations(NumIterations);
				//Test.Solver->SetVelocityIterations(0);
				//Test.Solver->GetEvolution()->SetShockPropagationIterations(NumShockPropagationIterations, NumShockPropagationIterations);
				Test.MakeFloor();
				Test.MakeStackOfCubes(NumOfCubes, 200, FVec3(0, 0, 0));
				Test.MakeStackOfCubes(NumOfCubes, 200, FVec3(600, 0, 0));

				for (int32 Step = 0; Step < NumSteps; ++Step)
				{
					Test.Advance();
				}

				for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
				{
					FinalPosition[PartitioningType].Add(ParticleHandle->GetX());
				}
			}


			for (int32 PartitioningType = 0; PartitioningType < PartitioningTypes - 1; ++PartitioningType)
			{
				//printf("PartitioningType: %d, %d\n", PartitioningType, PartitioningType + 1);
				for (int32 Index = 0; Index < FinalPosition[PartitioningType].Num(); ++Index)
				{
					EXPECT_VECTOR_NEAR(FinalPosition[PartitioningType][Index], FinalPosition[PartitioningType + 1][Index], UE_KINDA_SMALL_NUMBER);
				}
			}
		}
	}

	// Compares test results between no partitioning and graph coloring.
	// Note that we do not expect equivalence since we change the execution order.
	TEST_F(PartitioningSolverTest, ChainOfSpheres_NoPartitioningVsColoring)
	{
		const int32 NumOfCubes = 20;
		const int32 ParallelModes = 2;
		const int32 PartitioningTypes = 2;
		const int32 NumSteps = 10;
		const int32 NumIterations = 8;
		const int32 NumShockPropagationIterations = 0;

		for (int32 ParallelMode = 0; ParallelMode < ParallelModes; ++ParallelMode)
		{
			TArray<TArray<FVec3>> FinalPosition;
			FinalPosition.SetNum(PartitioningTypes);
			for (int32 PartitioningType = 0; PartitioningType < PartitioningTypes; ++PartitioningType)
			{
				FinalPosition[PartitioningType].Reserve(NumOfCubes);

				CVarGraphColoring->Set(PartitioningType);
				CVarParallelSolve->Set(ParallelMode);

				FPartitioningSolverTest Test;
				Test.Solver->SetPositionIterations(NumIterations);
				//Test.Solver->SetVelocityIterations(0);
				//Test.Solver->GetEvolution()->SetShockPropagationIterations(NumShockPropagationIterations, NumShockPropagationIterations);
				Test.MakeChainOfSpheres(NumOfCubes, 200, FVec3(0, 0, 0));
				Test.MakeChainOfSpheres(NumOfCubes, 200, FVec3(600, 0, 0));

				for (int32 Step = 0; Step < NumSteps; ++Step)
				{
					Test.Advance();
				}

				for (FRigidBodyHandle_External* ParticleHandle : Test.ParticleHandles)
				{
					FinalPosition[PartitioningType].Add(ParticleHandle->GetX());
				}
			}


			for (int32 PartitioningType = 0; PartitioningType < PartitioningTypes - 1; ++PartitioningType)
			{
				//printf("ParallelModes: %d, %d\n", PartitioningType, PartitioningType + 1);
				for (int32 Index = 0; Index < FinalPosition[PartitioningType].Num(); ++Index)
				{
					EXPECT_VECTOR_NEAR(FinalPosition[PartitioningType][Index], FinalPosition[PartitioningType + 1][Index], UE_KINDA_SMALL_NUMBER);
				}
			}
		}
	}
}