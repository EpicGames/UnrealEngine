// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestSolverProxies.h"
#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

#include "Chaos/ChaosEngineInterface.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ErrorReporter.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Chaos/Utilities.h"
#include "PBDRigidsSolver.h"
#include "ChaosSolversModule.h"

#include "Modules/ModuleManager.h"


namespace ChaosTest {

	using namespace Chaos;

	void SingleParticleProxySingleThreadTest()
	{
		Chaos::FImplicitObjectPtr Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));

		FChaosSolversModule* Module = FChaosSolversModule::GetModule();

		// Make a solver
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);

		// Make a particle


		auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& Particle = Proxy->GetGameThreadAPI();
		Particle.SetGeometry(Sphere);
		Particle.SetX(FVec3(0, 0, 0));
		Particle.SetGravityEnabled(false);
		Solver->RegisterObject(Proxy);

		Particle.SetV(FVec3(0, 0, 10));

		::ChaosTest::SetParticleSimDataToCollide({ Proxy->GetParticle_LowLevel() });

		Solver->AdvanceAndDispatch_External(100.0f);
		Solver->UpdateGameThreadStructures();

		// Make sure game thread data has changed
		FVec3 V = Particle.V();
		EXPECT_EQ(V.X, 0.f);
		EXPECT_GT(V.Z, 0.f);

		FVec3 X = Particle.X();
		EXPECT_EQ(X.X, 0.f);
		EXPECT_GT(X.Z, 0.f);

		// Throw out the proxy
		Solver->UnregisterObject(Proxy);

		Module->DestroySolver(Solver);
	}

	void SingleParticleProxyWakeEventPropagationTest()
	{
		using namespace Chaos;
		auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));

		FChaosSolversModule* Module = FChaosSolversModule::GetModule();

		// Make a solver
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);

		// Make a particle

		auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& Particle = Proxy->GetGameThreadAPI();
		Particle.SetGeometry(Sphere);
		Particle.SetX(FVec3(0, 0, 220));
		Particle.SetV(FVec3(0, 0, -10));
		Particle.SetCCDEnabled(true);
		Solver->RegisterObject(Proxy);
		Solver->AddDirtyProxy(Proxy);

		auto Proxy2 = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& Particle2 = Proxy2->GetGameThreadAPI();
		Particle2.SetGeometry(Sphere);
		Particle2.SetX(FVec3(0, 0, 100));
		Particle2.SetV(FVec3(0, 0, 0));
		Solver->RegisterObject(Proxy2);
		Particle2.SetObjectState(Chaos::EObjectStateType::Sleeping);

		::ChaosTest::SetParticleSimDataToCollide({ Proxy->GetParticle_LowLevel(),Proxy2->GetParticle_LowLevel() });

		// let top paticle collide and wake up second particle
		int32 LoopCount = 0;
		while (Particle2.GetWakeEvent() == EWakeEventEntry::None && LoopCount++ < 20)
		{
			Solver->AdvanceAndDispatch_External(100.0f);
			Solver->UpdateGameThreadStructures();
		}

		// Make sure game thread data has changed
		FVec3 V = Particle.V();
		EXPECT_EQ(Particle.GetWakeEvent(), EWakeEventEntry::None);
		EXPECT_EQ(Particle.ObjectState(), Chaos::EObjectStateType::Dynamic);

		EXPECT_EQ(Particle2.GetWakeEvent(), EWakeEventEntry::Awake);
		EXPECT_EQ(Particle2.ObjectState(), Chaos::EObjectStateType::Dynamic);

		Particle2.ClearEvents();
		EXPECT_EQ(Particle2.GetWakeEvent(), EWakeEventEntry::None);

		// Throw out the proxy
		Solver->UnregisterObject(Proxy);
		Solver->UnregisterObject(Proxy2);

		Module->DestroySolver(Solver);
	}

	void SingleParticleProxyNoUniqueIndexLeaks()
	{
		// this test make sure that we are not leaking uniqueIdx when creating and destroying particle in one frame (without PT running)

		auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));

		FChaosSolversModule* Module = FChaosSolversModule::GetModule();

		// Make a solver
		auto* Solver = Module->CreateSolver(nullptr, -1);

		// Make a particle
		auto Proxy = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& Particle = Proxy->GetGameThreadAPI();
		Particle.SetGeometry(Sphere);
		Particle.SetX(FVec3(0, 0, 0));
		Particle.SetGravityEnabled(false);

		// first register the object
		Solver->RegisterObject(Proxy);

		// keep track of the actual unique Idx
		FUniqueIdx FirstIdx = Particle.UniqueIdx();

		// unregister the object in the same GT frame 
		Solver->UnregisterObject(Proxy);

		// run PT to make sure the callbacks are running and updating the internal pending lists 
		Solver->AdvanceAndDispatch_External(100.0f);
		Solver->UpdateGameThreadStructures();

		// unique idx should be scheduled for cleanup
		EXPECT_TRUE(Solver->GetEvolution()->IsUniqueIndexPendingRelease(FirstIdx));

		// run PT again so that the cleanup is done
		Solver->AdvanceAndDispatch_External(100.0f);
		Solver->UpdateGameThreadStructures();
		Solver->AdvanceAndDispatch_External(100.0f);
		Solver->UpdateGameThreadStructures();

		// Unique idx should be gone from the pending lists
		EXPECT_FALSE(Solver->GetEvolution()->IsUniqueIndexPendingRelease(FirstIdx));

		Module->DestroySolver(Solver);
	}

	void JointProxySolverInitTest()
	{
		// Test that when we create a joint constraint, that the proxy is not added to solver proxy array by game thread.
		// This array is physics thread only, it should only be added after ticking solver.

		using namespace Chaos;
		auto Sphere = Chaos::FImplicitObjectPtr(new Chaos::FSphere(FVec3(0), 10));
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();

		// Make a solver
		auto* Solver = Module->CreateSolver(nullptr, /*AsyncDt=*/-1);

		// Make 2 particles for our constraint
		auto Proxy1 = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& Particle1 = Proxy1->GetGameThreadAPI();
		Particle1.SetGeometry(Sphere);
		Solver->RegisterObject(Proxy1);

		auto Proxy2 = FSingleParticlePhysicsProxy::Create(Chaos::FPBDRigidParticle::CreateParticle());
		auto& Particle2 = Proxy2->GetGameThreadAPI();
		Particle2.SetGeometry(Sphere);
		Solver->RegisterObject(Proxy2);

		// Confirm solver doesn't have any joint constraint proxies yet.
		EXPECT_EQ(Solver->GetJointConstraintPhysicsProxies_Internal().Num(), 0);

		FTransform IdentityTransform(FTransform::Identity);
		FPhysicsConstraintHandle Constraint = FChaosEngineInterface::CreateConstraint(Proxy1, Proxy2, IdentityTransform, IdentityTransform);

		// joint Proxy has been created, should be in dirty proxy list to be sent to physics thread.
		// Note that we expect three dirty proxies, 2 for particles, 1 for joint.
		EXPECT_EQ(Solver->GetDirtyProxyBucketInfoNum_External(EPhysicsProxyType::SingleParticleProxy), 2);
		EXPECT_EQ(Solver->GetDirtyProxyBucketInfoNum_External(EPhysicsProxyType::JointConstraintType), 1);

		// Although proxy exists, it should not have been added to physics thread only proxy array yet, as doing so from game thread code is wrong.
		// Confirm game thread did not add it.
		EXPECT_EQ(Solver->GetJointConstraintPhysicsProxies_Internal().Num(), 0);

		// Tick physics thread, this should add proxy to solver proxy array.
		Solver->AdvanceAndDispatch_External(/*Dt=*/1.0f);
		EXPECT_EQ(Solver->GetJointConstraintPhysicsProxies_Internal().Num(), 1);

		// Release constraint to test removal
		FChaosEngineInterface::ReleaseConstraint(Constraint);

		// Have only released constraint on game thread, proxy should still be in array.
		EXPECT_EQ(Solver->GetJointConstraintPhysicsProxies_Internal().Num(), 1);

		// Tick physics thread, this should remove proxy from array.
		Solver->AdvanceAndDispatch_External(/*Dt=*/1.0f);
		EXPECT_EQ(Solver->GetJointConstraintPhysicsProxies_Internal().Num(), 0);

		Solver->UnregisterObject(Proxy1);
		Solver->UnregisterObject(Proxy2);
		Module->DestroySolver(Solver);
	}

	GTEST_TEST(AllTraits, SingleParticleProxyTests)
	{
		ChaosTest::SingleParticleProxySingleThreadTest();
		ChaosTest::SingleParticleProxyWakeEventPropagationTest();
		//ChaosTest::SingleParticleProxyNoUniqueIndexLeaks();
		ChaosTest::JointProxySolverInitTest();
	}
}
