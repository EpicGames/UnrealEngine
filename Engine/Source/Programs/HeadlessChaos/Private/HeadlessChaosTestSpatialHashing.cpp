// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestSpatialHashing.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Modules/ModuleManager.h"
#include "Chaos/Matrix.h"
#include "Chaos/PBDRigidClustering.h"
#include "Chaos/PBDRigidClusteringCollisionParticleAlgo.h"

DEFINE_LOG_CATEGORY_STATIC(AHSP_Test, Verbose, All);


namespace ChaosTest {
	using namespace Chaos;

	TArray<FVec3> GenerateSamplePoints(int32 NumPoints, int32 InitialDistance) {

		TArray<FVec3> ReturnValue;
		for (int i = 1; i <= NumPoints/2; i++) {
			int32 iCubed = i * i * i;
			FVec3 Vec1(0), Vec2(0);

			Vec1 = FVec3(FReal(FMath::Rand()) / RAND_MAX, FReal(FMath::Rand()) / RAND_MAX, FReal(FMath::Rand()) / RAND_MAX) * InitialDistance;
			Vec2 = FVec3(FReal(FMath::Rand()) / RAND_MAX, FReal(FMath::Rand()) / RAND_MAX, FReal(FMath::Rand()) / RAND_MAX) * InitialDistance;
			FVec3 Mid((Vec1 - Vec2) / 2);
			Vec1 = Mid + (Mid - Vec1).Normalize()*(InitialDistance / FReal(iCubed));
			Vec2 = Mid - (Mid - Vec1).Normalize()*(InitialDistance / FReal(iCubed));

			ReturnValue.Add(Vec1);
			ReturnValue.Add(Vec2);
		}
		return ReturnValue;
	}

	void SpatialHashing()
	{
		UE_LOGF(AHSP_Test, Verbose, "SpatialHashing");
		TArray<FVec3> Samples = Chaos::CleanCollisionParticles(GenerateSamplePoints(100, 1000), 1.0);
		for (int32 Index1 = 0; Index1 < Samples.Num(); Index1++)
		{
			for (int32 Index2 = 0; Index2 < Samples.Num(); Index2++)
			{
				if (Index1 != Index2 && Index1<Index2)
				{
					FVec3 Sample1(Samples[Index1]), Sample2(Samples[Index2]);
					float Delta = (Sample2 - Sample1).Size();
					if (Delta < 1.0)
					{
						UE_LOGF(AHSP_Test, Verbose, "... [%d](%3.5f,%3.5f,%3.5f)", Index1, Sample1.X, Sample1.Y, Sample1.Z);
						UE_LOGF(AHSP_Test, Verbose, "... [%d](%3.5f,%3.5f,%3.5f)", Index2, Sample2.X, Sample2.Y, Sample2.Z);
						UE_LOGF(AHSP_Test, Verbose, "... ... %3.5f", Delta);
						EXPECT_TRUE(false);
					}
				}
			}
		}
	}


}