// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestGJK.h"

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/GJK.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/GJK.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleRegister.h"

namespace ChaosTest
{
	using namespace Chaos;

	// InSupportDirection need not be normalized;
	// InExpectedSupport is any point on the support plane - we only check the distance along the support direction
	template<typename BoxType>
	void CheckBoxSupportDir(const BoxType& Box, const FVec3& InSupportDirection, const FVec3& InExpectedSupport)
	{
		const FReal DistanceTolerance = UE_KINDA_SMALL_NUMBER;

		const FVec3 SupportDirection = InSupportDirection.GetSafeNormal();
		const FReal ExpectedSupportDistance = FVec3::DotProduct(InExpectedSupport, SupportDirection);

		// NOTE: Support functions should return a vertex position that is furthest along the support direct.
		// When there are multiple options, any of the options should be ok
		// Note: using non-normalized direction
		const FReal Margin = 0;
		FReal SupportDelta = 0;
		int32 SupportVertexIndex = INDEX_NONE;
		FVec3 Support = Box.SupportCore(InSupportDirection, Margin, &SupportDelta, SupportVertexIndex);

		// The support point has the expected distance along the support direction
		const FReal SupportDistance = FVec3::DotProduct(Support, SupportDirection);
		EXPECT_NEAR(SupportDistance, ExpectedSupportDistance, DistanceTolerance);

		// We have a vertex index
		EXPECT_NE(SupportVertexIndex, INDEX_NONE);
		if (SupportVertexIndex != INDEX_NONE)
		{
			// SupportVertexIndex is for a point with the correct support distance
			const FVec3 SupportVertex = Box.GetVertex(SupportVertexIndex);
			const FReal SupportVertexDistance = FVec3::DotProduct(SupportVertex, SupportDirection);
			EXPECT_NEAR(SupportVertexDistance, ExpectedSupportDistance, DistanceTolerance);

			// The internal Aabb vertex has the correct support distance
			const FAABB3 Aabb = Box.BoundingBox();
			const FVec3 AabbVertex = Aabb.GetVertex(SupportVertexIndex);
			const FReal AabbVertexDistance = FVec3::DotProduct(AabbVertex, SupportDirection);
			EXPECT_NEAR(AabbVertexDistance, ExpectedSupportDistance, DistanceTolerance);

			// The Aabb and Box vertices are the same
			EXPECT_NEAR(AabbVertex.X, SupportVertex.X, DistanceTolerance);
			EXPECT_NEAR(AabbVertex.Y, SupportVertex.Y, DistanceTolerance);
			EXPECT_NEAR(AabbVertex.Z, SupportVertex.Z, DistanceTolerance);

			// Make sure that GetMostOpposingPlane produces a plane that points in the right direction and contains the support vertex
			const int32 OpposingPlaneIndex = Box.GetMostOpposingPlane(-InSupportDirection);
			EXPECT_NE(OpposingPlaneIndex, INDEX_NONE);
			if (OpposingPlaneIndex != INDEX_NONE)
			{
				// Check the direction is actualy in the direction of support
				const FVec3 OpposingPlaneNormal = Box.GetPlane(OpposingPlaneIndex).Normal();
				const FReal OpposingNormalDotSupport = FVec3::DotProduct(OpposingPlaneNormal, SupportDirection);
				EXPECT_GT(OpposingNormalDotSupport, FReal(0)) << "SupportDir=(" << InSupportDirection.X << "," << InSupportDirection.Y << "," << InSupportDirection.Z << ")";

				// Make sure the plane contains the support vertex
				bool bFoundSupportVertex = false;
				for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < Box.NumPlaneVertices(OpposingPlaneIndex); ++PlaneVertexIndex)
				{
					const int32 VertexIndex = Box.GetPlaneVertex(OpposingPlaneIndex, PlaneVertexIndex);
					if (VertexIndex == SupportVertexIndex)
					{
						bFoundSupportVertex = true;
						break;
					}
				}
				EXPECT_TRUE(bFoundSupportVertex);
			}
		}
	}

	template<typename BoxType>
	void CheckBoxSupportVertex(const BoxType& Box, int32 X, int32 Y, int32 Z)
	{
		EXPECT_TRUE((X != 0) || (Y != 0) || (Z != 0));

		const FAABB3 Aabb = Box.BoundingBox();
		const FVec3 UnitCubePoint = FVec3(FReal(X), FReal(Y), FReal(Z));
		const FVec3 SupportDir = UnitCubePoint;	// No need to normalize
		const FVec3 SupportVertex = Aabb.GetCenter() + FReal(0.5) * Aabb.Extents() * UnitCubePoint;
		CheckBoxSupportDir(Box, SupportDir, SupportVertex);
	}

	template<typename BoxType>
	void CheckBoxSupportAxes(const BoxType& Box)
	{
		// Check all cardinal and diagonal directions
		for (int32 X = -1; X <= 1; ++X)
		{
			for (int32 Y = -1; Y <= 1; ++Y)
			{
				for (int32 Z = -1; Z <= 1; ++Z)
				{
					if ((X != 0) || (Y != 0) || (Z != 0))
					{
						CheckBoxSupportVertex(Box, X, Y, Z);
					}
				}
			}
		}
	}

	// Test SupportCore for all cardinal and diagonal axes
	GTEST_TEST(SupportTests, TestBoxSupportCore)
	{
		const FVec3 Center = FVec3(3.5, -94.1, 15.6);
		const FVec3 HalfExtent = FVec3(100, 200, 300);
		const FReal Margin = FReal(0);

		const FImplicitBox3 Box = FImplicitBox3(Center - HalfExtent, Center + HalfExtent, Margin);
		CheckBoxSupportAxes(Box);
	}

	GTEST_TEST(SupportTests, TestScaledBoxSupportCore)
	{
		const FVec3 Center = FVec3(3.5, -94.1, 15.6);
		const FVec3 HalfExtent = FVec3(100, 200, 300);
		const FReal Margin = FReal(0);
		const FVec3 Scale = FVec3(8.7, 3.2, 1.1);

		TRefCountPtr<FImplicitBox3> Box = TRefCountPtr<FImplicitBox3>(new FImplicitBox3(Center - HalfExtent, Center + HalfExtent, Margin));
		TRefCountPtr<TImplicitObjectScaled<FImplicitBox3>> BoxScaled = TRefCountPtr<TImplicitObjectScaled<FImplicitBox3>>(new TImplicitObjectScaled<FImplicitBox3>(Box, Scale));
		CheckBoxSupportAxes(*BoxScaled.GetReference());
	}

	// Test capsule support functions when scaled
	GTEST_TEST(SupportTests, TestScaledCapsuleSupportCore)
	{
		FVec3 PointA{ -1, 0, -100 };
		FVec3 PointB{ 1, 0, 100 };
		FVec3 ScaleX{ 100, 1, 1 };
		FCapsulePtr Capsule(new FImplicitCapsule3(PointA, PointB, 10));
		TImplicitObjectScaled<FImplicitCapsule3> CapsuleScaled(Capsule, ScaleX);

		FVec3 SupportDir{ 0.1f, 0.0f, -1.0f }; // Pointing down and slightly to the right

		int32 Vertex = INDEX_NONE;
		FVec3 Support = Capsule->SupportCore(SupportDir, 0, nullptr, Vertex);
		EXPECT_EQ(Support, PointA); // Expect bottom point

		Support = CapsuleScaled.SupportCore(SupportDir, 0, nullptr, Vertex);
		EXPECT_EQ(Support, ScaleX * PointA); // Still expect bottom point (But scaled)
	}
}
