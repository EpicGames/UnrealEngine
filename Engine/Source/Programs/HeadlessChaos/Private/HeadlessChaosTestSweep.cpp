// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestSweep.h"

#include "HeadlessChaos.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/GeometryQueries.h"

namespace ChaosTest
{
	using namespace Chaos;

	void CapsuleSweepAgainstTriMeshReal()
	{
		// Trimesh is from SM_Cattus_POI_Rib, this was a real world failure that is now fixed.

		using namespace Chaos;
		FTriangleMeshImplicitObject::ParticlesType TrimeshParticles(
		{
			{29.0593967f, -5.21321106f, -10.2669592f},
			{34.5006638f, -3.16600156f, -14.5092020f},
			{28.8770218f, -5.00888205f, -10.4567394f},
			{38.9350929f, -1.47836626f, -17.6896191f},
			{39.1246262f, -1.58615386f, -17.4255066f},
			{42.4614334f, -0.114946000f, -19.8563728f},
			{38.7996712f, -1.91535223f, -16.3864536f},
			{29.4634132f, -5.15883255f, -9.64184284f},
			{29.6582336f, -5.05185938f, -9.37638092f},
			{45.0206337f, 0.641714215f, -21.0300026f},
			{46.2034111f, 1.19252074f, -22.0121441f},
			{44.3153610f, 0.398101240f, -19.9310055f},
			{38.2805557f, -1.87191427f, -15.7972298f},
			{48.8065224f, 1.82796133f, -22.7662964f},
			{49.7862473f, 2.21876359f, -23.4238491f},
			{49.1621628f, 1.89441979f, -22.1913338f},
			{52.2055054f, 2.59018326f, -23.6808910f},
			{55.6262436f, 3.71029496f, -24.4840908f},
			{52.2546806f, 2.55433965f, -22.8908424f},
			{47.6990814f, 1.76376379f, -21.3171616f},
			{55.9418335f, 3.34045410f, -24.2242432f},
			{56.7527275f, 4.26803923f, -24.7314072f},
			{56.6030273f, 3.37918210f, -23.5389385f},
			{57.1201477f, 3.92609882f, -24.5670509f},
			{57.5926590f, 5.32376480f, -24.8299332f},
			{57.3900146f, 3.86687994f, -24.3628445f},
			{58.3100967f, 5.59623432f, -24.5884380f},
			{57.9720459f, 6.88918400f, -24.9411621f},
			{58.3736610f, 5.22753286f, -23.6481304f},
			{58.3795776f, 9.20724392f, -24.4941502f},
			{57.6161652f, 7.89536858f, -25.0186253f},
			{57.7064209f, 8.63112640f, -24.8542480f},
			{57.4299011f, 9.53679848f, -24.7764454f},
			{58.6315613f, 9.11761856f, -24.1844330f},
			{58.5211067f, 7.24213171f, -23.5957718f},
			{57.5271606f, 9.99959183f, -24.4184647f},
			{58.2185135f, 9.55542183f, -23.2548218f},
			{57.3097725f, 10.2978697f, -23.5637169f},
			{56.3722000f, 9.18606091f, -24.9406128f},
			{58.2657166f, 8.36739063f, -23.1939850f},
			{56.3347893f, 9.67211342f, -24.4351349f},
			{57.3641586f, 9.99938393f, -23.0135994f},
			{58.1764908f, 5.82101727f, -23.1902943f},
			{57.3481255f, 8.60126781f, -22.9416485f},
			{56.1609459f, 9.56552029f, -23.1539116f},
			{56.2483864f, 9.85045052f, -23.4782410f},
			{55.7174797f, 8.29436588f, -23.0638103f},
			{54.7229729f, 8.73790932f, -23.3331184f},
			{54.0776367f, 8.77919769f, -23.7519341f},
			{53.9943695f, 8.17912292f, -24.5668526f},
			{51.3485641f, 7.47948122f, -23.0380058f},
			{51.2222366f, 7.41138554f, -23.6724663f},
			{53.9260139f, 7.74872541f, -24.6529694f},
			{52.2321625f, 7.36892176f, -22.8332882f},
			{50.7700424f, 6.47493553f, -23.9456253f},
			{56.6027756f, 8.31463623f, -25.0338459f},
			{57.2853279f, 6.51692963f, -25.0099850f},
			{50.7739677f, 5.00270128f, -24.0342751f},
			{47.1190453f, 4.83312464f, -22.8379707f},
			{49.7170258f, 2.62024260f, -23.6686172f},
			{43.0110207f, 0.426635355f, -20.4862480f},
			{42.7733459f, 3.24669766f, -20.7472992f},
			{34.1604767f, -2.98149395f, -14.5161381f},
			{28.7377853f, -4.72821426f, -10.5358887f},
			{28.2660236f, -3.50151801f, -10.7232647f},
			{34.4872894f, 0.0475072451f, -15.1762705f},
			{27.9901352f, -2.42618680f, -10.5657129f},
			{27.6759071f, 0.205285028f, -10.2063637f},
			{36.6210670f, 2.21063828f, -16.3613548f},
			{27.6877193f, 0.690624714f, -10.0054235f},
			{42.8507233f, 3.80447602f, -20.6911526f},
			{39.7815170f, 3.57913852f, -17.9840279f},
			{42.7781487f, 4.36921978f, -19.9032001f},
			{27.8298340f, 0.836181641f, -9.67990112f},
			{28.1982155f, 0.777253330f, -9.15566635f},
			{46.0600357f, 5.31439161f, -22.0218697f},
			{36.9135399f, 2.88628912f, -15.1429548f},
			{47.6548805f, 5.92783451f, -22.1377392f},
			{44.6417084f, 4.89560652f, -20.4432411f},
			{47.3978271f, 5.70256472f, -21.4681816f},
			{42.0819168f, 3.81306863f, -18.2309093f},
			{34.6343575f, 1.72932339f, -12.9041548f},
			{28.5061035f, 0.184601068f, -8.87326050f},
			{28.7382660f, -0.523662448f, -8.71943665f},
			{47.9805756f, 5.56415987f, -21.3477497f},
			{35.0854912f, 1.10745859f, -13.0555801f},
			{29.5277596f, -4.36109161f, -9.26433849f},
			{44.2550583f, 3.73801875f, -19.3567085f},
			{48.6162643f, 5.16706753f, -21.4052410f},
			{33.8666039f, -3.11327124f, -12.2873840f},
			{29.5907612f, -4.66688442f, -9.28112030f},
			{37.8056297f, -1.60099864f, -15.0567427f},
			{43.0328369f, 1.30700612f, -18.5348587f},
			{45.9731026f, 1.34582365f, -20.3940430f},
			{48.3442001f, 2.43499756f, -21.2875118f},
			{53.0610924f, 3.33885503f, -22.4557037f},
			{54.3709755f, 5.39051533f, -22.8000984f},
			{55.9649162f, 3.45125723f, -23.1164742f},
			{56.6591415f, 4.77854252f, -22.8085098f},
			{57.1962891f, 4.12890625f, -23.1115723f},
			{25.7303848f, -5.68249369f, -8.51585770f},
			{26.5309143f, -4.24291611f, -9.48308945f},
			{24.8292999f, -5.12226152f, -8.47907925f},
			{20.3731937f, -6.52481079f, -5.81806421f},
			{26.4056702f, -3.28238487f, -9.07682896f},
			{23.9174252f, -5.14487743f, -7.70250511f},
			{21.3869915f, -5.92491102f, -6.74390554f},
			{18.1200867f, -6.53198242f, -4.76391602f},
			{21.6681881f, -5.50328684f, -6.94691038f},
			{18.0004005f, -5.80428123f, -5.25099564f},
			{15.9813662f, -6.10711002f, -4.21955252f},
			{18.8343563f, -5.61828232f, -5.61174917f},
			{20.2135773f, -4.88624334f, -5.77969599f},
			{15.4357758f, -5.21927977f, -4.48837900f},
			{11.1496544f, -2.60679626f, -4.38681793f},
			{10.0865431f, -3.16139269f, -3.85664177f},
			{8.63047791f, -2.38198924f, -3.79941773f},
			{14.6292696f, -4.35020351f, -4.44057178f},
			{7.90794277f, -1.09154844f, -4.05677366f},
			{6.95699024f, -1.20261848f, -3.97343445f},
			{8.23572540f, -0.744559944f, -4.00086451f},
			{3.10794330f, 0.166724160f, -4.27361870f},
			{3.13311577f, -0.346626908f, -4.11541176f},
			{1.48072076f, -0.108915798f, -4.42642593f},
			{0.311369270f, -0.0673256740f, -4.49766636f},
			{-1.59366548f, 0.725093842f, -4.37339926f},
			{-3.45725179f, -0.749191761f, -4.34381390f},
			{-6.95951319f, 0.0120848129f, -3.59336853f},
			{-6.83396530f, -1.23874116f, -3.47961354f},
			{-7.89957952f, -1.66941118f, -2.82847047f},
			{0.703817546f, 0.936612248f, -4.19160843f},
			{-5.88422966f, 0.830549538f, -3.74061680f},
			{-8.81333447f, -0.651533544f, -2.60129619f},
			{3.10681534f, 0.612948775f, -3.77665353f},
			{-4.67421389f, 1.29445243f, -3.36058092f},
			{-0.481138408f, 1.18588996f, -3.53271747f},
			{7.11869955f, -0.267135799f, -3.69777322f},
			{-7.27621984f, 0.737751126f, -2.76471734f},
			{2.97556901f, 1.46023333f, -1.62857735f},
			{-8.61850739f, -0.257534623f, -2.48780465f},
			{-7.48897791f, 0.836947024f, -2.17553926f},
			{-6.28466797f, 1.53784180f, -1.55349731f},
			{-9.28733826f, -2.43306065f, -1.33127916f},
			{-5.33093262f, 1.85986328f, -1.26861572f},
			{-8.90132236f, -2.71124601f, -1.38725543f},
			{-9.17060471f, -2.78321695f, -0.819307029f},
			{-7.84990644f, -2.40683627f, -2.12729597f},
			{-7.91611624f, -3.38985372f, -0.952804804f},
			{-7.87829638f, -3.03356981f, -1.37831473f},
			{-2.43495178f, 1.76181090f, -1.93872476f},
			{-5.54367924f, 1.81969070f, -0.745879471f},
			{-1.47680271f, 1.64132023f, -1.37136936f},
			{-7.19018078f, 0.901845515f, -0.857508898f},
			{-2.50409842f, 1.67723644f, -1.04755175f},
			{-5.39768934f, 1.59197235f, -0.603915393f},
			{-8.16507626f, 0.222914696f, -1.11252105f},
			{-8.16396332f, -0.777114809f, -0.289943308f},
			{-9.22219181f, -2.11877251f, -0.794639707f},
			{-8.59923363f, -0.829777956f, -0.520817995f},
			{-8.82921314f, -1.96039867f, -0.298297912f},
			{-7.40556860f, -1.55998039f, -0.250481576f},
			{-8.14031315f, -2.90432310f, -0.262226462f},
			{-6.85163212f, -3.75249863f, -0.688983977f},
			{-7.04014444f, -3.11551261f, -0.126083776f},
			{-6.32870483f, -3.58578491f, -0.493814230f},
			{-6.28338480f, -3.61894345f, -0.929058373f},
			{-2.62741208f, -3.09605527f, -1.33044124f},
			{-1.79794896f, -2.99078751f, -1.21258605f},
			{-2.96854925f, -2.36802864f, -0.369529188f},
			{-4.30677605f, -1.66542077f, -0.303167671f},
			{-4.98426914f, 0.696137369f, -0.565792441f},
			{-2.85455132f, -0.0173058268f, -0.637033641f},
			{-2.70905447f, 1.40332532f, -0.955382884f},
			{-2.02012229f, 1.18688262f, -0.943184257f},
			{-2.42252111f, 0.0620364472f, -0.474374950f},
			{-1.56289959f, 1.40382648f, -0.681109011f},
			{-2.39032364f, -1.25889599f, 0.0907319933f},
			{-2.19775391f, 0.174560547f, 0.274658203f},
			{-0.306748420f, 1.84567726f, -1.35950983f},
			{0.250524580f, 2.27636194f, -0.823237658f},
			{-1.23819447f, 1.49723995f, 0.237386853f},
			{-2.13792968f, -0.616698205f, 0.937073231f},
			{-0.347599745f, 1.71002686f, 0.470531255f},
			{-1.66023159f, -2.55999684f, -0.452154934f},
			{-1.58301616f, -2.24312091f, 0.468077749f},
			{-1.67812920f, 0.0274793506f, 0.998333335f},
			{-1.36489558f, -1.84878838f, 0.930600643f},
			{0.255698651f, -2.16175246f, 0.109909050f},
			{-0.787146091f, -1.07865918f, 0.862207949f},
			{-0.293874621f, -2.31530595f, 0.560731053f},
			{0.162645504f, 1.18506360f, 0.511415780f},
			{0.535789251f, -1.17199028f, 0.772845566f},
			{1.08959603f, -1.89062190f, 0.519282222f},
			{-0.329488188f, -2.73174953f, -0.953910708f},
			{3.05201745f, -1.60201991f, -0.293554634f},
			{1.98701501f, -1.01330709f, 0.433782279f},
			{4.13116980f, -1.43739676f, -0.0357451625f},
			{2.42845988f, 0.716189802f, 0.176060840f},
			{5.01461887f, -0.924357474f, -0.0785766020f},
			{2.14807916f, 1.47517538f, 0.204042286f},
			{8.51385689f, -0.236161187f, -0.669285059f},
			{1.14428675f, 2.08710790f, -0.255287379f},
			{6.16604090f, 0.561538637f, -0.544156194f},
			{6.23778296f, 0.760041595f, -0.706411839f},
			{4.30203772f, 1.36588299f, -1.17367637f},
			{8.15555477f, 0.428962231f, -1.73520589f},
			{12.0248108f, -0.490237832f, -1.43530607f},
			{11.9705954f, -0.198881984f, -1.88936043f},
			{10.0048714f, -0.699601173f, -3.28313756f},
			{11.2095804f, -0.147232190f, -2.45466590f},
			{16.6165028f, -0.696181834f, -3.11862493f},
			{11.7848577f, -1.81683028f, -3.73632550f},
			{11.8666601f, -1.22696412f, -3.14981008f},
			{15.7040558f, -0.663861752f, -3.65228081f},
			{14.0951519f, -1.07241631f, -3.61164665f},
			{13.9181585f, -2.03203130f, -3.63557220f},
			{14.9848728f, -1.53837466f, -3.98249745f},
			{15.0845509f, -3.46569848f, -3.95276761f},
			{16.0242348f, -2.97810125f, -4.17965460f},
			{17.6243019f, -4.46606350f, -4.79625034f},
			{17.4456787f, -3.35461426f, -5.32739258f},
			{18.7262955f, -3.89710188f, -5.70676517f},
			{16.9230785f, -2.57194042f, -5.40591431f},
			{22.7467842f, -3.61460471f, -6.97978115f},
			{18.4024391f, -1.30696046f, -6.12640095f},
			{24.8947697f, -0.243050858f, -8.32624245f},
			{20.7879944f, -1.08460391f, -6.77528811f},
			{21.7145271f, -0.556654394f, -6.90972757f},
			{17.4861660f, -1.25891256f, -5.46996546f},
			{24.8378277f, 0.196830407f, -7.98421001f},
			{18.6861229f, -0.568430483f, -5.20232296f},
			{23.9673500f, -0.0726047829f, -6.57525635f},
			{20.4216003f, -0.520928204f, -4.92496729f},
			{23.2731094f, -0.885929525f, -5.62632990f},
			{17.6102352f, -1.26811695f, -3.06619239f},
			{18.1110725f, -2.89222097f, -2.71354604f},
			{15.2493439f, -1.51500702f, -1.99904561f},
			{23.4795017f, -5.85999155f, -5.31351185f},
			{23.3095360f, -6.35326576f, -5.25977182f},
			{17.5398540f, -4.66228580f, -2.38013840f},
			{23.1760674f, -6.62616539f, -5.57958126f},
			{19.5313797f, -6.43315840f, -3.60291767f},
			{15.4807367f, -3.66881418f, -1.85560131f},
			{14.3018312f, -4.70082331f, -1.46323049f},
			{12.8267784f, -4.37319517f, -1.09716678f},
			{15.7109251f, -6.05440664f, -2.37732816f},
			{12.3741608f, -1.41286337f, -1.26915634f},
			{13.8603306f, -5.49541807f, -1.90867615f},
			{17.6683712f, -6.81585407f, -3.37876749f},
			{12.2477732f, -4.72466660f, -1.29786050f},
			{11.1379576f, -3.74680114f, -0.753773510f},
			{9.36099339f, -2.47074175f, -0.534102321f},
			{10.0452557f, -3.30079317f, -0.685146451f},
			{6.68738413f, -2.30865097f, -0.461158574f},
			{7.36793947f, -3.03040981f, -1.47094512f},
			{11.8176231f, -4.84084797f, -1.82727480f},
			{4.59196663f, -2.28209496f, -1.41259789f},
			{7.10861683f, -2.85069633f, -2.06923938f},
			{3.39536858f, -2.24798036f, -1.65714610f},
			{0.172218561f, -2.71166372f, -1.40673709f},
			{2.89844537f, -1.88502884f, -2.27524662f},
			{-2.25631189f, -2.96263766f, -1.67407131f},
			{4.27257490f, -1.76102304f, -2.35111189f},
			{-1.88132656f, -2.04832077f, -2.39434218f},
			{-6.50054932f, -3.11684728f, -1.45549214f},
			{-6.90812969f, -1.50060546f, -3.38022733f},
			{-2.73349977f, -0.821462870f, -4.29114628f},
			{1.80500829f, -1.06030691f, -3.12149286f},
			{4.53264713f, -1.14842916f, -3.26621342f},
			{7.26582527f, -2.38612676f, -2.95009947f},
			{11.0593281f, -3.88886213f, -3.55232120f},
			{13.7664843f, -5.74423409f, -2.92658687f},
			{17.2798290f, -6.74407482f, -4.16304970f},
			{22.1492062f, -6.65115118f, -6.29792738f},
			{ 26.2703266f, -5.87738276f, -8.47085190f }
		});

		TArray<TVec3<int32>> Indices(
		{
			{1, 0, 2},
			{3, 0, 1},
			{0, 3, 4},
			{5, 4, 3},
			{4, 6, 0},
			{7, 0, 6},
			{7, 6, 8},
			{4, 5, 9},
			{5, 10, 9},
			{4, 11, 6},
			{4, 9, 11},
			{12, 8, 6},
			{6, 11, 12},
			{9, 10, 13},
			{10, 14, 13},
			{9, 15, 11},
			{9, 13, 15},
			{13, 14, 16},
			{17, 16, 14},
			{13, 18, 15},
			{18, 13, 16},
			{19, 11, 15},
			{19, 15, 18},
			{11, 19, 12},
			{16, 17, 20},
			{21, 20, 17},
			{16, 22, 18},
			{22, 16, 20},
			{23, 20, 21},
			{24, 23, 21},
			{23, 25, 20},
			{20, 25, 22},
			{26, 23, 24},
			{26, 25, 23},
			{27, 26, 24},
			{25, 28, 22},
			{28, 25, 26},
			{26, 27, 29},
			{29, 27, 30},
			{31, 29, 30},
			{31, 32, 29},
			{33, 26, 29},
			{28, 26, 34},
			{33, 34, 26},
			{35, 33, 29},
			{35, 29, 32},
			{34, 33, 36},
			{37, 33, 35},
			{33, 37, 36},
			{38, 35, 32},
			{34, 36, 39},
			{35, 38, 40},
			{40, 37, 35},
			{36, 41, 39},
			{37, 41, 36},
			{42, 34, 39},
			{34, 42, 28},
			{43, 39, 41},
			{43, 42, 39},
			{44, 41, 37},
			{44, 43, 41},
			{37, 45, 44},
			{37, 40, 45},
			{44, 46, 43},
			{42, 43, 46},
			{45, 47, 44},
			{46, 44, 47},
			{48, 45, 40},
			{47, 45, 48},
			{48, 40, 49},
			{38, 49, 40},
			{48, 50, 47},
			{51, 48, 49},
			{50, 48, 51},
			{49, 38, 52},
			{50, 53, 47},
			{53, 46, 47},
			{52, 54, 49},
			{49, 54, 51},
			{38, 55, 52},
			{32, 55, 38},
			{55, 32, 31},
			{55, 31, 30},
			{30, 56, 55},
			{52, 55, 56},
			{30, 27, 56},
			{24, 56, 27},
			{56, 24, 21},
			{56, 57, 52},
			{52, 57, 54},
			{14, 21, 17},
			{57, 58, 54},
			{51, 54, 58},
			{21, 14, 59},
			{59, 56, 21},
			{59, 57, 56},
			{59, 58, 57},
			{10, 59, 14},
			{59, 10, 60},
			{5, 60, 10},
			{3, 60, 5},
			{59, 61, 58},
			{60, 61, 59},
			{60, 3, 62},
			{1, 62, 3},
			{2, 62, 1},
			{62, 2, 63},
			{62, 63, 64},
			{60, 62, 65},
			{62, 64, 65},
			{60, 65, 61},
			{66, 65, 64},
			{66, 67, 65},
			{67, 68, 65},
			{61, 65, 68},
			{68, 67, 69},
			{61, 68, 70},
			{58, 61, 70},
			{69, 71, 68},
			{70, 68, 71},
			{69, 72, 71},
			{70, 71, 72},
			{72, 69, 73},
			{74, 72, 73},
			{58, 70, 75},
			{70, 72, 75},
			{58, 75, 51},
			{74, 76, 72},
			{75, 77, 51},
			{50, 51, 77},
			{72, 78, 75},
			{78, 77, 75},
			{78, 72, 76},
			{79, 50, 77},
			{79, 77, 78},
			{79, 53, 50},
			{80, 78, 76},
			{80, 79, 78},
			{76, 74, 81},
			{81, 80, 76},
			{82, 81, 74},
			{82, 83, 81},
			{80, 84, 79},
			{84, 53, 79},
			{83, 85, 81},
			{85, 80, 81},
			{83, 86, 85},
			{87, 84, 80},
			{80, 85, 87},
			{88, 53, 84},
			{87, 88, 84},
			{89, 85, 86},
			{89, 86, 90},
			{90, 8, 89},
			{12, 89, 8},
			{91, 85, 89},
			{89, 12, 91},
			{85, 92, 87},
			{92, 85, 91},
			{91, 12, 93},
			{93, 92, 91},
			{12, 19, 93},
			{87, 92, 94},
			{92, 93, 94},
			{93, 19, 94},
			{87, 94, 88},
			{94, 19, 95},
			{19, 18, 95},
			{88, 94, 96},
			{96, 94, 95},
			{53, 88, 96},
			{18, 97, 95},
			{97, 18, 22},
			{95, 98, 96},
			{98, 53, 96},
			{95, 97, 98},
			{53, 98, 46},
			{46, 98, 42},
			{97, 22, 99},
			{98, 97, 99},
			{98, 99, 42},
			{28, 99, 22},
			{99, 28, 42},
			{100, 63, 2},
			{100, 64, 63},
			{64, 100, 101},
			{102, 101, 100},
			{103, 102, 100},
			{104, 64, 101},
			{64, 104, 66},
			{101, 102, 105},
			{105, 104, 101},
			{102, 103, 106},
			{107, 106, 103},
			{102, 108, 105},
			{108, 102, 106},
			{109, 106, 107},
			{110, 109, 107},
			{111, 108, 106},
			{109, 111, 106},
			{112, 105, 108},
			{111, 112, 108},
			{113, 109, 110},
			{113, 111, 109},
			{114, 113, 110},
			{115, 114, 110},
			{115, 116, 114},
			{117, 111, 113},
			{114, 117, 113},
			{116, 118, 114},
			{119, 118, 116},
			{114, 118, 120},
			{117, 114, 120},
			{121, 118, 119},
			{118, 121, 120},
			{119, 122, 121},
			{123, 121, 122},
			{124, 121, 123},
			{121, 124, 125},
			{126, 125, 124},
			{127, 125, 126},
			{128, 127, 126},
			{128, 129, 127},
			{130, 121, 125},
			{125, 127, 131},
			{130, 125, 131},
			{132, 127, 129},
			{127, 132, 131},
			{121, 130, 133},
			{120, 121, 133},
			{130, 131, 134},
			{130, 135, 133},
			{134, 135, 130},
			{133, 136, 120},
			{134, 131, 137},
			{132, 137, 131},
			{133, 138, 136},
			{135, 138, 133},
			{139, 137, 132},
			{139, 140, 137},
			{137, 140, 141},
			{137, 141, 134},
			{132, 142, 139},
			{141, 143, 134},
			{142, 132, 144},
			{132, 129, 144},
			{142, 144, 145},
			{129, 146, 144},
			{144, 147, 145},
			{148, 144, 146},
			{147, 144, 148},
			{149, 134, 143},
			{149, 135, 134},
			{143, 150, 149},
			{143, 141, 150},
			{135, 149, 151},
			{141, 152, 150},
			{149, 150, 153},
			{151, 149, 153},
			{150, 152, 154},
			{153, 150, 154},
			{152, 141, 155},
			{155, 141, 140},
			{139, 155, 140},
			{156, 152, 155},
			{154, 152, 156},
			{157, 155, 139},
			{157, 139, 142},
			{157, 142, 145},
			{157, 158, 155},
			{158, 156, 155},
			{158, 157, 159},
			{158, 159, 156},
			{157, 145, 159},
			{159, 160, 156},
			{156, 160, 154},
			{161, 159, 145},
			{147, 161, 145},
			{162, 161, 147},
			{163, 159, 161},
			{163, 161, 162},
			{159, 163, 160},
			{163, 162, 164},
			{165, 164, 162},
			{166, 164, 165},
			{164, 166, 167},
			{163, 164, 168},
			{168, 164, 167},
			{160, 163, 169},
			{163, 168, 169},
			{160, 169, 170},
			{154, 160, 170},
			{168, 171, 169},
			{169, 171, 170},
			{154, 170, 172},
			{170, 171, 172},
			{154, 172, 153},
			{173, 153, 172},
			{172, 171, 173},
			{173, 151, 153},
			{173, 171, 174},
			{151, 173, 175},
			{174, 175, 173},
			{171, 176, 174},
			{171, 168, 176},
			{175, 174, 177},
			{177, 174, 176},
			{178, 151, 175},
			{135, 151, 178},
			{138, 135, 178},
			{175, 179, 178},
			{179, 138, 178},
			{175, 177, 180},
			{180, 179, 175},
			{177, 181, 180},
			{177, 176, 181},
			{182, 179, 180},
			{168, 183, 176},
			{168, 167, 183},
			{184, 181, 176},
			{183, 184, 176},
			{180, 181, 185},
			{180, 185, 182},
			{186, 181, 184},
			{186, 185, 181},
			{184, 183, 187},
			{185, 186, 188},
			{186, 184, 189},
			{189, 184, 187},
			{190, 185, 188},
			{182, 185, 190},
			{186, 191, 188},
			{191, 186, 189},
			{188, 191, 190},
			{187, 192, 189},
			{191, 189, 192},
			{187, 183, 193},
			{183, 167, 193},
			{194, 192, 187},
			{193, 194, 187},
			{195, 191, 192},
			{196, 192, 194},
			{192, 196, 195},
			{197, 191, 195},
			{190, 191, 197},
			{196, 198, 195},
			{198, 197, 195},
			{199, 190, 197},
			{190, 199, 182},
			{197, 198, 200},
			{201, 182, 199},
			{182, 201, 179},
			{197, 202, 199},
			{202, 197, 200},
			{203, 199, 202},
			{199, 203, 201},
			{202, 200, 203},
			{204, 179, 201},
			{201, 203, 204},
			{179, 204, 138},
			{205, 138, 204},
			{204, 203, 205},
			{138, 205, 136},
			{200, 206, 203},
			{203, 207, 205},
			{206, 207, 203},
			{136, 205, 208},
			{205, 207, 209},
			{205, 209, 208},
			{206, 210, 207},
			{211, 136, 208},
			{120, 136, 211},
			{120, 211, 117},
			{208, 209, 212},
			{211, 208, 212},
			{207, 213, 209},
			{213, 207, 210},
			{214, 212, 209},
			{214, 209, 213},
			{211, 212, 215},
			{212, 214, 216},
			{216, 215, 212},
			{211, 215, 217},
			{117, 211, 217},
			{216, 218, 215},
			{215, 218, 217},
			{117, 217, 219},
			{218, 219, 217},
			{219, 111, 117},
			{111, 219, 112},
			{219, 218, 220},
			{219, 221, 112},
			{220, 221, 219},
			{218, 222, 220},
			{221, 220, 222},
			{222, 218, 216},
			{112, 221, 223},
			{223, 105, 112},
			{223, 104, 105},
			{224, 221, 222},
			{223, 225, 104},
			{104, 225, 66},
			{226, 223, 221},
			{226, 225, 223},
			{226, 221, 224},
			{66, 225, 67},
			{226, 224, 227},
			{227, 225, 226},
			{222, 228, 224},
			{224, 228, 227},
			{216, 228, 222},
			{216, 214, 228},
			{214, 213, 228},
			{67, 225, 229},
			{229, 225, 227},
			{229, 69, 67},
			{229, 73, 69},
			{74, 73, 229},
			{227, 228, 230},
			{229, 227, 230},
			{213, 230, 228},
			{231, 74, 229},
			{230, 231, 229},
			{232, 230, 213},
			{230, 232, 231},
			{210, 232, 213},
			{74, 231, 233},
			{233, 231, 232},
			{233, 82, 74},
			{233, 83, 82},
			{234, 232, 210},
			{232, 234, 233},
			{235, 83, 233},
			{235, 233, 234},
			{234, 210, 236},
			{236, 235, 234},
			{210, 206, 236},
			{206, 200, 236},
			{235, 237, 83},
			{86, 83, 237},
			{237, 90, 86},
			{90, 237, 238},
			{238, 8, 90},
			{237, 235, 239},
			{8, 238, 240},
			{240, 7, 8},
			{241, 238, 237},
			{240, 238, 241},
			{237, 239, 241},
			{239, 235, 242},
			{235, 236, 242},
			{239, 242, 243},
			{242, 236, 243},
			{241, 239, 243},
			{236, 244, 243},
			{241, 243, 245},
			{244, 236, 246},
			{246, 236, 200},
			{245, 243, 247},
			{247, 243, 244},
			{245, 248, 241},
			{245, 247, 248},
			{240, 241, 248},
			{247, 244, 249},
			{246, 250, 244},
			{244, 250, 249},
			{200, 251, 246},
			{250, 246, 251},
			{251, 200, 198},
			{251, 198, 196},
			{252, 250, 251},
			{252, 249, 250},
			{196, 253, 251},
			{251, 253, 252},
			{249, 252, 253},
			{253, 196, 194},
			{254, 249, 253},
			{253, 194, 254},
			{249, 255, 247},
			{249, 254, 255},
			{255, 248, 247},
			{256, 254, 194},
			{194, 193, 256},
			{255, 254, 257},
			{256, 257, 254},
			{256, 193, 258},
			{257, 256, 258},
			{258, 193, 259},
			{167, 259, 193},
			{260, 258, 259},
			{259, 167, 261},
			{260, 259, 261},
			{167, 166, 261},
			{166, 165, 261},
			{262, 258, 260},
			{262, 257, 258},
			{260, 261, 263},
			{261, 165, 264},
			{261, 264, 263},
			{264, 165, 162},
			{147, 264, 162},
			{148, 264, 147},
			{146, 264, 148},
			{264, 146, 265},
			{146, 129, 265},
			{129, 128, 265},
			{126, 265, 128},
			{266, 264, 265},
			{265, 126, 266},
			{266, 263, 264},
			{126, 124, 123},
			{123, 266, 126},
			{266, 267, 263},
			{267, 266, 123},
			{263, 267, 260},
			{267, 123, 122},
			{267, 262, 260},
			{122, 268, 267},
			{262, 267, 268},
			{268, 122, 119},
			{268, 119, 116},
			{268, 116, 115},
			{268, 269, 262},
			{115, 269, 268},
			{257, 262, 269},
			{270, 269, 115},
			{257, 269, 270},
			{270, 115, 110},
			{270, 271, 257},
			{270, 110, 271},
			{271, 255, 257},
			{271, 248, 255},
			{110, 272, 271},
			{248, 271, 272},
			{272, 110, 107},
			{107, 103, 272},
			{272, 273, 248},
			{273, 272, 103},
			{240, 248, 273},
			{100, 273, 103},
			{273, 274, 240},
			{273, 100, 274},
			{240, 274, 7},
			{2, 274, 100},
			{0, 7, 274},
			{274, 2, 0}
		});

		TArray<uint16> Materials;
		for (int32 i = 0; i < Indices.Num(); ++i)
		{
			Materials.Emplace(0);
		}

		FTriangleMeshImplicitObjectPtr TriangleMesh( new FTriangleMeshImplicitObject(MoveTemp(TrimeshParticles), MoveTemp(Indices), MoveTemp(Materials)));
		TImplicitObjectScaled<FTriangleMeshImplicitObject> ScaledTriangleMesh = TImplicitObjectScaled<FTriangleMeshImplicitObject>(TriangleMesh, FVec3(50,50,50));

		const FVec3 X1 = { 0,0,-19.45 };
		const FVec3 X2 = X1 + FVec3(0, 0, 38.9);
		const FReal Radius = 25.895;
		const FCapsule Capsule = FCapsule(X1, X2, Radius);

		const FVec3 CapsuleToTrimeshTranslation = { 1818.55884, 27.8377075, -630.160645 };
		const FRigidTransform3 CapsuleToTrimesh(CapsuleToTrimeshTranslation, FQuat::Identity);

		const FVec3 TrimeshTranslation = { -1040.00000, 700.000000, 992.000000 };
		const FRigidTransform3 TrimeshTransform(TrimeshTranslation, FQuat::Identity);

		const FVec3 Dir(0, 0, -1);
		const FReal Length = 159.100098;

		FReal OutTime = -1;
		FVec3 Normal(0.0);
		FVec3 Position(0.0);
		int32 FaceIndex = -1;
		FVec3 FaceNormal(0.0);
		bool bResult = ScaledTriangleMesh.LowLevelSweepGeom(Capsule, CapsuleToTrimesh, Dir, Length, OutTime, Position, Normal, FaceIndex, FaceNormal, 0.0f, true);
		FVec3 WorldPosition = TrimeshTransform.TransformPositionNoScale(Position);

		EXPECT_EQ(bResult, true);
		EXPECT_EQ(FaceIndex, 415);
		EXPECT_NEAR(WorldPosition.X, 763.85413, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(WorldPosition.Y, 728.80212, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(WorldPosition.Z, 303.77856, KINDA_SMALL_NUMBER);
	}

	struct FSphereSweepFixture : public testing::Test
	{
		struct FSweepResultData
		{
			bool bResult;
			FReal TOI;
			FVec3 Position, Normal, FaceNormal;
			int32 FaceId;
		};

		struct FSweepCCDResultData
		{
			bool bResult;
			FReal TOI;
			FReal Phi;
			FVec3 Position, Normal, FaceNormal;
			int32 FaceId;
		};

		FSweepResultData SweepQuerySphere(const FImplicitObject& TestObject, const FRigidTransform3& TestObjectTransform, const FVec3& SweepStart, const FVec3& SweepDirection, const FReal SweepLength = 100)
		{
			const Chaos::FSphere Sphere(FVec3::ZeroVector, SphereRadius);
			const FRigidTransform3 StartTM(SweepStart, TRotation<FReal, 3>::Identity);
			const FVec3 NormalizedSweepDirection = SweepDirection.GetSafeNormal();

			FSweepResultData Result;
			Result.bResult = SweepQuery(TestObject, TestObjectTransform, Sphere, StartTM, NormalizedSweepDirection, SweepLength, Result.TOI, Result.Position, Result.Normal, Result.FaceId, Result.FaceNormal, 0.0, bComputeMTD);
			return Result;
		}

		FSweepResultData SweepQuerySphere(const FImplicitObject& TestObject, const FVec3& SweepStart, const FVec3& SweepDirection, const FReal SweepLength = 100)
		{
			return SweepQuerySphere(TestObject, FRigidTransform3(), SweepStart, SweepDirection, SweepLength);
		}

		FReal SphereRadius = 25;
		bool bComputeMTD = true;
	};

	struct FTriangleMeshSweepFixture : public FSphereSweepFixture
	{
		struct FMeshInitData
		{
			FTriangleMeshImplicitObject::ParticlesType TrimeshParticles;
			TArray<TVec3<int32>> Indices;
			TArray<uint16> Materials;
			TUniquePtr<TArray<int32>> ExternalFaceIndexMap;

			void AutoBuildMaterials()
			{
				Materials.Empty();
				for (int32 i = 0; i < Indices.Num(); ++i)
				{
					Materials.Emplace(0);
				}
			}

			void AutoBuildExternalFaceIndices()
			{
				ExternalFaceIndexMap = MakeUnique<TArray<int32>>();
				for (int32 I = 0; I < Indices.Num(); ++I)
				{
					ExternalFaceIndexMap->Add(I);
				}
			}
		};

		static FMeshInitData CreateSimpleTriInitData()
		{
			FMeshInitData Result;
			Result.TrimeshParticles = FTriangleMeshImplicitObject::ParticlesType(
			{
				{0, 0, 0},
				{100, 0, 0},
				{100, 100, 0},
			});
			Result.Indices =
			{
				{0, 1, 2},
			};
			Result.AutoBuildMaterials();
			return Result;
		}

		static FMeshInitData CreateSimpleQuadInitData()
		{
			FMeshInitData Result;
			Result.TrimeshParticles = FTriangleMeshImplicitObject::ParticlesType(
			{
			   {0, 0, 0},
			   {100, 0, 0},
			   {100, 100, 0},
			   {0, 100, 0},
			});
			Result.Indices =
			{
				{0, 1, 2},
				{0, 2, 3},
			};
			Result.AutoBuildMaterials();
			return Result;
		}

		FSweepResultData SweepSphere(const FTriangleMeshImplicitObject& TriangleMesh, const FVec3& SweepStart, const FVec3& SweepDirection, const FReal SweepLength = 100)
		{
			const Chaos::FSphere Sphere(FVec3::ZeroVector, SphereRadius);
			const FRigidTransform3 StartTM(SweepStart, TRotation<FReal, 3>::Identity);
			const FVec3 NormalizedSweepDirection = SweepDirection.GetSafeNormal();

			FSweepResultData Result;
			Result.bResult = TriangleMesh.SweepGeom(Sphere, StartTM, NormalizedSweepDirection, SweepLength, Result.TOI, Result.Position, Result.Normal, Result.FaceId, Result.FaceNormal, 0.0, bComputeMTD);
			return Result;
		}

		FSweepResultData SweepSphere(FMeshInitData& InitData, const FVec3& SweepStart, const FVec3& SweepDirection, const FReal SweepLength = 100)
		{
			FTriangleMeshImplicitObject TriangleMesh(MoveTemp(InitData.TrimeshParticles), MoveTemp(InitData.Indices), MoveTemp(InitData.Materials), nullptr, nullptr, true);
			return SweepSphere(TriangleMesh, SweepStart, SweepDirection, SweepLength);
		}
	};

	TEST_F(FTriangleMeshSweepFixture, SphereWithInitialIntersectionOfTwoTriangles_SweepAgainstTriMesh_CorrectTriangleIsHit)
	{
		// Test a straight down sweep where the sphere has an initial intersection with both triangles.
		FMeshInitData InitData = CreateSimpleQuadInitData();
		FTriangleMeshImplicitObject TriangleMesh(MoveTemp(InitData.TrimeshParticles), MoveTemp(InitData.Indices), MoveTemp(InitData.Materials), nullptr, nullptr, true);

		// Test where tri0 has the first TOI.
		FSweepResultData Result = SweepSphere(TriangleMesh, FVec3(60, 50, 10), FVec3(0, 0, -1));
		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, -15, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(60, 50, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1), KINDA_SMALL_NUMBER);
		EXPECT_EQ(Result.FaceId, 0);

		// Test where tri1 has the first TOI.
		Result = SweepSphere(TriangleMesh, FVec3(40, 50, 10), FVec3(0, 0, -1));
		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, -15, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(40, 50, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1), KINDA_SMALL_NUMBER);
		EXPECT_EQ(Result.FaceId, 1);
	}

	TEST_F(FTriangleMeshSweepFixture, SphereWithInitialIntersectionOfOneTriangle_SweepAgainstTriMesh_CorrectTriangleIsHit)
	{
		// Test a diagonal sweep where one triangle has an initial intersection and the other has a positive TOI.
		FMeshInitData InitData = CreateSimpleQuadInitData();
		FTriangleMeshImplicitObject TriangleMesh(MoveTemp(InitData.TrimeshParticles), MoveTemp(InitData.Indices), MoveTemp(InitData.Materials), nullptr, nullptr, true);

		// Test where tri0 has the initial intersection.
		FSweepResultData Result = SweepSphere(TriangleMesh, FVec3(90, 50, 10), FVec3(0, 0, -1));
		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, -15, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(90, 50, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1), KINDA_SMALL_NUMBER);
		EXPECT_EQ(Result.FaceId, 0);

		// Test where tri1 has the initial intersection.
		Result = SweepSphere(TriangleMesh, FVec3(10, 50, 10), FVec3(0, 0, -1));
		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, -15, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(10, 50, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1), KINDA_SMALL_NUMBER);
		EXPECT_EQ(Result.FaceId, 1);
	}

	TEST_F(FTriangleMeshSweepFixture, SphereWithZeroLengthSweep_TestAllAxes_HitIsExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FTriangleMeshImplicitObject TriangleMesh(MoveTemp(InitData.TrimeshParticles), MoveTemp(InitData.Indices), MoveTemp(InitData.Materials), nullptr, nullptr, true);

		// Test each cardinal axis
		FSweepResultData Result = SweepSphere(TriangleMesh, FVec3(20, 20, 0), FVec3(-1, 0, 0), 0);
		EXPECT_EQ(Result.bResult, true);

		Result = SweepSphere(TriangleMesh, FVec3(20, 20, 0), FVec3(1, 0, 0), 0);
		EXPECT_EQ(Result.bResult, true);

		Result = SweepSphere(TriangleMesh, FVec3(20, 20, 0), FVec3(0, -1, 0), 0);
		EXPECT_EQ(Result.bResult, true);

		Result = SweepSphere(TriangleMesh, FVec3(20, 20, 0), FVec3(0, 1, 0), 0);
		EXPECT_EQ(Result.bResult, true);

		Result = SweepSphere(TriangleMesh, FVec3(20, 20, 0), FVec3(0, 0, -1), 0);
		EXPECT_EQ(Result.bResult, true);

		Result = SweepSphere(TriangleMesh, FVec3(20, 20, 0), FVec3(0, 0, 1), 0);
		EXPECT_EQ(Result.bResult, true);

		// Test the zero vector direction
		Result = SweepSphere(TriangleMesh, FVec3(20, 20, 0), FVec3(0, 0, 0), 0);
		EXPECT_EQ(Result.bResult, true);

		Result = SweepSphere(TriangleMesh, FVec3(20, 20, 0), FVec3(0, 0, 0), 1);
		EXPECT_EQ(Result.bResult, true);
	}

	// All front-face hits (sweeping opposed to the normal) should generate valid hits.
	TEST_F(FTriangleMeshSweepFixture, SphereInFront_SweepOpposedToNormal_HitIsExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		const FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, 50), FVec3(0, 0, -1));

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 25, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(20, 20, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1), KINDA_SMALL_NUMBER);
		EXPECT_EQ(Result.FaceId, 0);
	}

	TEST_F(FTriangleMeshSweepFixture, SphereInFrontWithInitialOverlap_SweepOpposedToNormal_HitIsExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, 10), FVec3(0, 0, -1));

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, -15, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(20, 20, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1), KINDA_SMALL_NUMBER);
		EXPECT_EQ(Result.FaceId, 0);
	}

	TEST_F(FTriangleMeshSweepFixture, SphereBehindWithInitialOverlap_SweepOpposedToNormal_HitIsExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, -10), FVec3(0, 0, -1));

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, -15, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(20, 20, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, -1), KINDA_SMALL_NUMBER);
		EXPECT_EQ(Result.FaceId, 0);
	}

	struct FTriangleMeshBackfaceSweepFixture : public FTriangleMeshSweepFixture
	{
		static constexpr FReal ValueWithinParallelEpsilon = UE_SMALL_NUMBER;
		static constexpr FReal ValueOutsideParallelEpsilon = 4 * UE_KINDA_SMALL_NUMBER;
	};

	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereBehind_SweepOpposedToNormal_HitIsNotExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, -50), FVec3(0, 0, -1));

		EXPECT_EQ(Result.bResult, false);
	}

	// All back-face hits (sweeping along the normal) should not generate a valid hit.
	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereBehind_SweepAlongNormal_HitIsNotExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, -50), FVec3(0, 0, 1));

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereBehindWithInitialOverlap_SweepAlongNormal_HitIsNotExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, -10), FVec3(0, 0, 1));

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereInFrontWithInitialOverlap_SweepAlongNormal_HitIsNotExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, 10), FVec3(0, 0, 1));

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereInFront_SweepAlongNormal_HitIsNotExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, 50), FVec3(0, 0, 1));

		EXPECT_EQ(Result.bResult, false);
	}

	// Parallel checks are more interesting. If there is an initial overlap then no hit should be generated. 
	// This parallel checks uses an epsilon to determine the boundaries.
	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereWithInitialOverlap_SweepExactlyParallel_HitIsExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, 10), FVec3(1, 0, 0));

		EXPECT_EQ(Result.bResult, true);
	}

	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereWithInitialOverlap_SweepOpposedToNormalWithinParallelEpsilon_HitIsExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, 10), FVec3(1, 0, ValueWithinParallelEpsilon));

		EXPECT_EQ(Result.bResult, true);
	}

	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereWithInitialOverlap_SweepOpposedToNormalOutsideParallelEpsilon_HitIsNotExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, 10), FVec3(1, 0, ValueOutsideParallelEpsilon));

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereWithInitialOverlap_SweepAlongNormalWithinParallelEpsilon_HitIsExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, 10), FVec3(1, 0, -ValueWithinParallelEpsilon));

		EXPECT_EQ(Result.bResult, true);
	}

	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereWithInitialOverlap_SweepAlongNormalOutsideParallelEpsilon_HitIsExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(20, 20, 10), FVec3(1, 0, 4 * -ValueOutsideParallelEpsilon));

		EXPECT_EQ(Result.bResult, true);
	}

	// Need a few extra tests for parallel to validate that no initial overlap always returns false.
	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereInAabbWithNoInitialOverlap_SweepExactlyParallel_HitIsNotExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(0, 100, 0), FVec3(1, -1, 0));

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereInAabbWithNoInitialOverlap_AlongNormalWithinPositiveEpsilon_HitIsNotExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(0, 100, 0), FVec3(1, -1, ValueWithinParallelEpsilon));

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FTriangleMeshBackfaceSweepFixture, SphereInAabbWithNoInitialOverlap_AlongNormalWithinNegativeEpsilon_HitIsNotExpected)
	{
		FMeshInitData InitData = CreateSimpleTriInitData();
		FSweepResultData Result = SweepSphere(InitData, FVec3(0, 100, 0), FVec3(1, -1, -ValueWithinParallelEpsilon));

		EXPECT_EQ(Result.bResult, false);
	}

	// Some extra tests for negative scales
	struct FScaledTriMeshSweepFixture : public FTriangleMeshBackfaceSweepFixture
	{
		static FMeshInitData CreateSimpleTriInitData()
		{
			FMeshInitData Result;
			// Use a slightly different triangle mesh than above. This allows easy negation of the mesh without having to move the sweep's location.
			Result.TrimeshParticles = FTriangleMeshImplicitObject::ParticlesType(
				{
					{-100, -100, 0},
					{100, -100, 0},
					{0, 100, 0},
				});
			Result.Indices =
			{
				{0, 1, 2},
			};
			Result.AutoBuildMaterials();
			return Result;
		}

		static FMeshInitData CreateTriInitData(const FVec3& P0, const FVec3& P1, const FVec3& P2)
		{
			FMeshInitData Result;
			Result.TrimeshParticles = FTriangleMeshImplicitObject::ParticlesType({P0, P1, P2});
			Result.Indices = { {0, 1, 2}, };
			Result.AutoBuildMaterials();
			return Result;
		}

		FSweepResultData SweepSphere(const TImplicitObjectScaled<FTriangleMeshImplicitObject>& TriangleMesh, const FVec3& SweepStart, const FVec3& SweepDirection, const FReal SweepLength = 100)
		{
			const Chaos::FSphere Sphere(FVec3::ZeroVector, SphereRadius);
			const FRigidTransform3 StartTM(SweepStart, TRotation<FReal, 3>::Identity);
			const FVec3 NormalizedSweepDirection = SweepDirection.GetSafeNormal();

			FSweepResultData Result;
			Result.bResult = SweepQuery(TriangleMesh, FRigidTransform3(), Sphere, StartTM, NormalizedSweepDirection, SweepLength, Result.TOI, Result.Position, Result.Normal, Result.FaceId, Result.FaceNormal, 0.0, true);
			return Result;
		}

		FSweepResultData SweepSphere(FMeshInitData& InitData, const FVec3& SweepStart, const FVec3& SweepDirection, const FReal SweepLength = 100)
		{
			FTriangleMeshImplicitObject TriangleMesh(MoveTemp(InitData.TrimeshParticles), MoveTemp(InitData.Indices), MoveTemp(InitData.Materials), nullptr, nullptr, true);
			// Unfortunately, it is not possible to currently call FTriangleMeshImplicitObject::SweepGeom directly. 
			// There's code inside of TImplicitObjectScaled that negates some inputs values (the direction) that is necessary for the tests to work (follows the editor path).
			TImplicitObjectScaled<FTriangleMeshImplicitObject> ScaledMesh(&TriangleMesh, TriMeshScale);
			return SweepSphere(ScaledMesh, SweepStart, SweepDirection, SweepLength);
		}

		FSweepCCDResultData SweepSphereCCD(FMeshInitData& InitData, const FVec3& SweepStart, const FVec3& SweepDirection, const FReal SweepLength = 100)
		{
			const Chaos::FSphere Sphere(FVec3::ZeroVector, SphereRadius);
			const FRigidTransform3 StartTM(SweepStart, TRotation<FReal, 3>::Identity);
			const FVec3 NormalizedSweepDirection = SweepDirection.GetSafeNormal();
			const FRigidTransform3 BToATM = StartTM.GetRelativeTransformNoScale(FRigidTransform3());

			FTriangleMeshImplicitObject TriangleMesh(MoveTemp(InitData.TrimeshParticles), MoveTemp(InitData.Indices), MoveTemp(InitData.Materials), nullptr, nullptr, true);
			TImplicitObjectScaled<FTriangleMeshImplicitObject> ScaledMesh(&TriangleMesh, TriMeshScale);

			FSweepCCDResultData Result;
			Result.bResult = ScaledMesh.LowLevelSweepGeomCCD(Sphere, BToATM, NormalizedSweepDirection, SweepLength, 0, 0, Result.TOI, Result.Phi, Result.Position, Result.Normal, Result.FaceId, Result.FaceNormal);
			return Result;
		}

		FSweepResultData SweepSphereVsSimpleTri(const FVec3& SweepStart, const FVec3& SweepDirection, const FReal SweepLength = 100)
		{
			FMeshInitData InitData = CreateSimpleTriInitData();
			return SweepSphere(InitData, SweepStart, SweepDirection, SweepLength);
		}

		FVec3 TriMeshScale = FVec3(1, 1, 1);
	};

	// Test sweeps along the original front face
	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeXAxis_SweepInFrontOpposedToNormal_HitIsExpected)
	{
		// Negating the x axis should flip the winding order, but up is still the front face
		TriMeshScale = FVec3(-1, 1, 1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, 50), FVec3(0, 0, -1));

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 25, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(20, 20, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeYAxis_SweepInFrontOpposedToNormal_HitIsExpected)
	{
		// Negating the y axis should flip the winding order, but up is still the front face
		TriMeshScale = FVec3(1, -1, 1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, 50), FVec3(0, 0, -1));

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 25, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(20, 20, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeZAxis_SweepInBackAlongToNormal_HitIsNotExpected)
	{
		// Negating the z axis should cause down to be the front face
		TriMeshScale = FVec3(1, 1, -1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, 50), FVec3(0, 0, -1));

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeXYAxis_SweepInFrontOpposedToNormal_HitIsExpected)
	{
		TriMeshScale = FVec3(-1, -1, 1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, 50), FVec3(0, 0, -1));

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 25, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(20, 20, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeXZAxis_SweepInFrontOpposedToNormal_HitIsNotExpected)
	{
		TriMeshScale = FVec3(-1, 1, -1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, 50), FVec3(0, 0, -1));

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeYZAxis_SweepInFrontOpposedToNormal_HitIsNotExpected)
	{
		TriMeshScale = FVec3(1, -1, -1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, 50), FVec3(0, 0, -1));

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeXYZAxis_SweepInFrontOpposedToNormal_HitIsNotExpected)
	{
		TriMeshScale = FVec3(-1, -1, -1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, 50), FVec3(0, 0, -1));

		EXPECT_EQ(Result.bResult, false);
	}

	// Test sweeps along the original back face
	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeXAxis_SweepOnBackAlongNormal_HitIsNotExpected)
	{
		// Negating the x axis should flip the winding order, but up is still the front face
		TriMeshScale = FVec3(-1, 1, 1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, -50), FVec3(0, 0, 1));

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeYAxis_SweepOnBackAlongNormal_HitIsNotExpected)
	{
		// Negating the y axis should flip the winding order, but up is still the front face
		TriMeshScale = FVec3(1, -1, 1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, -50), FVec3(0, 0, 1));

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeZAxis_SweepOnBackAlongNormal_HitIsExpected)
	{
		// Negating the z axis should cause down to be the front face
		TriMeshScale = FVec3(1, 1, -1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, -50), FVec3(0, 0, 1));

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 25, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(20, 20, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, -1), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeXYAxis_SweepOnBackAlongNormal_HitIsNotExpected)
	{
		TriMeshScale = FVec3(-1, -1, 1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, -50), FVec3(0, 0, 1));

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeXZAxis_SweepOnBackAlongNormal_HitIsExpected)
	{
		TriMeshScale = FVec3(-1, 1, -1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, -50), FVec3(0, 0, 1));

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 25, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(20, 20, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, -1), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeYZAxis_SweepOnBackAlongNormal_HitIsExpected)
	{
		TriMeshScale = FVec3(1, -1, -1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, -50), FVec3(0, 0, 1));

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 25, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(20, 20, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, -1), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeXYZAxis_SweepOnBackAlongNormal_HitIsExpected)
	{
		TriMeshScale = FVec3(-1, -1, -1);
		const FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(20, 20, -50), FVec3(0, 0, 1));

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 25, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(20, 20, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, -1), KINDA_SMALL_NUMBER);
	}

	// Test parallel sweeps
	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeXAxis_SphereWithInitialOverlap_SweepOpposedToNormalWithinParallelEpsilon_HitIsExpected)
	{
		TriMeshScale = FVec3(-1, 1, 1);
		FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(0, 0, 10), FVec3(1, 0, ValueWithinParallelEpsilon));

		EXPECT_EQ(Result.bResult, true);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeZAxis_SphereWithInitialOverlap_SweepOpposedToNormalWithinParallelEpsilon_HitIsExpected)
	{
		TriMeshScale = FVec3(1, 1, -1);
		FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(0, 0, 10), FVec3(1, 0, ValueWithinParallelEpsilon));

		EXPECT_EQ(Result.bResult, true);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeXAxis_SphereWithInitialOverlap_SweepAlongNormalWithinParallelEpsilon_HitIsExpected)
	{
		TriMeshScale = FVec3(-1, 1, 1);
		FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(0, 0, 10), FVec3(1, 0, -ValueWithinParallelEpsilon));

		EXPECT_EQ(Result.bResult, true);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithNegativeZAxis_SphereWithInitialOverlap_SweepAlongNormalWithinParallelEpsilon_HitIsExpected)
	{
		TriMeshScale = FVec3(1, 1, -1);
		FSweepResultData Result = SweepSphereVsSimpleTri(FVec3(0, 0, 10), FVec3(1, 0, -ValueWithinParallelEpsilon));

		EXPECT_EQ(Result.bResult, true);
	}

	// Do a bunch of tests with a slanted triangle and different non-uniform scales
	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithUniformScale_Sweep_ResultsAreExpected)
	{
		SphereRadius = 1;
		TriMeshScale = FVec3(1, 1, 1);
		FMeshInitData InitData = CreateTriInitData(FVec3(5, 0, 0), FVec3(0, 5, 0), FVec3(0, 0, 5));
		FVec3 SweepStart = FVec3(2.5, 10, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepResultData Result = SweepSphere(InitData, SweepStart, SweepDir);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 8.2679491043090820, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(1.9226497411727905, 1.1547006368637085, 1.9226497411727905), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(1, 1, 1).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithDoubleXScale_Sweep_ResultsAreExpected)
	{
		SphereRadius = 1;
		TriMeshScale = FVec3(2, 1, 1);
		FMeshInitData InitData = CreateTriInitData(FVec3(5, 0, 0), FVec3(0, 5, 0), FVec3(0, 0, 5));
		FVec3 SweepStart = FVec3(2.5, 10, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepResultData Result = SweepSphere(InitData, SweepStart, SweepDir);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 7.2500000000000000, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(2.1666667461395264, 2.0833334922790527, 1.8333333730697632), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(.5, 1, 1).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithHalfXScale_Sweep_ResultsAreExpected)
	{
		SphereRadius = 1;
		TriMeshScale = FVec3(.5, 1, 1);
		FMeshInitData InitData = CreateTriInitData(FVec3(10, 0, 0), FVec3(0, 10, 0), FVec3(0, 0, 10));
		FVec3 SweepStart = FVec3(2.5, 10, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepResultData Result = SweepSphere(InitData, SweepStart, SweepDir);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 5.0505099296569824, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(1.6835033893585205, 4.5412416458129883, 2.0917518138885498), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(2, 1, 1).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithZeroXScale_Sweep_ResultsAreExpected)
	{
		SphereRadius = 1;
		TriMeshScale = FVec3(0, 1, 1);
		FMeshInitData InitData = CreateTriInitData(FVec3(5, 0, 0), FVec3(0, 5, 0), FVec3(0, 0, 5));
		FVec3 SweepStart = FVec3(10, 2.5, 2.5);
		FVec3 SweepDir = FVec3(-1, 0, 0);
		FSweepResultData Result = SweepSphere(InitData, SweepStart, SweepDir);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 9, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(0, 2.5, 2.5), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(1, 0, 0).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithZeroYScale_Sweep_ResultsAreExpected)
	{
		SphereRadius = 1;
		TriMeshScale = FVec3(1, 0, 1);
		FMeshInitData InitData = CreateTriInitData(FVec3(5, 0, 0), FVec3(0, 5, 0), FVec3(0, 0, 5));
		FVec3 SweepStart = FVec3(2.5, 10, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepResultData Result = SweepSphere(InitData, SweepStart, SweepDir);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 9, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(2.5, 0, 2.5), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 1, 0).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithZeroZScale_Sweep_ResultsAreExpected)
	{
		SphereRadius = 1;
		TriMeshScale = FVec3(1, 1, 0);
		FMeshInitData InitData = CreateTriInitData(FVec3(5, 0, 0), FVec3(0, 5, 0), FVec3(0, 0, 5));
		FVec3 SweepStart = FVec3(2.5, 2.5, 10);
		FVec3 SweepDir = FVec3(0, 0, -1);
		FSweepResultData Result = SweepSphere(InitData, SweepStart, SweepDir);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 9, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(2.5, 2.5, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithYNormalAndZeroYScale_Sweep_ResultsAreExpected)
	{
		// This is a special case where TriMeshScale * HitNormal = (1, 0, 1) * (0, 1, 0) = (0, 0, 0).
		// This was a previous bug that would return (0, 0, 0) instead of the expected (0, 1, 0).
		SphereRadius = 1;
		TriMeshScale = FVec3(1, 0, 1);
		FMeshInitData InitData = CreateTriInitData(FVec3(5, 0, 0), FVec3(0, 0, 0), FVec3(0, 0, 5));
		FVec3 SweepStart = FVec3(2.5, 10, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepResultData Result = SweepSphere(InitData, SweepStart, SweepDir);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 9, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(2.5, 0, 2.5), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 1, 0).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithLargeScaleThatBarelyHits_Sweep_ResultsAreExpected)
	{
		// This test case helps to ensure that the TOI is correctly handled internally. The trimesh does some back and forth scaling of time and this was added to catch a breaking case.
		FReal SweepLength = 10;
		SphereRadius = 1;
		TriMeshScale = FVec3(50, 50, 50);
		FMeshInitData InitData = CreateTriInitData(FVec3(0, 1, 0), FVec3(0, 1, 5), FVec3(5, 1, 0));
		FVec3 SweepStart = FVec3(2.5, 60.9, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepResultData Result = SweepSphere(InitData, SweepStart, SweepDir, SweepLength);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 9.9, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(2.5, 50, 2.5), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 1, 0).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}
	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithLargeScaleThatBarelyMisses_Sweep_ResultsAreExpected)
	{
		// This test case helps to ensure that the TOI is correctly handled internally. The trimesh does some back and forth scaling of time and this was added to catch a breaking case.
		FReal SweepLength = 10;
		SphereRadius = 1;
		TriMeshScale = FVec3(50, 50, 50);
		FMeshInitData InitData = CreateTriInitData(FVec3(0, 1, 0), FVec3(0, 1, 5), FVec3(5, 1, 0));
		FVec3 SweepStart = FVec3(2.5, 61.1, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepResultData Result = SweepSphere(InitData, SweepStart, SweepDir, SweepLength);

		EXPECT_EQ(Result.bResult, false);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithUniformScale_SweepCCD_ResultsAreExpected)
	{
		const FReal SweepLength = 100;
		SphereRadius = 1;
		TriMeshScale = FVec3(1, 1, 1);
		FMeshInitData InitData = CreateTriInitData(FVec3(5, 0, 0), FVec3(0, 5, 0), FVec3(0, 0, 5));
		FVec3 SweepStart = FVec3(2.5, 10, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepCCDResultData Result = SweepSphereCCD(InitData, SweepStart, SweepDir, SweepLength);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 8.2679491043090820 / SweepLength, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Result.Phi, 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(1.9226497411727905, 1.1547006368637085, 1.9226497411727905), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(1, 1, 1).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithDoubleXScale_SweepCCD_ResultsAreExpected)
	{
		const FReal SweepLength = 100;
		SphereRadius = 1;
		TriMeshScale = FVec3(2, 1, 1);
		FMeshInitData InitData = CreateTriInitData(FVec3(5, 0, 0), FVec3(0, 5, 0), FVec3(0, 0, 5));
		FVec3 SweepStart = FVec3(2.5, 10, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepCCDResultData Result = SweepSphereCCD(InitData, SweepStart, SweepDir, SweepLength);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 7.2500000000000000 / SweepLength, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Result.Phi, 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(2.1666667461395264, 2.0833334922790527, 1.8333333730697632), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(.5, 1, 1).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithHalfXScale_SweepCCD_ResultsAreExpected)
	{
		const FReal SweepLength = 100;
		SphereRadius = 1;
		TriMeshScale = FVec3(.5, 1, 1);
		FMeshInitData InitData = CreateTriInitData(FVec3(10, 0, 0), FVec3(0, 10, 0), FVec3(0, 0, 10));
		FVec3 SweepStart = FVec3(2.5, 10, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepCCDResultData Result = SweepSphereCCD(InitData, SweepStart, SweepDir, SweepLength);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 5.0505099296569824 / SweepLength, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Result.Phi, 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(1.6835033893585205, 4.5412416458129883, 2.0917518138885498), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(2, 1, 1).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithZeroXScale_SweepCCD_ResultsAreExpected)
	{
		const FReal SweepLength = 100;
		SphereRadius = 1;
		TriMeshScale = FVec3(0, 1, 1);
		FMeshInitData InitData = CreateTriInitData(FVec3(5, 0, 0), FVec3(0, 5, 0), FVec3(0, 0, 5));
		FVec3 SweepStart = FVec3(10, 2.5, 2.5);
		FVec3 SweepDir = FVec3(-1, 0, 0);
		FSweepCCDResultData Result = SweepSphereCCD(InitData, SweepStart, SweepDir, SweepLength);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 9.0 / SweepLength, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Result.Phi, 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(0, 2.5, 2.5), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(1, 0, 0).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithZeroYScale_SweepCCD_ResultsAreExpected)
	{
		const FReal SweepLength = 100;
		SphereRadius = 1;
		TriMeshScale = FVec3(1, 0, 1);
		FMeshInitData InitData = CreateTriInitData(FVec3(5, 0, 0), FVec3(0, 5, 0), FVec3(0, 0, 5));
		FVec3 SweepStart = FVec3(2.5, 10, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepCCDResultData Result = SweepSphereCCD(InitData, SweepStart, SweepDir, SweepLength);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 9.0 / SweepLength, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Result.Phi, 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(2.5, 0, 2.5), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 1, 0).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithZeroZScale_SweepCCD_ResultsAreExpected)
	{
		const FReal SweepLength = 100;
		SphereRadius = 1;
		TriMeshScale = FVec3(1, 1, 0);
		FMeshInitData InitData = CreateTriInitData(FVec3(5, 0, 0), FVec3(0, 5, 0), FVec3(0, 0, 5));
		FVec3 SweepStart = FVec3(2.5, 2.5, 10);
		FVec3 SweepDir = FVec3(0, 0, -1);
		FSweepCCDResultData Result = SweepSphereCCD(InitData, SweepStart, SweepDir, SweepLength);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 9.0 / SweepLength, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Result.Phi, 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(2.5, 2.5, 0), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithYNormalAndZeroYScale_SweepCCD_ResultsAreExpected)
	{
		// This is a special case where TriMeshScale * HitNormal = (1, 0, 1) * (0, 1, 0) = (0, 0, 0).
		// This was a previous bug that would return (0, 0, 0) instead of the expected (0, 1, 0).
		const FReal SweepLength = 100;
		SphereRadius = 1;
		TriMeshScale = FVec3(1, 0, 1);
		FMeshInitData InitData = CreateTriInitData(FVec3(5, 0, 0), FVec3(0, 0, 0), FVec3(0, 0, 5));
		FVec3 SweepStart = FVec3(2.5, 10, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepCCDResultData Result = SweepSphereCCD(InitData, SweepStart, SweepDir, SweepLength);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 9.0 / SweepLength, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Result.Phi, 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(2.5, 0, 2.5), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 1, 0).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithLargeScaleThatBarelyHits_SweepCCD_ResultsAreExpected)
	{
		// This test case helps to ensure that the TOI is correctly handled internally. The trimesh does some back and forth scaling of time and this was added to catch a breaking case.
		FReal SweepLength = 10;
		SphereRadius = 1;
		TriMeshScale = FVec3(50, 50, 50);
		FMeshInitData InitData = CreateTriInitData(FVec3(0, 1, 0), FVec3(0, 1, 5), FVec3(5, 1, 0));
		FVec3 SweepStart = FVec3(2.5, 60.9, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepCCDResultData Result = SweepSphereCCD(InitData, SweepStart, SweepDir, SweepLength);

		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 9.9 / SweepLength, KINDA_SMALL_NUMBER);
		EXPECT_NEAR(Result.Phi, 0, KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Position, FVec3(2.5, 50, 2.5), KINDA_SMALL_NUMBER);
		EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 1, 0).GetUnsafeNormal(), KINDA_SMALL_NUMBER);
	}

	TEST_F(FScaledTriMeshSweepFixture, TriMeshWithLargeScaleThatBarelyMisses_SweepCCD_ResultsAreExpected)
	{
		// This test case helps to ensure that the TOI is correctly handled internally. The trimesh does some back and forth scaling of time and this was added to catch a breaking case.
		FReal SweepLength = 10;
		SphereRadius = 1;
		TriMeshScale = FVec3(50, 50, 50);
		FMeshInitData InitData = CreateTriInitData(FVec3(0, 1, 0), FVec3(0, 1, 5), FVec3(5, 1, 0));
		FVec3 SweepStart = FVec3(2.5, 61.1, 2.5);
		FVec3 SweepDir = FVec3(0, -1, 0);
		FSweepCCDResultData Result = SweepSphereCCD(InitData, SweepStart, SweepDir, SweepLength);

		EXPECT_EQ(Result.bResult, false);
	}

	struct FSweepCCDFixture : public FTriangleMeshSweepFixture
	{
		FSweepCCDResultData SweepSphereCCD(FTriangleMeshImplicitObject& TriangleMesh, const FVec3& SweepStart, const FVec3& SweepDirection, const FQuat& SweepRotation = FQuat::Identity, const FReal SweepLength = 100)
		{
			const Chaos::FSphere Sphere(FVec3::ZeroVector, SphereRadius);
			const FRigidTransform3 StartTM(SweepStart, SweepRotation);
			const FVec3 NormalizedSweepDirection = SweepDirection.GetSafeNormal();
			const FRigidTransform3 BToATM = StartTM.GetRelativeTransformNoScale(FRigidTransform3());

			FSweepCCDResultData Result;
			Result.bResult = TriangleMesh.SweepGeomCCD(Sphere, BToATM, NormalizedSweepDirection, SweepLength, IgnorePenetration, TargetPenetration, Result.TOI, Result.Phi, Result.Position, Result.Normal, Result.FaceId, Result.FaceNormal);
			return Result;
		}

		FReal IgnorePenetration = 0;
		FReal TargetPenetration = 0;
	};

	TEST_F(FSweepCCDFixture, SimpleTriMeshWithTwoHits_SweepCCD_ResultsAreExpected)
	{
		const FReal SweepLength = 1000;
		SphereRadius = 15;

		FMeshInitData InitData;
		InitData.TrimeshParticles = FTriangleMeshImplicitObject::ParticlesType
		({
			FVec3(0, 0, 0), FVec3(10, 0, 0), FVec3(0, 10, 0),
			FVec3(0, 0, 5), FVec3(10, 0, 5), FVec3(0, 10, 5),
		});
		InitData.Indices =
		{
			{0, 1, 2},
			{3, 4, 5},
		};
		InitData.AutoBuildMaterials();
		InitData.AutoBuildExternalFaceIndices();

		FTriangleMeshImplicitObject TriangleMesh(MoveTemp(InitData.TrimeshParticles), MoveTemp(InitData.Indices), MoveTemp(InitData.Materials), MoveTemp(InitData.ExternalFaceIndexMap), nullptr, true);

		FVec3 SweepStart = FVec3(2, 2, 50);
		FVec3 SweepDir = FVec3(0, 0, -1);
		FQuat SweepRotation = FQuat::Identity;

		// First do a test with no target penetration
		{
			TargetPenetration = 0;
			const double ExpectedTOI = (SweepStart.Z - SphereRadius - 5 + TargetPenetration) / SweepLength;

			FSweepCCDResultData Result = SweepSphereCCD(TriangleMesh, SweepStart, SweepDir, SweepRotation, SweepLength);

			EXPECT_EQ(Result.bResult, true);
			EXPECT_NEAR(Result.TOI, ExpectedTOI, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(Result.Phi, 0, KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.Position, FVec3(2, 2, 5), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1), KINDA_SMALL_NUMBER);
			EXPECT_EQ(Result.FaceId, 1);
		}

		// Then do another test with target penetration. The same triangle should be hit, but with different TOI and PHI.
		{
			TargetPenetration = 6.3500003814697266;
			const double ExpectedTOI = (SweepStart.Z - SphereRadius - 5 + TargetPenetration) / SweepLength;

			FSweepCCDResultData Result = SweepSphereCCD(TriangleMesh, SweepStart, SweepDir, SweepRotation, SweepLength);

			EXPECT_EQ(Result.bResult, true);
			EXPECT_NEAR(Result.TOI, ExpectedTOI, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(Result.Phi, -TargetPenetration, KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.Position, FVec3(2, 2, 5), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0, 0, 1), KINDA_SMALL_NUMBER);
			EXPECT_EQ(Result.FaceId, 1);
		}
	}

	TEST_F(FSweepCCDFixture, SavedTriMeshWithMultipleHits_SweepCCD_ResultsAreExpected)
	{
		// This is a mesh that showed a bug with CCD.
		// The above test (SimpleTriMeshWithTwoHits_SweepCCD_ResultsAreExpected) is an extremely simplified version of this test.
		// The root issue was that TOI was computed using the clipped distance when traversing the BVH instead of the original length, causing TOI to approach 1.
		const FReal SweepLength = 109.97858833658735;
		SphereRadius = 15.8750010;

		FMeshInitData InitData;
		InitData.TrimeshParticles = FTriangleMeshImplicitObject::ParticlesType
		({
			FVec3(-344.745148, -362.888794, -1.48117601e-06),
			FVec3(-361.505798, -361.505798, -9.83252107e-07),
			FVec3(-350.139618, -370.883759, -1.48008394e-06),
			FVec3(-350.139618, -370.883759, 26.9759979),
			FVec3(-337.094055, -377.964661, 26.9759979),
			FVec3(-362.888794, -344.745148, -5.74946910e-07),
			FVec3(-370.883759, -350.139587, -5.73328350e-07),
			FVec3(-377.964661, -337.094025, -2.63812211e-07),
			FVec3(-370.883759, -350.139587, 26.9760017),
			FVec3(-361.505798, -361.505798, 26.9760017),
			FVec3(-354.686462, -354.686462, -9.84611802e-07),
			FVec3(-344.745148, -362.888794, 26.9759979),
			FVec3(-354.686462, -354.686462, 26.9760017),
			FVec3(-362.888794, -344.745148, 26.9760017),
			FVec3(-349.114685, -369.364716, 28.7999992),
			FVec3(-360.210144, -360.210114, 28.8000011),
			FVec3(-345.770111, -364.407837, 28.7999992),
			FVec3(-369.364716, -349.114655, 28.8000011),
			FVec3(-364.407837, -345.770111, 28.8000011),
			FVec3(-355.982147, -355.982147, 28.8000011),
			FVec3(-337.094055, -377.964661, -2.05032484e-06),
			FVec3(-320.754150, -372.984985, -2.68089730e-06),
			FVec3(-322.677887, -382.439697, -2.68047484e-06),
			FVec3(-322.677887, -382.439697, 26.9759979),
			FVec3(-333.345245, -369.076477, -2.05111360e-06),
			FVec3(-333.345245, -369.076477, 26.9759979),
			FVec3(-320.754150, -372.984985, 26.9759979),
			FVec3(-307.200012, -384.000000, 26.9759979),
			FVec3(-306.717346, -374.399994, 26.9759979),
			FVec3(-336.381775, -376.275909, 28.7999992),
			FVec3(-322.312378, -380.643311, 28.7999992),
			FVec3(-321.119629, -374.781403, 28.7999992),
			FVec3(-334.057526, -370.765228, 28.7999992),
			FVec3(-307.108276, -382.175995, 28.7999992),
			FVec3(-306.809082, -376.224030, 28.7999992),
			FVec3(-369.076477, -333.345215, -2.65682075e-07),
			FVec3(-372.984955, -320.754150, -7.02783609e-08),
			FVec3(-382.439697, -322.677887, -6.82030006e-08),
			FVec3(-377.964661, -337.094025, 26.9760017),
			FVec3(-369.076477, -333.345215, 26.9760017),
			FVec3(-382.439697, -322.677887, 26.9760017),
			FVec3(-384.000000, -307.200012, -6.82121053e-14),
			FVec3(-374.399994, -306.717316, -5.29428446e-10),
			FVec3(-372.984955, -320.754150, 26.9760017),
			FVec3(-384.000000, -307.200012, 26.9760017),
			FVec3(-374.399994, -306.717316, 26.9760017),
			FVec3(-376.275909, -336.381744, 28.8000011),
			FVec3(-370.765198, -334.057495, 28.8000011),
			FVec3(-380.643280, -322.312378, 28.8000011),
			FVec3(-374.781372, -321.119629, 28.8000011),
			FVec3(-382.175995, -307.108276, 28.8000011),
			FVec3(-376.224030, -306.809052, 28.8000011),
		});
		InitData.Indices =
		{
			{14, 9, 3},
			{15, 14, 16},
			{8, 9, 17},
			{15, 18, 17},
			{9, 14, 15},
			{17, 9, 15},
			{19, 15, 16},
			{11, 19, 16},
			{19, 11, 12},
			{18, 15, 19},
			{29, 3, 4},
			{3, 29, 14},
			{14, 29, 32},
			{14, 32, 16},
			{1, 0, 2},
			{2, 3, 1},
			{3, 2, 4},
			{5, 1, 6},
			{8, 6, 9},
			{9, 6, 1},
			{0, 1, 10},
			{10, 11, 0},
			{1, 5, 10},
			{9, 1, 3},
			{11, 10, 12},
			{20, 4, 2},
			{2, 24, 20},
			{24, 2, 0},
			{30, 4, 23},
			{4, 30, 29},
			{31, 29, 30},
			{16, 25, 11},
			{25, 16, 32},
			{29, 31, 32},
			{32, 26, 25},
			{27, 30, 23},
			{33, 31, 30},
			{30, 27, 33},
			{31, 33, 34},
			{34, 26, 31},
			{28, 26, 34},
			{26, 32, 31},
			{20, 21, 22},
			{22, 23, 20},
			{0, 25, 24},
			{21, 20, 24},
			{4, 20, 23},
			{25, 0, 11},
			{26, 24, 25},
			{24, 26, 21},
			{23, 22, 27},
			{28, 21, 26},
			{8, 7, 6},
			{5, 12, 10},
			{12, 5, 13},
			{12, 13, 19},
			{18, 19, 13},
			{6, 35, 5},
			{35, 6, 7},
			{35, 13, 5},
			{38, 7, 8},
			{13, 35, 39},
			{38, 8, 46},
			{46, 8, 17},
			{47, 17, 18},
			{39, 18, 13},
			{17, 47, 46},
			{18, 39, 47},
			{48, 38, 46},
			{40, 38, 48},
			{46, 49, 48},
			{49, 46, 47},
			{43, 47, 39},
			{48, 44, 40},
			{49, 50, 48},
			{47, 43, 49},
			{44, 48, 50},
			{43, 51, 49},
			{49, 51, 50},
			{51, 43, 45},
			{36, 7, 37},
			{38, 37, 7},
			{7, 36, 35},
			{36, 39, 35},
			{40, 37, 38},
			{41, 36, 37},
			{40, 41, 37},
			{36, 41, 42},
			{42, 43, 36},
			{39, 36, 43},
			{44, 41, 40},
			{43, 42, 45},
		};
		InitData.AutoBuildMaterials();
		InitData.AutoBuildExternalFaceIndices();
		FTriangleMeshImplicitObject TriangleMesh(MoveTemp(InitData.TrimeshParticles), MoveTemp(InitData.Indices), MoveTemp(InitData.Materials), MoveTemp(InitData.ExternalFaceIndexMap), nullptr, true);

		FVec3 SweepStart = FVec3(-277.57524933822191, -298.05159216244965, 12.770771045445827);
		FVec3 SweepDir = FVec3(-0.92428685934217092, -0.38139225184986136, 0.015286707794476266);
		FQuat Rotation(-0.00085530578740522696, -0.00082053338777325429, 0.70703569140501821, 0.70717683524049568);

		const FReal ExpectedDistance = 81.848526000976562;
		{
			IgnorePenetration = 0;
			TargetPenetration = 0;
			FSweepCCDResultData Result = SweepSphereCCD(TriangleMesh, SweepStart, SweepDir, Rotation, SweepLength);

			const FReal ExpectedTOI = (ExpectedDistance + TargetPenetration) / SweepLength;
			EXPECT_EQ(Result.bResult, true);
			EXPECT_NEAR(Result.TOI, ExpectedTOI, KINDA_SMALL_NUMBER);
			EXPECT_NEAR(Result.Phi, 0, KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.Position, FVec3(-367.17901611328125, -336.84103393554688, 14.021965980529785), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0.87888097408604637, 0.47704112337361676, -0.0000000000000000), KINDA_SMALL_NUMBER);
			EXPECT_EQ(Result.FaceId, 61);
		}
		{
			IgnorePenetration = 12.700000762939453;
			TargetPenetration = 6.3500003814697266;
			FSweepCCDResultData Result = SweepSphereCCD(TriangleMesh, SweepStart, SweepDir, Rotation, SweepLength);

			const FReal ExpectedTOI = (ExpectedDistance + TargetPenetration) / SweepLength;
			EXPECT_EQ(Result.bResult, true);
			// Note: This test is slightly outside KINDA_SMALL_NUMBER in the epsilon check, as the way the TOI is computed is slightly different and loses a little numerical precision.
			EXPECT_NEAR(Result.TOI, ExpectedTOI, 1.e-3f);
			EXPECT_NEAR(Result.Phi, -TargetPenetration, KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.Position, FVec3(-367.17901611328125, -336.84103393554688, 14.021965980529785), KINDA_SMALL_NUMBER);
			EXPECT_VECTOR_NEAR(Result.Normal, FVec3(0.87888097408604637, 0.47704112337361676, -0.0000000000000000), KINDA_SMALL_NUMBER);
			EXPECT_EQ(Result.FaceId, 61);
		}
	}

	TEST_F(FSphereSweepFixture, TransformedUnionContainingNonUniformScaledObject_SweepSphere_ResultsAreExpected)
	{
		// This test is to catch a bug where a Transformed union with a scaled object would 
		// ensure because it attempted to go down the sweep as raycast path when there was a non-uniform scale.
		bComputeMTD = false;
		SphereRadius = 1;

		
		// Create a union that has a scaled and non-scaled object
		FImplicitObjectPtr SphereObjPtr = new FImplicitSphere3(FVec3(0, 0, 5), 1);
		FImplicitObjectPtr ScaledBoxObjPtr = new TImplicitObjectScaled<FImplicitBox3>(new FImplicitBox3(FVec3(-1), FVec3(1)), FVec3(2, 1, 1));
		TArray<FImplicitObjectPtr> Objects{ ScaledBoxObjPtr, SphereObjPtr };
		FImplicitObjectPtr UnionObjectPtr = new FImplicitObjectUnion(MoveTemp(Objects));
		// Build a transform around the union (this is necessary to produce the bug)
		FRigidTransform3 ObjTransform(FVec3::Zero(), FRotation3::Identity, FVec3(1));
		TImplicitObjectTransformed<FReal, 3> TestObject(UnionObjectPtr, ObjTransform);

		// Test a few configurations to make sure we hit and don't hit where expected
		FSweepResultData Result;

		// To the left of the box
		Result = SweepQuerySphere(TestObject, FVec3(-3.1, 20, 0), FVec3(0, -1, 0));
		EXPECT_EQ(Result.bResult, false);

		// Should hit left edge of the box
		Result = SweepQuerySphere(TestObject, FVec3(-2, 20, 0), FVec3(0, -1, 0));
		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 18, KINDA_SMALL_NUMBER);

		// Should hit right edge of the box
		Result = SweepQuerySphere(TestObject, FVec3(2, 20, 0), FVec3(0, -1, 0));
		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 18, KINDA_SMALL_NUMBER);

		// To the right of the box
		Result = SweepQuerySphere(TestObject, FVec3(3.1, 20, 0), FVec3(0, -1, 0));
		EXPECT_EQ(Result.bResult, false);

		// Center of the sphere
		Result = SweepQuerySphere(TestObject, FVec3(0, 20, 5), FVec3(0, -1, 0));
		EXPECT_EQ(Result.bResult, true);
		EXPECT_NEAR(Result.TOI, 18, KINDA_SMALL_NUMBER);

		// To the left of the sphere
		Result = SweepQuerySphere(TestObject, FVec3(-2.1, 20, 5), FVec3(0, -1, 0));
		EXPECT_EQ(Result.bResult, false);

		// To the right of the sphere
		Result = SweepQuerySphere(TestObject, FVec3(2.1, 20, 5), FVec3(0, -1, 0));
		EXPECT_EQ(Result.bResult, false);
	}
}
