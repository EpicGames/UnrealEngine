// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestTriangleMesh.h"
#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"
#include "Chaos/Triangle.h"
#include "Chaos/TriangleMesh.h"

namespace ChaosTest
{
    using namespace Chaos;

    void TriangleMeshProjectTest()
    {
		FTriangleMesh TriMesh;
		TParticles<double, 3> Particles;
		Particles.AddParticles(3);
		Particles.SetX(0, FVec3(0, 0, 0));
		Particles.SetX(1, FVec3(1, 0, 0));
		Particles.SetX(2, FVec3(0, 1, 0));
		TArray<TVec3<int32>> Elements;
		Elements.Add(TVec3<int32>(0, 1, 2));
		TriMesh.Init(MoveTemp(Elements));

		TTriangle<double> Tri(Particles.GetX(0), Particles.GetX(1), Particles.GetX(2));
		Chaos::FVec3 Center = Tri.GetCentroid();
		Chaos::FVec3 Norm = Tri.GetNormal();

		TArray<TVec3<double>> FaceNormals;
		FaceNormals.Add(Norm);
		TArray<TVec3<double>> PointNormals;
		PointNormals.Add(Norm);
		PointNormals.Add(Norm);
		PointNormals.Add(Norm);
		EXPECT_TRUE(PointNormals.Num() > 0);

		FTriangleMesh::TBVHType<double> BVH;
		TriMesh.BuildBVH(TConstArrayView<TVec3<double>>(Particles.XArray()), BVH);

		int32 HitTriIdx = INDEX_NONE;
		FVec3 Weights;

		// Center
		
		Chaos::FVec3 Pt = Center;
		bool Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_TRUE(Hit);
		EXPECT_VECTOR_NEAR(Weights, FVec3(1.0 / 3), 0.0001);

		Pt = Center + Norm;
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_TRUE(Hit);
		EXPECT_VECTOR_NEAR(Weights, FVec3(1.0 / 3), 0.0001);

		Pt = Center - Norm;
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_TRUE(Hit);
		EXPECT_VECTOR_NEAR(Weights, FVec3(1.0 / 3), 0.0001);

		// Vertices

		Pt = Particles.GetX(0);
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_TRUE(Hit);
		EXPECT_VECTOR_NEAR(Weights, FVec3(1,0,0), 0.01);

		Pt = Particles.GetX(1);
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_TRUE(Hit);
		EXPECT_VECTOR_NEAR(Weights, FVec3(0, 1, 0), 0.01);

		Pt = Particles.GetX(2);
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_TRUE(Hit);
		EXPECT_VECTOR_NEAR(Weights, FVec3(0, 0, 1), 0.01);

		// Edges

		Pt = (Particles.GetX(0) + Particles.GetX(1)) * 0.5;
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_TRUE(Hit);
		EXPECT_VECTOR_NEAR(Weights, FVec3(.5, .5, 0), 0.01);

		Pt = (Particles.GetX(1) + Particles.GetX(2)) * 0.5;
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_TRUE(Hit);
		EXPECT_VECTOR_NEAR(Weights, FVec3(0, .5, .5), 0.01);

		Pt = (Particles.GetX(0) + Particles.GetX(2)) * 0.5;
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_TRUE(Hit);
		EXPECT_VECTOR_NEAR(Weights, FVec3(.5, 0, .5), 0.01);

		// Off vertices - Expect false

		Pt = Center + (Particles.GetX(0) - Center) * 1.1;
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_FALSE(Hit);

		Pt = Center + (Particles.GetX(1) - Center) * 1.1;
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_FALSE(Hit);

		Pt = Center + (Particles.GetX(2) - Center) * 1.1;
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_FALSE(Hit);

		// Off edges - Expect false

		Pt = (Particles.GetX(0) + Particles.GetX(1)) * 0.5;
		Pt = Center + (Pt - Center) * 1.1;
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_FALSE(Hit);

		Pt = (Particles.GetX(1) + Particles.GetX(2)) * 0.5;
		Pt = Center + (Pt - Center) * 1.1;
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_FALSE(Hit);

		Pt = (Particles.GetX(0) + Particles.GetX(2)) * 0.5;
		Pt = Center + (Pt - Center) * 1.1;
		Hit = TriMesh.SmoothProject(BVH, Particles.XArray(), PointNormals, Pt, HitTriIdx, Weights);
		EXPECT_FALSE(Hit);
	}

} // namespace ChaosTest
