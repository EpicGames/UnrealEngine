// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaosTestUtility.h"

#include "HeadlessChaos.h"
#include "Chaos/Box.h"
#include "Chaos/Convex.h"
#include "Chaos/ErrorReporter.h"
#include "Chaos/Levelset.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/Plane.h"
#include "Chaos/Sphere.h"
#include "Chaos/Cylinder.h"
#include "Chaos/TaperedCylinder.h"
#include "Chaos/Utilities.h"

namespace ChaosTest {

	using namespace Chaos;

	int32 AppendAnalyticSphere(FPBDRigidParticles& InParticles, FReal Scale)
	{
		InParticles.AddParticles(1);
		int32 RigidBodyIndex = InParticles.Size() - 1;

		InParticles.SetX(RigidBodyIndex, FVec3(0, 0, 0));
		InParticles.SetV(RigidBodyIndex, FVec3(0, 0, 0));
		InParticles.SetR(RigidBodyIndex, FRotation3::MakeFromEuler(FVec3(0, 0, 0)).GetNormalized());
		InParticles.SetW(RigidBodyIndex, FVec3(0, 0, 0));
		InParticles.SetP(RigidBodyIndex, InParticles.GetX(RigidBodyIndex));
		InParticles.SetQ(RigidBodyIndex, InParticles.GetR(RigidBodyIndex));

		InParticles.M(RigidBodyIndex) = 1.0;
		InParticles.InvM(RigidBodyIndex) = 1.0;
		InParticles.I(RigidBodyIndex) = TVec3<FRealSingle>(1, 1, 1);
		InParticles.InvI(RigidBodyIndex) = TVec3<FRealSingle>(1, 1, 1);
		InParticles.SetGeometry(RigidBodyIndex, MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), Scale));
		InParticles.SetObjectState(RigidBodyIndex, EObjectStateType::Dynamic);

		return RigidBodyIndex;
	}

	int32 AppendAnalyticBox(FPBDRigidParticles& InParticles, FVec3 Scale)
	{
		InParticles.AddParticles(1);
		int32 RigidBodyIndex = InParticles.Size() - 1;

		InParticles.SetX(RigidBodyIndex, FVec3(0, 0, 0));
		InParticles.SetV(RigidBodyIndex, FVec3(0, 0, 0));
		InParticles.SetR(RigidBodyIndex, FRotation3::MakeFromEuler(FVec3(0, 0, 0)).GetNormalized());
		InParticles.SetW(RigidBodyIndex, FVec3(0, 0, 0));
		InParticles.SetP(RigidBodyIndex, InParticles.GetX(RigidBodyIndex));
		InParticles.SetQ(RigidBodyIndex, InParticles.GetR(RigidBodyIndex));

		InParticles.M(RigidBodyIndex) = 1.0;
		InParticles.InvM(RigidBodyIndex) = 1.0;
		InParticles.I(RigidBodyIndex) = TVec3<FRealSingle>(1, 1, 1);
		InParticles.InvI(RigidBodyIndex) = TVec3<FRealSingle>(1, 1, 1);
		InParticles.SetGeometry(RigidBodyIndex, MakeImplicitObjectPtr<TBox<FReal, 3>>(-Scale / 2.0, Scale / 2.0));
		InParticles.SetObjectState(RigidBodyIndex, EObjectStateType::Dynamic);

		return RigidBodyIndex;
	}

	void InitAnalyticBox2(FKinematicGeometryParticleHandle* Particle, FVec3 Scale)
	{
		Particle->SetX(FVec3(0, 0, 0));
		Particle->SetV(FVec3(0, 0, 0));
		Particle->SetR(FRotation3::MakeFromEuler(FVec3(0, 0, 0)).GetNormalized());
		Particle->SetW(FVec3(0, 0, 0));
		Particle->SetGeometry(MakeImplicitObjectPtr<TBox<FReal, 3>>(-Scale / 2.0, Scale / 2.0));

		FPBDRigidParticleHandle* DynamicParticle = Particle->CastToRigidParticle();
		if(DynamicParticle && DynamicParticle->ObjectState() == EObjectStateType::Dynamic)
		{
			DynamicParticle->SetP(Particle->GetX());
			DynamicParticle->SetQ(Particle->GetR());

			DynamicParticle->M() = 1.0;
			DynamicParticle->InvM() = 1.0;
			DynamicParticle->I() = TVec3<FRealSingle>(1, 1, 1);
			DynamicParticle->InvI() = TVec3<FRealSingle>(1, 1, 1);
		}
	}

	int32 AppendParticleBox(FPBDRigidParticles& InParticles, FVec3 Scale, TArray<TVec3<int32>>* elements)
	{
		InParticles.AddParticles(1);
		int32 RigidBodyIndex = InParticles.Size() - 1;

		InParticles.SetX(RigidBodyIndex, FVec3(0, 0, 0));
		InParticles.SetV(RigidBodyIndex, FVec3(0, 0, 0));
		InParticles.SetR(RigidBodyIndex, FRotation3::MakeFromEuler(FVec3(0, 0, 0)).GetNormalized());
		InParticles.SetW(RigidBodyIndex, FVec3(0, 0, 0));
		InParticles.SetP(RigidBodyIndex, InParticles.GetX(RigidBodyIndex));
		InParticles.SetQ(RigidBodyIndex, InParticles.GetR(RigidBodyIndex));

		check(Scale.X == Scale.Y && Scale.X == Scale.Z);
		FReal ScaleSq = Scale.X * Scale.X;
		InParticles.M(RigidBodyIndex) = 1.0;
		InParticles.InvM(RigidBodyIndex) = 1.0;
		InParticles.I(RigidBodyIndex) = TVec3<FRealSingle>(ScaleSq / 6.0, ScaleSq / 6.0, ScaleSq / 6.0);
		InParticles.InvI(RigidBodyIndex) = TVec3<FRealSingle>(6.0 / ScaleSq, 6.0 / ScaleSq, 6.0 / ScaleSq);
		InParticles.SetGeometry(RigidBodyIndex, MakeImplicitObjectPtr<TBox<FReal, 3>>(-Scale / 2.0, Scale / 2.0));
		InParticles.SetObjectState(RigidBodyIndex, EObjectStateType::Dynamic);

		int32 CollisionIndex = 0;
		InParticles.CollisionParticlesInitIfNeeded(RigidBodyIndex);
		InParticles.CollisionParticles(RigidBodyIndex)->AddParticles(8);
		InParticles.CollisionParticles(RigidBodyIndex)->SetX(CollisionIndex++, FVec3(-Scale[0] / 2.0, -Scale[1] / 2.0, -Scale[2] / 2.0));
		InParticles.CollisionParticles(RigidBodyIndex)->SetX(CollisionIndex++, FVec3(+Scale[0] / 2.0, -Scale[1] / 2.0, -Scale[2] / 2.0));
		InParticles.CollisionParticles(RigidBodyIndex)->SetX(CollisionIndex++, FVec3(-Scale[0] / 2.0, +Scale[1] / 2.0, -Scale[2] / 2.0));
		InParticles.CollisionParticles(RigidBodyIndex)->SetX(CollisionIndex++, FVec3(+Scale[0] / 2.0, +Scale[1] / 2.0, -Scale[2] / 2.0));
		InParticles.CollisionParticles(RigidBodyIndex)->SetX(CollisionIndex++, FVec3(-Scale[0] / 2.0, -Scale[1] / 2.0, +Scale[2] / 2.0));
		InParticles.CollisionParticles(RigidBodyIndex)->SetX(CollisionIndex++, FVec3(+Scale[0] / 2.0, -Scale[1] / 2.0, +Scale[2] / 2.0));
		InParticles.CollisionParticles(RigidBodyIndex)->SetX(CollisionIndex++, FVec3(-Scale[0] / 2.0, +Scale[1] / 2.0, +Scale[2] / 2.0));
		InParticles.CollisionParticles(RigidBodyIndex)->SetX(CollisionIndex++, FVec3(+Scale[0] / 2.0, +Scale[1] / 2.0, +Scale[2] / 2.0));

		if (elements != nullptr)
		{
			/*
			//cw
			elements->Add(TVec3<int32>(4, 1, 5)); // Front
			elements->Add(TVec3<int32>(1, 4, 0));
			elements->Add(TVec3<int32>(7, 2, 6)); // Back
			elements->Add(TVec3<int32>(2, 7, 3));
			elements->Add(TVec3<int32>(6, 0, 4)); // Right
			elements->Add(TVec3<int32>(0, 6, 2));
			elements->Add(TVec3<int32>(5, 3, 7)); // Left
			elements->Add(TVec3<int32>(3, 5, 1));
			elements->Add(TVec3<int32>(6, 5, 7)); // Top
			elements->Add(TVec3<int32>(5, 6, 4));
			elements->Add(TVec3<int32>(0, 2, 1)); // Front
			elements->Add(TVec3<int32>(2, 0, 3));
			*/
			//ccw
			elements->Add(TVec3<int32>(1,4,5)); // Front
			elements->Add(TVec3<int32>(4,1,0));
			elements->Add(TVec3<int32>(2,7,6)); // Back
			elements->Add(TVec3<int32>(7,2,3));
			elements->Add(TVec3<int32>(0,6,4)); // Right
			elements->Add(TVec3<int32>(6,0,2));
			elements->Add(TVec3<int32>(3,5,7)); // Left
			elements->Add(TVec3<int32>(5,3,1));
			elements->Add(TVec3<int32>(5,6,7)); // Top
			elements->Add(TVec3<int32>(6,5,4));
			elements->Add(TVec3<int32>(2,0,1)); // Front
			elements->Add(TVec3<int32>(0,2,3));
		}

		return RigidBodyIndex;
	}


	void InitDynamicParticleBox2(FPBDRigidParticleHandle* Particle, const FVec3& Scale, FReal Margin, TArray<TVector<int32, 3>>* OutElements)
	{
		Particle->SetX(FVec3(0, 0, 0));
		Particle->SetV(FVec3(0, 0, 0));
		Particle->SetR(FRotation3::MakeFromEuler(FVec3(0, 0, 0)).GetNormalized());
		Particle->SetW(FVec3(0, 0, 0));

		Particle->SetP(Particle->GetX());
		Particle->SetQ(Particle->GetR());

		// Assume unit mass - this gets scaled externally by the actual mass
		const FVec3 ScaleSq = Scale * Scale;
		Particle->M() = 1.0;
		Particle->InvM() = 1.0;
		Particle->I() = (1.0 / 12.0) * TVec3<FRealSingle>(ScaleSq.Y + ScaleSq.Z, ScaleSq.X + ScaleSq.Z, ScaleSq.X + ScaleSq.Y);
		Particle->InvI() = TVec3<FRealSingle>(6.0 / ScaleSq);

		Particle->SetGeometry(MakeImplicitObjectPtr<TBox<FReal, 3>>(-Scale / 2.0, Scale / 2.0, Margin));

		int32 CollisionIndex = 0;
		Particle->CollisionParticlesInitIfNeeded();
		Particle->CollisionParticles()->AddParticles(8);
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(-Scale[0] / 2.0, -Scale[1] / 2.0, -Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(+Scale[0] / 2.0, -Scale[1] / 2.0, -Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(-Scale[0] / 2.0, +Scale[1] / 2.0, -Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(+Scale[0] / 2.0, +Scale[1] / 2.0, -Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(-Scale[0] / 2.0, -Scale[1] / 2.0, +Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(+Scale[0] / 2.0, -Scale[1] / 2.0, +Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(-Scale[0] / 2.0, +Scale[1] / 2.0, +Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(+Scale[0] / 2.0, +Scale[1] / 2.0, +Scale[2] / 2.0));

		Particle->SetLocalBounds(Particle->GetGeometry()->BoundingBox());
		Particle->UpdateWorldSpaceState(FRigidTransform3::Identity, FVec3(0));
		Particle->SetHasBounds(true);

		if (OutElements != nullptr)
		{
			/*
			//cw
			OutElements->Add(TVec3<int32>(4, 1, 5)); // Front
			OutElements->Add(TVec3<int32>(1, 4, 0));
			OutElements->Add(TVec3<int32>(7, 2, 6)); // Back
			OutElements->Add(TVec3<int32>(2, 7, 3));
			OutElements->Add(TVec3<int32>(6, 0, 4)); // Right
			OutElements->Add(TVec3<int32>(0, 6, 2));
			OutElements->Add(TVec3<int32>(5, 3, 7)); // Left
			OutElements->Add(TVec3<int32>(3, 5, 1));
			OutElements->Add(TVec3<int32>(6, 5, 7)); // Top
			OutElements->Add(TVec3<int32>(5, 6, 4));
			OutElements->Add(TVec3<int32>(0, 2, 1)); // Front
			OutElements->Add(TVec3<int32>(2, 0, 3));
			*/
			//ccw
			OutElements->Add(TVec3<int32>(1, 4, 5)); // Front
			OutElements->Add(TVec3<int32>(4, 1, 0));
			OutElements->Add(TVec3<int32>(2, 7, 6)); // Back
			OutElements->Add(TVec3<int32>(7, 2, 3));
			OutElements->Add(TVec3<int32>(0, 6, 4)); // Right
			OutElements->Add(TVec3<int32>(6, 0, 2));
			OutElements->Add(TVec3<int32>(3, 5, 7)); // Left
			OutElements->Add(TVec3<int32>(5, 3, 1));
			OutElements->Add(TVec3<int32>(5, 6, 7)); // Top
			OutElements->Add(TVec3<int32>(6, 5, 4));
			OutElements->Add(TVec3<int32>(2, 0, 1)); // Front
			OutElements->Add(TVec3<int32>(0, 2, 3));
		}

		::ChaosTest::SetParticleSimDataToCollide({ Particle});

	}

	void InitDynamicParticleSphere2(FPBDRigidParticleHandle* Particle, const FVec3& Scale, TArray<TVec3<int32>>* OutElements) 
	{
		Particle->SetX(FVec3(0, 0, 0));
		Particle->SetV(FVec3(0, 0, 0));
		Particle->SetR(FRotation3::MakeFromEuler(FVec3(0, 0, 0)).GetNormalized());
		Particle->SetW(FVec3(0, 0, 0));

		Particle->SetP(Particle->GetX());
		Particle->SetQ(Particle->GetR());

		check(Scale.X == Scale.Y && Scale.X == Scale.Z);
		FReal ScaleSq = Scale.X * Scale.X;
		Particle->M() = 1.0;
		Particle->InvM() = 1.0;
		Particle->I() = TVec3<FRealSingle>(ScaleSq / 6.0);
		Particle->InvI() = TVec3<FRealSingle>(6.0 / ScaleSq);

		Particle->SetGeometry(MakeImplicitObjectPtr<Chaos::FSphere>(FVec3(0), Scale.X / 2.0));

		int32 CollisionIndex = 0;
		Particle->CollisionParticlesInitIfNeeded();
		Particle->CollisionParticles()->AddParticles(6);

		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(-Scale[0] / 2.0, 0, 0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(+Scale[0] / 2.0, 0, 0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(0, -Scale[1] / 2.0, 0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(0, +Scale[1] / 2.0, 0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(0, 0, -Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(0, 0, +Scale[2] / 2.0));

		if (OutElements != nullptr)
		{
			/*
			//cw
			OutElements->Add(TVec3<int32>(4, 1, 5)); // Front
			OutElements->Add(TVec3<int32>(1, 4, 0));
			OutElements->Add(TVec3<int32>(7, 2, 6)); // Back
			OutElements->Add(TVec3<int32>(2, 7, 3));
			OutElements->Add(TVec3<int32>(6, 0, 4)); // Right
			OutElements->Add(TVec3<int32>(0, 6, 2));
			OutElements->Add(TVec3<int32>(5, 3, 7)); // Left
			OutElements->Add(TVec3<int32>(3, 5, 1));
			OutElements->Add(TVec3<int32>(6, 5, 7)); // Top
			OutElements->Add(TVec3<int32>(5, 6, 4));
			OutElements->Add(TVec3<int32>(0, 2, 1)); // Front
			OutElements->Add(TVec3<int32>(2, 0, 3));
			*/
			//ccw
			OutElements->Add(TVec3<int32>(1, 4, 5)); // Front
			OutElements->Add(TVec3<int32>(4, 1, 0));
			OutElements->Add(TVec3<int32>(2, 7, 6)); // Back
			OutElements->Add(TVec3<int32>(7, 2, 3));
			OutElements->Add(TVec3<int32>(0, 6, 4)); // Right
			OutElements->Add(TVec3<int32>(6, 0, 2));
			OutElements->Add(TVec3<int32>(3, 5, 7)); // Left
			OutElements->Add(TVec3<int32>(5, 3, 1));
			OutElements->Add(TVec3<int32>(5, 6, 7)); // Top
			OutElements->Add(TVec3<int32>(6, 5, 4));
			OutElements->Add(TVec3<int32>(2, 0, 1)); // Front
			OutElements->Add(TVec3<int32>(0, 2, 3));
		}
	}

	void InitDynamicParticleCylinder2(FPBDRigidParticleHandle* Particle, const FVec3& Scale, TArray<TVec3<int32>>* OutElements, bool Tapered) 
	{
		Particle->SetX(FVec3(0, 0, 0));
		Particle->SetV(FVec3(0, 0, 0));
		Particle->SetR(FRotation3::MakeFromEuler(FVec3(0, 0, 0)).GetNormalized());
		Particle->SetW(FVec3(0, 0, 0));

		Particle->SetP(Particle->GetX());
		Particle->SetQ(Particle->GetR());

		check(Scale.X == Scale.Y && Scale.X == Scale.Z);
		FReal ScaleSq = Scale.X * Scale.X;
		Particle->M() = 1.0;
		Particle->InvM() = 1.0;
		Particle->I() = TVec3<FRealSingle>(ScaleSq / 6.0);
		Particle->InvI() = TVec3<FRealSingle>(6.0 / ScaleSq);
		
		if (Tapered)
		{
			Particle->SetGeometry(MakeImplicitObjectPtr<FTaperedCylinder>(FVec3(0, 0, Scale.X / 2.0), FVec3(0, 0, -Scale.X / 2.0), Scale.X / 2.0, Scale.X / 2.0));
		}
		else 
		{
			Particle->SetGeometry(MakeImplicitObjectPtr<FCylinder>(FVec3(0, 0, Scale.X / 2.0), FVec3(0, 0, -Scale.X / 2.0), Scale.X / 2.0));
		}

		int32 CollisionIndex = 0;
		Particle->CollisionParticlesInitIfNeeded();
		Particle->CollisionParticles()->AddParticles(8);

		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(-Scale[0] / 2.0, 0, +Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(-Scale[0] / 2.0, 0, -Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(+Scale[0] / 2.0, 0, +Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(+Scale[0] / 2.0, 0, -Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(0, -Scale[1] / 2.0, +Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(0, -Scale[1] / 2.0, -Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(0, +Scale[1] / 2.0, +Scale[2] / 2.0));
		Particle->CollisionParticles()->SetX(CollisionIndex++, FVec3(0, +Scale[1] / 2.0, -Scale[2] / 2.0));

		if (OutElements != nullptr)
		{
			/*
			//cw
			OutElements->Add(TVec3<int32>(4, 1, 5)); // Front
			OutElements->Add(TVec3<int32>(1, 4, 0));
			OutElements->Add(TVec3<int32>(7, 2, 6)); // Back
			OutElements->Add(TVec3<int32>(2, 7, 3));
			OutElements->Add(TVec3<int32>(6, 0, 4)); // Right
			OutElements->Add(TVec3<int32>(0, 6, 2));
			OutElements->Add(TVec3<int32>(5, 3, 7)); // Left
			OutElements->Add(TVec3<int32>(3, 5, 1));
			OutElements->Add(TVec3<int32>(6, 5, 7)); // Top
			OutElements->Add(TVec3<int32>(5, 6, 4));
			OutElements->Add(TVec3<int32>(0, 2, 1)); // Front
			OutElements->Add(TVec3<int32>(2, 0, 3));
			*/
			//ccw
			OutElements->Add(TVec3<int32>(1, 4, 5)); // Front
			OutElements->Add(TVec3<int32>(4, 1, 0));
			OutElements->Add(TVec3<int32>(2, 7, 6)); // Back
			OutElements->Add(TVec3<int32>(7, 2, 3));
			OutElements->Add(TVec3<int32>(0, 6, 4)); // Right
			OutElements->Add(TVec3<int32>(6, 0, 2));
			OutElements->Add(TVec3<int32>(3, 5, 7)); // Left
			OutElements->Add(TVec3<int32>(5, 3, 1));
			OutElements->Add(TVec3<int32>(5, 6, 7)); // Top
			OutElements->Add(TVec3<int32>(6, 5, 4));
			OutElements->Add(TVec3<int32>(2, 0, 1)); // Front
			OutElements->Add(TVec3<int32>(0, 2, 3));
		}
	}

	FPBDRigidParticleHandle* AppendDynamicParticleBox(FPBDRigidsSOAs& SOAs, const FVec3& Scale, TArray<TVec3<int32>>* OutElements)
	{
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(1);
		InitDynamicParticleBox2(Particles[0], Scale, 0.0f, OutElements);
		return Particles[0];
	}

	// Create a particle with box collision of specified size and margin (size includes margin)
	FPBDRigidParticleHandle* AppendDynamicParticleBoxMargin(FPBDRigidsSOAs& SOAs, const FVec3& Scale, FReal Margin, TArray<TVec3<int32>>* OutElements)
	{
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(1);
		InitDynamicParticleBox2(Particles[0], Scale, Margin, OutElements);
		return Particles[0];
	}

	FPBDRigidParticleHandle* AppendDynamicParticleSphere(FPBDRigidsSOAs& SOAs, const FVec3& Scale, TArray<TVec3<int32>>* OutElements)
	{
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(1);
		InitDynamicParticleSphere2(Particles[0], Scale, OutElements);
		return Particles[0];
	}

	FPBDRigidParticleHandle* AppendDynamicParticleCylinder(FPBDRigidsSOAs& SOAs, const FVec3& Scale, TArray<TVec3<int32>>* OutElements)
	{
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(1);
		InitDynamicParticleCylinder2(Particles[0], Scale, OutElements, false);
		return Particles[0];
	}

	FPBDRigidParticleHandle* AppendDynamicParticleTaperedCylinder(FPBDRigidsSOAs& SOAs, const FVec3& Scale, TArray<TVec3<int32>>* OutElements)
	{
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(1);
		InitDynamicParticleCylinder2(Particles[0], Scale, OutElements, true);
		return Particles[0];
	}

	FPBDRigidParticleHandle* AppendClusteredParticleBox(FPBDRigidsSOAs& SOAs, const FVec3& Scale, TArray<TVec3<int32>>* OutElements)
	{
		auto Particles = SOAs.CreateClusteredParticles(1);
		InitDynamicParticleBox2(Particles[0], Scale, 0.0f, OutElements);
		return Particles[0];
	}

	void InitStaticParticleBox(FGeometryParticleHandle* Particle, const FVec3& Scale, TArray<TVec3<int32>>* OutElements)
	{
		Particle->SetX(FVec3(0, 0, 0));
		Particle->SetR(FRotation3::FromIdentity());

		FReal ScaleSq = Scale.X * Scale.X;

		Particle->SetGeometry(MakeImplicitObjectPtr<TBox<FReal, 3>>(-Scale / 2.0, Scale / 2.0));

		// This is needed for calculating contacts (Bounds are bigger than they need to be, even allowing for rotation)
		Particle->SetLocalBounds(FAABB3(FVec3(-Scale[0]), FVec3(Scale[0])));
		Particle->UpdateWorldSpaceState(FRigidTransform3::Identity, FVec3(0));
		Particle->SetHasBounds(true);

		if (OutElements != nullptr)
		{
			/*
			//cw
			OutElements->Add(TVec3<int32>(4, 1, 5)); // Front
			OutElements->Add(TVec3<int32>(1, 4, 0));
			OutElements->Add(TVec3<int32>(7, 2, 6)); // Back
			OutElements->Add(TVec3<int32>(2, 7, 3));
			OutElements->Add(TVec3<int32>(6, 0, 4)); // Right
			OutElements->Add(TVec3<int32>(0, 6, 2));
			OutElements->Add(TVec3<int32>(5, 3, 7)); // Left
			OutElements->Add(TVec3<int32>(3, 5, 1));
			OutElements->Add(TVec3<int32>(6, 5, 7)); // Top
			OutElements->Add(TVec3<int32>(5, 6, 4));
			OutElements->Add(TVec3<int32>(0, 2, 1)); // Front
			OutElements->Add(TVec3<int32>(2, 0, 3));
			*/
			//ccw
			OutElements->Add(TVec3<int32>(1, 4, 5)); // Front
			OutElements->Add(TVec3<int32>(4, 1, 0));
			OutElements->Add(TVec3<int32>(2, 7, 6)); // Back
			OutElements->Add(TVec3<int32>(7, 2, 3));
			OutElements->Add(TVec3<int32>(0, 6, 4)); // Right
			OutElements->Add(TVec3<int32>(6, 0, 2));
			OutElements->Add(TVec3<int32>(3, 5, 7)); // Left
			OutElements->Add(TVec3<int32>(5, 3, 1));
			OutElements->Add(TVec3<int32>(5, 6, 7)); // Top
			OutElements->Add(TVec3<int32>(6, 5, 4));
			OutElements->Add(TVec3<int32>(2, 0, 1)); // Front
			OutElements->Add(TVec3<int32>(0, 2, 3));
		}

		::ChaosTest::SetParticleSimDataToCollide({ Particle });

	}

	FGeometryParticleHandle* AppendStaticParticleBox(FPBDRigidsSOAs& SOAs, const FVec3& Scale, TArray<TVec3<int32>>* OutElements)
	{
		TArray<FGeometryParticleHandle*> Particles = SOAs.CreateStaticParticles(1);
		InitStaticParticleBox(Particles[0], Scale, OutElements);
		return Particles[0];
	}

	int AppendStaticAnalyticFloor(FPBDRigidParticles& InParticles)
	{
		InParticles.AddParticles(1);
		int32 RigidBodyIndex = InParticles.Size() - 1;

		InParticles.SetX(RigidBodyIndex, FVec3(0, 0, 0));
		InParticles.SetV(RigidBodyIndex, FVec3(0, 0, 0));
		InParticles.SetR(RigidBodyIndex, FRotation3::MakeFromEuler(FVec3(0, 0, 0)).GetNormalized());
		InParticles.SetW(RigidBodyIndex, FVec3(0, 0, 0));
		InParticles.M(RigidBodyIndex) = 1.0;
		InParticles.InvM(RigidBodyIndex) = 0.0;
		InParticles.I(RigidBodyIndex) = TVec3<FRealSingle>(1);
		InParticles.InvI(RigidBodyIndex) = TVec3<FRealSingle>(0);
		InParticles.SetGeometry(RigidBodyIndex, MakeImplicitObjectPtr<TPlane<FReal, 3>>(FVec3(0, 0, 0), FVec3(0, 0, 1)));
		InParticles.SetObjectState(RigidBodyIndex, EObjectStateType::Kinematic);

		InParticles.SetP(RigidBodyIndex, InParticles.GetX(RigidBodyIndex));
		InParticles.SetQ(RigidBodyIndex, InParticles.GetR(RigidBodyIndex));

		return RigidBodyIndex;
	}

	FKinematicGeometryParticleHandle* AppendStaticAnalyticFloor(FPBDRigidsSOAs& SOAs)
	{
		TArray<FKinematicGeometryParticleHandle*> Particles = SOAs.CreateKinematicParticles(1);
		FKinematicGeometryParticleHandle* Particle = Particles[0];

		Particle->SetX(FVec3(0, 0, 0));
		Particle->SetV(FVec3(0, 0, 0));
		Particle->SetR(FRotation3::MakeFromEuler(FVec3(0, 0, 0)).GetNormalized());
		Particle->SetW(FVec3(0, 0, 0));
		Particle->SetGeometry(MakeImplicitObjectPtr<TPlane<FReal, 3>>(FVec3(0, 0, 0), FVec3(0, 0, 1)));

		::ChaosTest::SetParticleSimDataToCollide({ Particle });

		return Particle;
	}


	FKinematicGeometryParticleHandle* AppendStaticConvexFloor(FPBDRigidsSOAs& SOAs)
	{
		TArray<FKinematicGeometryParticleHandle*> Particles = SOAs.CreateKinematicParticles(1);
		FKinematicGeometryParticleHandle* Particle = Particles[0];

		Particle->SetX(FVec3(0, 0, 0));
		Particle->SetV(FVec3(0, 0, 0));
		Particle->SetR(FRotation3::MakeFromEuler(FVec3(0, 0, 0)).GetNormalized());
		Particle->SetW(FVec3(0, 0, 0));

		TArray<Chaos::FConvex::FVec3Type> Cube;
		Cube.SetNum(9);
		Cube[0] = { -1000, -1000, -20 };
		Cube[1] = { -1000, -1000, 0 };
		Cube[2] = { -1000, 1000, -20 };
		Cube[3] = { -1000, 1000, 0 };
		Cube[4] = { 1000, -1000, -20 };
		Cube[5] = { 1000, -1000, 0 };
		Cube[6] = { 1000, 1000, -20 };
		Cube[7] = { 1000, 1000, 0 };
		Cube[8] = { 0, 0, 0 };

		Particle->SetGeometry(MakeImplicitObjectPtr<FConvex>(Cube, 0.0f));

		::ChaosTest::SetParticleSimDataToCollide({ Particle });

		return Particle;
	}

	/**/
	FLevelSet ConstructLevelset(FParticles & SurfaceParticles, TArray<TVec3<int32>> & Elements)
	{
		// build Particles and bounds
		Chaos::FAABB3 BoundingBox(FVec3(0), FVec3(0));
		for (int32 CollisionParticleIndex = 0; CollisionParticleIndex < (int32)SurfaceParticles.Size(); CollisionParticleIndex++)
		{
			BoundingBox.GrowToInclude(SurfaceParticles.GetX(CollisionParticleIndex));
		}

		// build cell domain
		int32 MaxAxisSize = 10;
		int MaxAxis = BoundingBox.LargestAxis();
		FVec3 Extents = BoundingBox.Extents();
		Chaos::TVec3<int32> Counts(MaxAxisSize * Extents[0] / Extents[MaxAxis], MaxAxisSize * Extents[1] / Extents[MaxAxis], MaxAxisSize * Extents[2] / Extents[MaxAxis]);
		Counts[0] = Counts[0] < 1 ? 1 : Counts[0];
		Counts[1] = Counts[1] < 1 ? 1 : Counts[1];
		Counts[2] = Counts[2] < 1 ? 1 : Counts[2];

		TUniformGrid<FReal, 3> Grid(BoundingBox.Min(), BoundingBox.Max(), Counts, 1);
		FTriangleMesh CollisionMesh(MoveTemp(Elements));
		FErrorReporter ErrorReporter;
		return FLevelSet(ErrorReporter, Grid, SurfaceParticles, CollisionMesh);
	}

	/**/
	void AppendDynamicParticleConvexBox(FPBDRigidParticleHandle& InParticles, const FVec3& Scale, FReal Margin)
	{
		FConvex::FVec3Type ScaleFloat{ FRealSingle(Scale.X), FRealSingle(Scale.Y), FRealSingle(Scale.Z) };

		TArray<FConvex::FVec3Type> Cube;
		Cube.SetNum(9);
		Cube[0] = FConvex::FVec3Type{ -1, -1, -1 } * ScaleFloat;
		Cube[1] = FConvex::FVec3Type{ -1, -1, 1 } * ScaleFloat;
		Cube[2] = FConvex::FVec3Type{ -1, 1, -1 } * ScaleFloat;
		Cube[3] = FConvex::FVec3Type{ -1, 1, 1 } * ScaleFloat;
		Cube[4] = FConvex::FVec3Type{ 1, -1, -1 } * ScaleFloat;
		Cube[5] = FConvex::FVec3Type{ 1, -1, 1 } * ScaleFloat;
		Cube[6] = FConvex::FVec3Type{ 1, 1, -1 } * ScaleFloat;
		Cube[7] = FConvex::FVec3Type{ 1, 1, 1 } * ScaleFloat;
		Cube[8] = FConvex::FVec3Type{ 0, 0, 0 };

		InParticles.SetX(FVec3(0, 0, 0));
		InParticles.SetV(FVec3(0, 0, 0));
		InParticles.SetR(FRotation3::MakeFromEuler(FVec3(0, 0, 0)).GetNormalized());
		InParticles.SetW(FVec3(0, 0, 0));
		InParticles.SetP(InParticles.GetX());
		InParticles.SetQ(InParticles.GetR());

		// TODO: Change this error prone API to set bounds more automatically. This is easy to forget
		InParticles.SetLocalBounds(FAABB3(Cube[0], Cube[7]));
		InParticles.UpdateWorldSpaceState(FRigidTransform3::Identity, FVec3(0));
		InParticles.SetHasBounds(true);

		InParticles.M() = 1.0;
		InParticles.InvM() = 1.0;
		InParticles.I() = TVec3<FRealSingle>(1);
		InParticles.InvI() = TVec3<FRealSingle>(1);
		InParticles.SetGeometry(MakeImplicitObjectPtr<FConvex>(Cube, Margin));
		InParticles.SetObjectStateLowLevel(EObjectStateType::Dynamic);

		::ChaosTest::SetParticleSimDataToCollide({ &InParticles });
	}

	FPBDRigidParticleHandle* AppendDynamicParticleConvexBox(FPBDRigidsSOAs& SOAs, const FVec3& Scale)
	{
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(1);
		AppendDynamicParticleConvexBox(*Particles[0], Scale, 0.0f);
		return Particles[0];
	}

	FPBDRigidParticleHandle* AppendDynamicParticleConvexBoxMargin(FPBDRigidsSOAs& SOAs, const FVec3& Scale, FReal Margin)
	{
		TArray<FPBDRigidParticleHandle*> Particles = SOAs.CreateDynamicParticles(1);
		AppendDynamicParticleConvexBox(*Particles[0], Scale, Margin);
		return Particles[0];
	}

	/**/
	FVec3 ObjectSpacePoint(FPBDRigidParticles& InParticles, const int32 Index, const FVec3& WorldSpacePoint)
	{
		FRigidTransform3 LocalToWorld(InParticles.GetX(Index), InParticles.GetR(Index));
		return LocalToWorld.InverseTransformPosition(WorldSpacePoint);
	}

	FVec3 ObjectSpacePoint(FGeometryParticleHandle& Particle, const FVec3& WorldSpacePoint)
	{
		FRigidTransform3 LocalToWorld(Particle.GetX(), Particle.GetR());
		return LocalToWorld.InverseTransformPosition(WorldSpacePoint);
	}

	/**/
	FReal PhiWithNormal(FPBDRigidParticles& InParticles, const int32 Index, const FVec3& WorldSpacePoint, FVec3& Normal)
	{
		FRigidTransform3(InParticles.GetX(Index), InParticles.GetR(Index));
		FVec3 BodySpacePoint = ObjectSpacePoint(InParticles, Index, WorldSpacePoint);
		FReal LocalPhi = InParticles.GetGeometry(Index)->PhiWithNormal(BodySpacePoint, Normal);
		Normal = FRigidTransform3(InParticles.GetX(Index), InParticles.GetR(Index)).TransformVector(Normal);
		return LocalPhi;
	}

	/**/
	FReal SignedDistance(FPBDRigidParticles& InParticles, const int32 Index, const FVec3& WorldSpacePoint)
	{
		FVec3 Normal;
		return PhiWithNormal(InParticles, Index, WorldSpacePoint, Normal);
	}

	/**/
	FReal PhiWithNormal(FGeometryParticleHandle& Particle, const FVec3& WorldSpacePoint, FVec3& Normal)
	{
		FRigidTransform3(Particle.GetX(), Particle.GetR());
		FVec3 BodySpacePoint = ObjectSpacePoint(Particle, WorldSpacePoint);
		FReal LocalPhi = Particle.GetGeometry()->PhiWithNormal(BodySpacePoint, Normal);
		Normal = FRigidTransform3(Particle.GetX(), Particle.GetR()).TransformVector(Normal);
		return LocalPhi;
	}

	/**/
	FReal SignedDistance(FGeometryParticleHandle& Particle, const FVec3& WorldSpacePoint)
	{
		FVec3 Normal;
		return PhiWithNormal(Particle, WorldSpacePoint, Normal);
	}

	/**/
	FVec3 RandAxis()
	{
		for (int32 It = 0; It < 1000; ++It)
		{
			FVec3 Point = FVec3(FMath::RandRange(-1.0f, 1.0f), FMath::RandRange(-1.0f, 1.0f), FMath::RandRange(-1.0f, 1.0f));
			if (Point.Size() > KINDA_SMALL_NUMBER)
			{
				return FVec3(Point.GetSafeNormal());
			}
		}
		return FVec3(FVector::UpVector);
	}

	void SetShapeFilterToCollide(Chaos::FPerShapeData& Shape)
	{
		// Change the filter to block everything and to be marked as simple collision
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		const Chaos::Filter::FShapeFilterData ShapeFilterData = Chaos::Filter::FShapeFilterBuilder::BuildBlockAll(Chaos::EFilterFlags::SimpleCollision);
		Shape.SetShapeFilterData(ShapeFilterData);
		PRAGMA_ENABLE_INTERNAL_WARNINGS
	}

	void SetParticleSimDataToCollide(TArray< Chaos::FGeometryParticle* > ParticleArray)
	{
		for (Chaos::FGeometryParticle* Particle : ParticleArray)
		{
			for (const TUniquePtr<Chaos::FPerShapeData>& Shape :Particle->ShapesArray())
			{
				SetShapeFilterToCollide(*Shape);
			}
		}
	}
	void SetParticleSimDataToCollide(TArray< Chaos::FGeometryParticleHandle* > ParticleArray)
	{
		for (Chaos::FGeometryParticleHandle* Particle : ParticleArray)
		{
			for (const TUniquePtr<Chaos::FPerShapeData>& Shape : Particle->ShapesArray())
			{
				SetShapeFilterToCollide(*Shape);
			}
		}
	}

	TArray<FConvex::FVec3Type> MakeBoxVerts(const FConvex::FVec3Type& Center, const FConvex::FVec3Type& HalfSize)
	{
		return
		{
			Center + FConvex::FVec3Type(-HalfSize.X, -HalfSize.Y, -HalfSize.Z),
			Center + FConvex::FVec3Type(-HalfSize.X,  HalfSize.Y, -HalfSize.Z),
			Center + FConvex::FVec3Type(HalfSize.X,  HalfSize.Y, -HalfSize.Z),
			Center + FConvex::FVec3Type(HalfSize.X, -HalfSize.Y, -HalfSize.Z),
			Center + FConvex::FVec3Type(-HalfSize.X, -HalfSize.Y,  HalfSize.Z),
			Center + FConvex::FVec3Type(-HalfSize.X,  HalfSize.Y,  HalfSize.Z),
			Center + FConvex::FVec3Type(HalfSize.X,  HalfSize.Y,  HalfSize.Z),
			Center + FConvex::FVec3Type(HalfSize.X, -HalfSize.Y,  HalfSize.Z),
		};
	}

	FImplicitConvex3 CreateConvexBox(const FConvex::FVec3Type& BoxSize, const FReal Margin)
	{
		const FConvex::FVec3Type HalfSize = 0.5f * BoxSize;
		return FImplicitConvex3(MakeBoxVerts(FConvex::FVec3Type(0), HalfSize), Margin);
	}

	FImplicitConvex3 CreateConvexBox(const FConvex::FVec3Type& BoxMin, const FConvex::FVec3Type& BoxMax, const FReal Margin)
	{
		const FConvex::FVec3Type Center = 0.5f * (BoxMin + BoxMax);
		const FConvex::FVec3Type HalfSize = 0.5f * (BoxMax - BoxMin);
		return FImplicitConvex3(MakeBoxVerts(Center, HalfSize), Margin);
	}


	TImplicitObjectInstanced<FImplicitConvex3> CreateInstancedConvexBox(const FConvex::FVec3Type& BoxSize, const FReal Margin)
	{
		const FConvex::FVec3Type HalfSize = 0.5f * BoxSize;
		FConvexPtr BoxConvex( new FImplicitConvex3(MakeBoxVerts(FConvex::FVec3Type(0), HalfSize), 0.0f));
		return TImplicitObjectInstanced<FImplicitConvex3>(BoxConvex, Margin);
	}

	TImplicitObjectScaled<FImplicitConvex3> CreateScaledConvexBox(const FConvex::FVec3Type& BoxSize, const FVec3 BoxScale, const FReal Margin)
	{
		const FConvex::FVec3Type HalfSize = 0.5f * BoxSize;
		FConvexPtr BoxConvex( new FImplicitConvex3(MakeBoxVerts(FConvex::FVec3Type(0), HalfSize), 0.0f));
		return TImplicitObjectScaled<FImplicitConvex3>(BoxConvex, BoxScale, Margin);
	}

}
