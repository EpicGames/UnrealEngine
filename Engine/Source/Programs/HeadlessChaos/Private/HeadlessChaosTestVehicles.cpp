// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeadlessChaos.h"
#include "HeadlessChaosTestUtility.h"

// for System Unit Tests
#include "AerodynamicsSystem.h"
#include "AerofoilSystem.h"
#include "TransmissionSystem.h"
#include "EngineSystem.h"
#include "WheelSystem.h"
#include "TireSystem.h"
#include "SuspensionSystem.h"
#include "SuspensionUtility.h"
#include "SteeringUtility.h"
#include "TransmissionUtility.h"

// for Simulation Tests
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/Box.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"

//////////////////////////////////////////////////////////////////////////
// These tests are mostly working in real word units rather than Unreal 
// units as it's easier to tell if the simulations are working close to 
// reality. i.e. Google stopping distance @ 30MPH ==> typically 15 metres
//////////////////////////////////////////////////////////////////////////

namespace ChaosTest
{
	using namespace Chaos;

	GTEST_TEST(AllTraits, VehicleTest_VehicleUtilityGraph)
	{
		Chaos::FGraph Graph;

		Graph.Add(FVector2D(10.f, 0.f));
		Graph.Add(FVector2D(20.f, 30.f));
		Graph.Add(FVector2D(30.f, 40.f));
		Graph.Add(FVector2D(40.f, 60.f));
		Graph.Add(FVector2D(50.f, 90.f));

		float Tolerance = 0.001f;
		EXPECT_NEAR(Graph.EvaluateY(-5.f), 0.f, Tolerance);
		EXPECT_NEAR(Graph.EvaluateY(5.f), 0.f, Tolerance);
		EXPECT_NEAR(Graph.EvaluateY(10.f), 0.f, Tolerance);
		EXPECT_NEAR(Graph.EvaluateY(15.f), 15.f, Tolerance);
		EXPECT_NEAR(Graph.EvaluateY(40.f), 60.f, Tolerance);
		EXPECT_NEAR(Graph.EvaluateY(25.f), 35.f, Tolerance);
		EXPECT_NEAR(Graph.EvaluateY(45.f), 75.f, Tolerance);
		EXPECT_NEAR(Graph.EvaluateY(50.f), 90.f, Tolerance);
		EXPECT_NEAR(Graph.EvaluateY(55.f), 90.f, Tolerance);
		EXPECT_NEAR(Graph.EvaluateY(60.f), 90.f, Tolerance);
		EXPECT_NEAR(Graph.EvaluateY(1.0E21f), 90.f, Tolerance);
		EXPECT_NEAR(Graph.EvaluateY(-1.0E21f), 0.f, Tolerance);
	}

	GTEST_TEST(AllTraits, VehicleTest_CalculateSlipAngle)
	{
		float SlipAngleFwds = FVehicleUtility::CalculateSlipAngle(0, 5);
		float SlipAngleReverse = FVehicleUtility::CalculateSlipAngle(0, -5);

		EXPECT_NEAR(SlipAngleFwds, 0.0f, 0.001f);
		EXPECT_NEAR(SlipAngleFwds, SlipAngleReverse, 0.001f);

		float SlipAngleFwdsSide = FVehicleUtility::CalculateSlipAngle(1, 5);
		float SlipAngleReverseSide = FVehicleUtility::CalculateSlipAngle(1, -5);
		float SlipAngleFwdsSide2 = FVehicleUtility::CalculateSlipAngle(-1, 5);
		float SlipAngleReverseSide2 = FVehicleUtility::CalculateSlipAngle(-1, -5);
		EXPECT_NEAR(SlipAngleFwdsSide, SlipAngleReverseSide, 0.001f);
		EXPECT_NEAR(SlipAngleFwdsSide, SlipAngleFwdsSide2, 0.001f);
		EXPECT_NEAR(SlipAngleFwdsSide, SlipAngleReverseSide2, 0.001f);

		float SlipAngleSide1 = FVehicleUtility::CalculateSlipAngle(2, 0);
		float SlipAngleSide2 = FVehicleUtility::CalculateSlipAngle(-2, 0);
		EXPECT_NEAR(SlipAngleSide1, HALF_PI, 0.001f);
		EXPECT_NEAR(SlipAngleSide1, SlipAngleSide2, 0.001f);

	}

	GTEST_TEST(AllTraits, VehicleTest_SteeringUtilityTurnRadius)
	{
		float RadiusTolerance = 0.01f;

		float Radius = 3.0f;
		FVector PtA(0.f, Radius, 0.f);
		FVector PtB(Radius, 0.f, 0.f);
		FVector PtC(0.f, -Radius, 0.f);
		FVector PtD(FMath::Sin(PI/5.f)* Radius, FMath::Cos(PI / 5.f)* Radius, 0.f);

		float CalculatedRadius = FVehicleUtility::TurnRadiusFromThreePoints(PtA, PtB, PtC);
		EXPECT_LT(CalculatedRadius - Radius, RadiusTolerance);

		CalculatedRadius = FVehicleUtility::TurnRadiusFromThreePoints(PtB, PtA, PtC);
		EXPECT_LT(CalculatedRadius - Radius, RadiusTolerance);

		CalculatedRadius = FVehicleUtility::TurnRadiusFromThreePoints(PtC, PtB, PtA);
		EXPECT_LT(CalculatedRadius - Radius, RadiusTolerance);

		CalculatedRadius = FVehicleUtility::TurnRadiusFromThreePoints(PtA, PtB, PtD);
		EXPECT_LT(CalculatedRadius - Radius, RadiusTolerance);

		// no answer all points lie on a line, no radius possible, returns 0
		CalculatedRadius = FVehicleUtility::TurnRadiusFromThreePoints(FVector(1.f,0.f,0.f), FVector(2.f, 0.f, 0.f), FVector(3.f, 0.f, 0.f));
		EXPECT_LT(CalculatedRadius, RadiusTolerance);

		float LargeRadius = 55.0f;
		FVector LargePtA(FMath::Sin(PI / 5.f) * LargeRadius, FMath::Cos(PI / 5.f) * LargeRadius, 0.f);
		FVector LargePtB(FMath::Sin(PI / 4.f) * LargeRadius, FMath::Cos(PI / 4.f) * LargeRadius, 0.f);
		FVector LargePtC(FMath::Sin(PI / 3.f) * LargeRadius, FMath::Cos(PI / 3.f) * LargeRadius, 0.f);

		CalculatedRadius = FVehicleUtility::TurnRadiusFromThreePoints(LargePtA, LargePtB, LargePtC);
		EXPECT_LT(CalculatedRadius - LargeRadius, RadiusTolerance);

	}

	GTEST_TEST(AllTraits, VehicleTest_SteeringUtilityIntersectTwoCircles)
	{
		{
			float R1 = 3.f;
			float R2 = 2.f;
			FVector2D IntersectionPt;

			bool ResultOk = FSteeringUtility::IntersectTwoCircles(R1, R2, 0.5f, IntersectionPt);
			EXPECT_FALSE(ResultOk);

			ResultOk = FSteeringUtility::IntersectTwoCircles(R1, R2, 6.0f, IntersectionPt);
			EXPECT_FALSE(ResultOk);

			ResultOk = FSteeringUtility::IntersectTwoCircles(R1, R2, 5.0f, IntersectionPt);
			EXPECT_TRUE(ResultOk);
			EXPECT_LT(IntersectionPt.X - 3.f, SMALL_NUMBER);
			EXPECT_LT(IntersectionPt.Y, SMALL_NUMBER);

			ResultOk = FSteeringUtility::IntersectTwoCircles(R1, R2, 1.0f, IntersectionPt);
			EXPECT_TRUE(ResultOk);
			EXPECT_LT(IntersectionPt.X - 3.f, SMALL_NUMBER);
			EXPECT_LT(IntersectionPt.Y, SMALL_NUMBER);
		}

		{
			float Tolerance = 0.001f;
			float R1 = 3.f;
			float R2 = 2.f;
			FVector2D IntersectionPt;
			bool ResultOk = false;

			for (float D = 1.f; D <= 5.0f; D += 0.2f)
			{
				FVector2D C1(0, 0);
				FVector2D C2(D, 0);
				ResultOk = FSteeringUtility::IntersectTwoCircles(R1, R2, D, IntersectionPt);
				EXPECT_TRUE(ResultOk);
				EXPECT_GT(IntersectionPt.X, 0.f);
				EXPECT_GE(IntersectionPt.Y, 0.f);
				EXPECT_LT(IntersectionPt.Y, R1);
				EXPECT_LT(IntersectionPt.Y, R2);
				EXPECT_LT((IntersectionPt - C1).Size() - R1, Tolerance);
				EXPECT_LT((C2 - IntersectionPt).Size() - R2, Tolerance);
			}
		}
	}

	GTEST_TEST(AllTraits, VehicleTest_SteeringUtilityCalcJointPositions)
	{
		float T = 1.0f;			// Track width
		float Beta = 0.f;		// Angle
		float R = 0.25f;		// Radius
		FVector2D C1, C2;		// steering rod centre, track rod centre
		float R1, R2;			// steering rod radius, track rod radius
		FSteeringUtility::CalcJointPositions(T, Beta, R, C1, R1, C2, R2);

		EXPECT_LT(R1 - T/2.f, SMALL_NUMBER);
		EXPECT_LT(R2 - R, SMALL_NUMBER);
		EXPECT_LT(C1.X, SMALL_NUMBER);
		EXPECT_LT(C1.Y, SMALL_NUMBER);
		EXPECT_LT(C2.X - T/2.f, SMALL_NUMBER);
		EXPECT_LT(C2.Y-R, SMALL_NUMBER);

		T = 1.0f;
		Beta = 45.0f;
		R = 0.25f;
		FSteeringUtility::CalcJointPositions(T, Beta, R, C1, R1, C2, R2);

		float Dist = FMath::Sqrt(R*R/2.f);
		EXPECT_LT(R1 - (T / 2.f - Dist), SMALL_NUMBER);
		EXPECT_LT(R2 - R, SMALL_NUMBER);
		EXPECT_LT(C1.X, SMALL_NUMBER);
		EXPECT_LT(C1.Y, SMALL_NUMBER);
		EXPECT_LT(C2.X - T / 2.f, SMALL_NUMBER);
		EXPECT_LT(C2.Y - Dist, SMALL_NUMBER);

		T = 2.0f;
		Beta = 18.0f;
		R = 0.25f;
		FSteeringUtility::CalcJointPositions(T, Beta, R, C1, R1, C2, R2);

		float Input = 0.0f;
		float OutSteerAngle;
		FVector2D OutC1;
		FVector2D OutPt;
		FSteeringUtility::CalculateAkermannAngle(false, Input, C2, R1, R2, OutSteerAngle, OutC1, OutPt);

		EXPECT_LT(OutSteerAngle - Beta, KINDA_SMALL_NUMBER);
		EXPECT_GT(OutPt.X, 0.f);
		EXPECT_LT(OutPt.X, T/ 2.0f);

	}

	GTEST_TEST(AllTraits, VehicleTest_SteeringUtilityAkermannSetup)
	{
		float WheelBase = 3.8f;
		float TrackWidth = 1.8f;
		float R = 0.25f;
		float Beta = FSteeringUtility::CalculateBetaDegrees(TrackWidth, WheelBase);

		// This is a bit pointless I'm just doing the same sum - is there another way to confirm the results
		EXPECT_LT(Beta - RadToDeg(FMath::Atan2(0.9f, 3.8f)), KINDA_SMALL_NUMBER);

		// Beta is about 18 degrees +/- on a normal car
		EXPECT_GT(Beta, 10.f);
		EXPECT_LT(Beta, 25.f);

		float H, S;
		FSteeringUtility::AkermannSetup(TrackWidth, Beta, R, H, S);

		// This is a bit pointless I'm just doing the same sum - is there another way to confirm the results
		EXPECT_LT(S - (TrackWidth - 2.0f * FMath::DegreesToRadians(FMath::Sin(Beta)) * R), KINDA_SMALL_NUMBER);

		EXPECT_LT(H, R);
		EXPECT_LT(S, TrackWidth);
		EXPECT_GT(H, 0.f);
		EXPECT_GT(H, 0.f);
	}

	GTEST_TEST(AllTraits, VehicleTest_TransmissionUtilityIsWheelPowered)
	{
		TArray<FSimpleWheelSim> Wheels;

		FSimpleWheelConfig SetupFront;
		SetupFront.AxleType = FSimpleWheelConfig::EAxleType::Front;
		SetupFront.EngineEnabled = false;

		FSimpleWheelConfig SetupRear;
		SetupRear.AxleType = FSimpleWheelConfig::EAxleType::Rear;
		SetupRear.EngineEnabled = true;

		for (int I = 0; I < 2; I++)
		{
			FSimpleWheelSim WheelFront(&SetupFront);
			Wheels.Add(WheelFront);
		}

		for (int I = 0; I < 4; I++)
		{
			FSimpleWheelSim WheelRear(&SetupRear);
			Wheels.Add(WheelRear);
		}

		EXPECT_EQ(FTransmissionUtility::GetNumWheelsOnAxle(FSimpleWheelConfig::EAxleType::Front, Wheels), 2);
		EXPECT_EQ(FTransmissionUtility::GetNumWheelsOnAxle(FSimpleWheelConfig::EAxleType::Rear, Wheels), 4);

		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::AllWheelDrive, Wheels[0]), true); // front
		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::AllWheelDrive, Wheels[1]), true); // front
		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::AllWheelDrive, Wheels[2]), true); // rear
		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::AllWheelDrive, Wheels[5]), true); // rear

		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::FrontWheelDrive, Wheels[0]), true); // front
		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::FrontWheelDrive, Wheels[1]), true); // front
		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::FrontWheelDrive, Wheels[2]), false); // rear
		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::FrontWheelDrive, Wheels[5]), false); // rear

		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::RearWheelDrive, Wheels[0]), false); // front
		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::RearWheelDrive, Wheels[1]), false); // front
		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::RearWheelDrive, Wheels[2]), true); // rear
		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::RearWheelDrive, Wheels[5]), true); // rear

		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::UndefinedDrive, Wheels[0]), false); // front
		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::UndefinedDrive, Wheels[1]), false); // front
		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::UndefinedDrive, Wheels[2]), true); // rear
		EXPECT_EQ(FTransmissionUtility::IsWheelPowered(EDifferentialType::UndefinedDrive, Wheels[5]), true); // rear

	}

	GTEST_TEST(AllTraits, VehicleTest_TransmissionUtilityGetTorqueRatioForWheel)
	{
		TArray<FSimpleWheelSim> Wheels;

		FSimpleWheelConfig SetupFront;
		SetupFront.AxleType = FSimpleWheelConfig::EAxleType::Front;
		SetupFront.EngineEnabled = false;

		FSimpleWheelConfig SetupRear;
		SetupRear.AxleType = FSimpleWheelConfig::EAxleType::Rear;
		SetupRear.EngineEnabled = true;

		for (int I=0; I < 2; I++)
		{
			FSimpleWheelSim WheelFront(&SetupFront);
			Wheels.Add(WheelFront);
		}

		for (int I = 0; I < 4; I++)	
		{
			FSimpleWheelSim WheelRear(&SetupRear);
			Wheels.Add(WheelRear);
		}

		EXPECT_EQ(FTransmissionUtility::GetNumWheelsOnAxle(FSimpleWheelConfig::EAxleType::Front, Wheels), 2);
		EXPECT_EQ(FTransmissionUtility::GetNumWheelsOnAxle(FSimpleWheelConfig::EAxleType::Rear, Wheels), 4);

		FSimpleDifferentialConfig DiffSetup;
		DiffSetup.FrontRearSplit = 0.5f;
		FSimpleDifferentialSim Differential(&DiffSetup);
		
		float Error = 0.01f;

		DiffSetup.DifferentialType = EDifferentialType::AllWheelDrive;
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 0, Wheels), 0.25f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 1, Wheels), 0.25f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 2, Wheels), 0.125f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 3, Wheels), 0.125f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 4, Wheels), 0.125f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 5, Wheels), 0.125f, Error); // rear

		DiffSetup.DifferentialType = EDifferentialType::FrontWheelDrive;
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 0, Wheels), 0.5f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 1, Wheels), 0.5f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 2, Wheels), 0.0f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 3, Wheels), 0.0f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 4, Wheels), 0.0f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 5, Wheels), 0.0f, Error); // rear

		DiffSetup.DifferentialType = EDifferentialType::RearWheelDrive;
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 0, Wheels), 0.0f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 1, Wheels), 0.0f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 2, Wheels), 0.25f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 3, Wheels), 0.25f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 4, Wheels), 0.25f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 5, Wheels), 0.25f, Error); // rear

		DiffSetup.DifferentialType = EDifferentialType::UndefinedDrive;
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 0, Wheels), 0.0f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 1, Wheels), 0.0f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 2, Wheels), 0.25f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 3, Wheels), 0.25f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 4, Wheels), 0.25f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 5, Wheels), 0.25f, Error); // rear

		Differential.FrontRearSplit = 0.8f; // more torque to the rear
		DiffSetup.DifferentialType = EDifferentialType::AllWheelDrive;
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 0, Wheels), 0.1f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 1, Wheels), 0.1f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 2, Wheels), 0.2f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 3, Wheels), 0.2f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 4, Wheels), 0.2f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 5, Wheels), 0.2f, Error); // rear

		for (int I = 0; I < 6; I++)
		{
			Wheels[I].AccessSetup().AxleType = FSimpleWheelConfig::EAxleType::UndefinedAxle;
		}
		// front 2 have SetupFront.EngineEnabled = false, the rest are true
		EXPECT_EQ(FTransmissionUtility::GetNumDrivenWheels(Wheels), 4);
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 0, Wheels), 0.f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 1, Wheels), 0.f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 2, Wheels), 1.f / 4.f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 3, Wheels), 1.f / 4.f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 4, Wheels), 1.f / 4.f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 5, Wheels), 1.f / 4.f, Error); // rear

		// now turn off torque to the rear two
		Wheels[4].EngineEnabled = false;
		Wheels[5].EngineEnabled = false;
		EXPECT_EQ(FTransmissionUtility::GetNumDrivenWheels(Wheels), 2);
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 0, Wheels), 0.f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 1, Wheels), 0.f, Error); // front
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 2, Wheels), 1.f / 2.f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 3, Wheels), 1.f / 2.f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 4, Wheels), 0.f, Error); // rear
		EXPECT_NEAR(FTransmissionUtility::GetTorqueRatioForWheel(Differential, 5, Wheels), 0.f, Error); // rear

	}


	GTEST_TEST(AllTraits, VehicleTest_SystemTemplate)
	{
		FSimpleTireConfig Setup;
		{
			Setup.Radius = 0.44f;
		}

		FSimpleTireSim Tire(&Setup);

		EXPECT_LT(Tire.AccessSetup().Radius - Setup.Radius, SMALL_NUMBER);
		EXPECT_LT(Tire.Setup().Radius - Setup.Radius, SMALL_NUMBER);

	}

	// Aerodynamics
	GTEST_TEST(AllTraits, VehicleTest_Aerodynamics)
	{
		FSimpleAerodynamicsConfig Setup;
		{
			Setup.AreaMetresSquared = 1.f * 2.f; // 1x2 m
			Setup.DragCoefficient = 0.5f;
			Setup.DownforceCoefficient = 0.1f;
		}

		FSimpleAerodynamicsSim Aerofoil(&Setup);
		Aerofoil.SetDensityOfMedium(RealWorldConsts::AirDensity());

		float Drag = 0;
		Drag = Aerofoil.GetDragForceFromVelocity(0.f);
		EXPECT_LT(Drag, SMALL_NUMBER);

		Drag = Aerofoil.GetDragForceFromVelocity(1.f); // 1m.s-1
		EXPECT_LT(Drag - (RealWorldConsts::AirDensity() * 0.5f), SMALL_NUMBER);

		Drag = Aerofoil.GetDragForceFromVelocity(5.f); // 5m.s-1
		EXPECT_LT(Drag - (RealWorldConsts::AirDensity() * 0.5f * 25.f), SMALL_NUMBER);

		Drag = Aerofoil.GetDragForceFromVelocity(10.f); // 10m.s-1
		EXPECT_LT(Drag - (RealWorldConsts::AirDensity() * 0.5f * 100.f), SMALL_NUMBER);

		float Lift = 0;
		Lift = Aerofoil.GetLiftForceFromVelocity(0.f);
		EXPECT_LT(Lift, SMALL_NUMBER);

		Lift = Aerofoil.GetLiftForceFromVelocity(1.f);
		EXPECT_LT(Lift - (RealWorldConsts::AirDensity() * 0.1f), SMALL_NUMBER);

		Lift = Aerofoil.GetLiftForceFromVelocity(5.f);
		EXPECT_LT(Lift - (RealWorldConsts::AirDensity() * 0.1f * 25.f), SMALL_NUMBER);

		Lift = Aerofoil.GetLiftForceFromVelocity(10.f);
		EXPECT_LT(Lift - (RealWorldConsts::AirDensity() * 0.1f * 100.f), SMALL_NUMBER);

		// investigating Unral Units vs real world units
		/*{
			FSimpleAerodynamicsConfig Setup;
			{
				Setup.AreaMetresSquared = 1.f * 2.f; // 1x2 m
				Setup.DragCoefficient = 0.5f;
				Setup.DownforceCoefficient = 0.1f;
			}

			///////////////////////////////////////////////////////////////
			FSimpleAerodynamicsSim Aerofoil(&Setup);
			Aerofoil.SetDensityOfMedium(RealWorldConsts::AirDensity());

			float Drag5 = 0;
			Drag5 = Aerofoil.GetDragForceFromVelocity(5.f);


			/////////////////////

			FSimpleAerodynamicsConfig SetupB;
			{
				Setup.AreaMetresSquared = 1.f * 2.f; // 1x2 m
				Setup.DragCoefficient = 0.5f;
				Setup.DownforceCoefficient = 0.1f;
			}

			FSimpleAerodynamicsSim AerofoilB(&Setup);
			Aerofoil.SetDensityOfMedium(RealWorldConsts::AirDensity());


			////////////////////////////////////////////////////////////////////
			// Trying to get head around standard vs unreal units.
			// If the mass is the same between the two simulations then the
			// the following is true.
			float PosA = 0, PosB = 0;
			float VelA = 100.0f;
			float VelB = 10000.0f;

			float DeltaTime = 1.f / 30.f;

			for (int i = 0; i < 200; i++)
			{
				float DragA = Aerofoil.GetDragForceFromVelocity(VelA);
				float DragB = (AerofoilB.GetDragForceFromVelocity(CmToM(VelB))); // no final MToCm conversion

				VelA += (DragA / 1000.0f) * DeltaTime;
				VelB += (DragB / 1000.0f) * DeltaTime;
				PosA += VelA * DeltaTime;
				PosB += VelB * DeltaTime;

				UE_LOGF(LogChaos, Warning, "Drag %f %f", DragA, DragB);
				UE_LOGF(LogChaos, Warning, "Vel %f %f", VelA, VelB);

				UE_LOGF(LogChaos, Warning, "------");
			}
		}*/
	}

	GTEST_TEST(AllTraits, VehicleTest_Aerofoil)
	{
		FAerofoilConfig RWingSetup;
		RWingSetup.Offset.Set(-0.8f, 3.0f, 0.0f);
		RWingSetup.UpAxis.Set(0.0f, 0.f, 1.0f);
		RWingSetup.Area = 8.2f;
		RWingSetup.Camber = 3.0f;
		RWingSetup.MaxControlAngle = 1.0f;
		RWingSetup.StallAngle = 16.0f;
		RWingSetup.Type = EAerofoilType::Wing;

		FAerofoil RWing(&RWingSetup);

		RWing.SetControlSurface(0.0f);
		RWing.SetDensityOfMedium(RealWorldConsts::AirDensity());

		float Altitude = 100.0f;
		float DeltaTime = 1.0f / 30.0f;

		//////////////////////////////////////////////////////////////////////////

		FTransform BodyTransform = FTransform::Identity;
		FVector Velocity(1.0f, 0.0f, 0.0f);

		float AOAFlat = RWing.CalcAngleOfAttackDegrees(FVector(0, 0, 1), FVector(-1, 0, 0));
		EXPECT_LT(AOAFlat, SMALL_NUMBER);

		float AOAFlat2 = RWing.CalcAngleOfAttackDegrees(FVector(0, 0, 1), FVector(1, 0, 0));
		EXPECT_LT(AOAFlat2, SMALL_NUMBER);

		float AOA90 = RWing.CalcAngleOfAttackDegrees(FVector(0, 0, 1), FVector(0, 0, 1));
		EXPECT_LT(AOA90 - 90.0f, SMALL_NUMBER);

		float AOA45 = RWing.CalcAngleOfAttackDegrees(FVector(0, 0, 1), FVector(0, 0.707, 0.707));
		EXPECT_LT(AOA45 - 45.0f, SMALL_NUMBER);

		//////////////////////////////////////////////////////////////////////////
		// Lift
		{
			float Zero = RWing.CalcLiftCoefficient(0, 0);
			EXPECT_LT(Zero, SMALL_NUMBER);

			float Two = RWing.CalcLiftCoefficient(2, 0);
			float NegTwo = RWing.CalcLiftCoefficient(-2, 0);
			EXPECT_GT(Two, SMALL_NUMBER);
			EXPECT_LT(NegTwo, SMALL_NUMBER);
			EXPECT_LT(Two - FMath::Abs(NegTwo), SMALL_NUMBER);

			float Three = RWing.CalcLiftCoefficient(0, 3);
			float NegThree = RWing.CalcLiftCoefficient(0, -3);
			EXPECT_GT(Three, SMALL_NUMBER);
			EXPECT_LT(NegThree, SMALL_NUMBER);
			EXPECT_LT(Three - FMath::Abs(NegThree), SMALL_NUMBER);

			float Nine = RWing.CalcLiftCoefficient(6, 3);
			float NegNine = RWing.CalcLiftCoefficient(-6, -3);
			EXPECT_GT(Nine, SMALL_NUMBER);
			EXPECT_LT(NegNine, SMALL_NUMBER);
			EXPECT_LT(Nine - FMath::Abs(NegNine), SMALL_NUMBER);

			float Stall = RWing.CalcLiftCoefficient(RWingSetup.StallAngle, 0);
			float StallPlus = RWing.CalcLiftCoefficient(RWingSetup.StallAngle, 5);
			EXPECT_GT(Stall, Nine);
			EXPECT_GT(Stall, Three);
			EXPECT_GT(Stall, Two);
			EXPECT_GT(Stall, StallPlus);
		}

		// Drag
		{
			float Two = RWing.CalcDragCoefficient(2, 0);
			float NegTwo = RWing.CalcDragCoefficient(-2, 0);
			EXPECT_GT(Two, SMALL_NUMBER);
			EXPECT_GT(NegTwo, SMALL_NUMBER);
			EXPECT_LT(Two - NegTwo, SMALL_NUMBER);

			float Six = RWing.CalcDragCoefficient(4, 2);
			float NegSix = RWing.CalcDragCoefficient(-4, -2);
			EXPECT_GT(Six, SMALL_NUMBER);
			EXPECT_GT(NegSix, SMALL_NUMBER);
			EXPECT_LT(Six - NegSix, SMALL_NUMBER);

			float AltNegTwo = RWing.CalcDragCoefficient(2, -4);
			EXPECT_GT(AltNegTwo, SMALL_NUMBER);
			EXPECT_LT(AltNegTwo - NegTwo, SMALL_NUMBER);
		}


		////////////////////////////////////////////////////////////////////////////

		FVector Velocity1(0.0f, 0.0f, 10.0f);
		FVector RWForceZero = RWing.GetForce(BodyTransform, Velocity1, Altitude, DeltaTime);
		EXPECT_LT(FMath::Abs(RWForceZero.X), SMALL_NUMBER);
		EXPECT_LT(FMath::Abs(RWForceZero.Y), SMALL_NUMBER);
		EXPECT_LT(RWForceZero.Z, 0.f); // drag value opposes velocity direction

		FVector Velocity2(0.0f, 10.0f, 10.0f);
		FVector RWForce3 = RWing.GetForce(BodyTransform, Velocity2, Altitude, DeltaTime);
		EXPECT_LT(FMath::Abs(RWForce3.X), SMALL_NUMBER);
		EXPECT_LT(RWForce3.Y, 0.0f);
		EXPECT_LT(RWForce3.Z, 0.0f);

		FVector Velocity3(10.0f, 0.0f, 0.0f);
		FVector RWForce4 = RWing.GetForce(BodyTransform, Velocity3, Altitude, DeltaTime);
		EXPECT_LT(RWForce4.X, 0.0f);
		EXPECT_LT(FMath::Abs(RWForce4.Y), SMALL_NUMBER);
		EXPECT_GT(RWForce4.Z, 0.0f);

	}


	// Transmission
	GTEST_TEST(AllTraits, VehicleTest_TransmissionManualGearSelection)
	{
		FSimpleTransmissionConfig Setup;
		{
			Setup.ForwardRatios.Add(4.f);
			Setup.ForwardRatios.Add(3.f);
			Setup.ForwardRatios.Add(2.f);
			Setup.ForwardRatios.Add(1.f);
			Setup.ReverseRatios.Add(3.f);
			Setup.ReverseRatios.Add(6.f);
			Setup.FinalDriveRatio = 4.f;
			Setup.ChangeUpRPM = 3000;
			Setup.ChangeDownRPM = 1200;
			Setup.GearChangeTime = 0.0f;
			Setup.TransmissionType = ETransmissionType::Manual;
			Setup.AutoReverse = true;
			Setup.TransmissionEfficiency = 1.0f;
		}

		FSimpleTransmissionSim Transmission(&Setup);

		EXPECT_EQ(Transmission.GetCurrentGear(), 0);

		// Immediate Gear Change, since Setup.GearChangeTime = 0.0f
		Transmission.ChangeUp();

		EXPECT_EQ(Transmission.GetCurrentGear(), 1);
		Transmission.ChangeUp();
		Transmission.ChangeUp();
		Transmission.ChangeUp();
		EXPECT_EQ(Transmission.GetCurrentGear(), 4);

		Transmission.ChangeUp();
		EXPECT_EQ(Transmission.GetCurrentGear(), 4);

		Transmission.SetGear(1);
		EXPECT_EQ(Transmission.GetCurrentGear(), 1);

		Transmission.ChangeDown();
		EXPECT_EQ(Transmission.GetCurrentGear(), 0);

		Transmission.ChangeDown();
		EXPECT_EQ(Transmission.GetCurrentGear(), -1);

		Transmission.ChangeDown();
		EXPECT_EQ(Transmission.GetCurrentGear(), -2);

		Transmission.ChangeDown();
		EXPECT_EQ(Transmission.GetCurrentGear(), -2);

		Transmission.ChangeUp();
		EXPECT_EQ(Transmission.GetCurrentGear(), -1);

		Transmission.ChangeUp();
		EXPECT_EQ(Transmission.GetCurrentGear(), 0);

		Transmission.SetGear(1);

		// Now change settings so we have a delay in the gear changing
		Transmission.AccessSetup().GearChangeTime = 0.5f;

		Transmission.ChangeUp();
		EXPECT_EQ(Transmission.GetCurrentGear(), 0);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), 0);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), 2);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), 2);

		Transmission.SetGear(4);
		EXPECT_EQ(Transmission.GetCurrentGear(), 0);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), 0);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), 4);


	}

	GTEST_TEST(AllTraits, VehicleTest_TransmissionAutoGearSelection)
	{
		FSimpleTransmissionConfig Setup;
		{
			Setup.ForwardRatios.Add(4.f);
			Setup.ForwardRatios.Add(3.f);
			Setup.ForwardRatios.Add(2.f);
			Setup.ForwardRatios.Add(1.f);
			Setup.ReverseRatios.Add(3.f);
			Setup.ReverseRatios.Add(6.f);
			Setup.FinalDriveRatio = 4.f;
			Setup.ChangeUpRPM = 3000;
			Setup.ChangeDownRPM = 1200;
			Setup.GearChangeTime = 0.0f;
			Setup.TransmissionType = ETransmissionType::Automatic;
			Setup.AutoReverse = true;
			Setup.TransmissionEfficiency = 1.0f;
		}

		FSimpleTransmissionSim Transmission(&Setup);
		Transmission.SetGear(1, true);

		Transmission.SetEngineRPM(1400);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), 1);

		Transmission.SetEngineRPM(2000);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), 1);

		Transmission.SetEngineRPM(3000);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), 2);

		Transmission.SetEngineRPM(2000);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), 2);

		Transmission.SetEngineRPM(1000);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), 1);

		// stays in first, doesn't change to neutral
		Transmission.SetEngineRPM(1000);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), 1);

		Transmission.SetGear(-2, true);

		Transmission.SetEngineRPM(3000);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), -2);

		Transmission.SetEngineRPM(1000);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), -1);

		// stays in reverse first, doesn't change to neutral
		Transmission.SetEngineRPM(1000);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), -1);

		// changes to next reverse gear
		Transmission.SetEngineRPM(3000);
		Transmission.Simulate(0.25f);
		EXPECT_EQ(Transmission.GetCurrentGear(), -2);

	}

	GTEST_TEST(AllTraits, VehicleTest_TransmissionGearRatios)
	{
		FSimpleTransmissionConfig Setup;
		{
			Setup.ForwardRatios.Add(4.f);
			Setup.ForwardRatios.Add(3.f);
			Setup.ForwardRatios.Add(2.f);
			Setup.ForwardRatios.Add(1.f);
			Setup.ReverseRatios.Add(3.f);
			Setup.FinalDriveRatio = 4.f;
			Setup.ChangeUpRPM = 3000;
			Setup.ChangeDownRPM = 1200;
			Setup.GearChangeTime = 0.0f;
			Setup.TransmissionType = ETransmissionType::Automatic;
			Setup.AutoReverse = true;
			Setup.TransmissionEfficiency = 1.0f;
		}

		FSimpleTransmissionSim Transmission(&Setup);

		float Ratio = 0;
		Ratio = Transmission.GetGearRatio(-1);
		EXPECT_LT(-12.f - Ratio, SMALL_NUMBER); // -ve output for reverse gears

		Ratio = Transmission.GetGearRatio(0);
		EXPECT_LT(Ratio, SMALL_NUMBER);

		Ratio = Transmission.GetGearRatio(1);
		EXPECT_LT(16.f - Ratio, SMALL_NUMBER);

		Ratio = Transmission.GetGearRatio(2);
		EXPECT_LT(12.f - Ratio, SMALL_NUMBER);

		Ratio = Transmission.GetGearRatio(3);
		EXPECT_LT(8.f - Ratio, SMALL_NUMBER);

		Ratio = Transmission.GetGearRatio(4);
		EXPECT_LT(4.f - Ratio, SMALL_NUMBER);

	}

	// Engine
	GTEST_TEST(AllTraits, VehicleTest_EngineRPM)
	{
		// #todo: fix engine rev out of gear
		//FSimpleEngineConfig Setup;
		//{
		//	Setup.MaxRPM = 5000;
		//	Setup.EngineIdleRPM = 1000;
		//	Setup.MaxTorque = 100.f;
		//	//Setup.TorqueCurve;
		//}

		//FSimpleEngineSim Engine(&Setup);

		//Engine.SetThrottle(0.f);

		//float DeltaTime = 1.0f / 30.0f;
		//float TOLERANCE = 0.01f;

		//// engine idle - no throttle
		//for (int i = 0; i < 300; i++)
		//{
		//	Engine.Simulate(DeltaTime);
		//}

		//EXPECT_LT(Engine.GetEngineRPM() - Engine.Setup().EngineIdleRPM, TOLERANCE);

		//// apply half throttle
		//Engine.SetThrottle(0.5f);

		//for (int i = 0; i < 300; i++)
		//{
		//	Engine.Simulate(DeltaTime);

		//	//UE_LOGF(LogChaos, Warning, "EngineSpeed %.2f rad/sec (%.1f RPM)", Engine.GetEngineSpeed(), Engine.GetEngineRPM());
		//}

		//EXPECT_GT(Engine.GetEngineRPM(), Engine.Setup().EngineIdleRPM);

		////UE_LOGF(LogChaos, Warning, "");
		////UE_LOGF(LogChaos, Warning, "No Throttle");

		//Engine.SetThrottle(0.0f);

		//// engine idle - no throttle
		//for (int i = 0; i < 300; i++)
		//{
		//	Engine.Simulate(DeltaTime);

		//	//UE_LOGF(LogChaos, Warning, "EngineSpeed %.2f rad/sec (%.1f RPM)", Engine.GetEngineSpeed(), Engine.GetEngineRPM());
		//}

		//EXPECT_LT(Engine.GetEngineRPM() - Engine.Setup().EngineIdleRPM, TOLERANCE);

	}

	// Wheel
	void SimulateBraking(FSimpleWheelSim& Wheel
		, const float Gravity
		, float VehicleSpeed
		, float DeltaTime
		, float& StoppingDistanceOut
		, float& SimulationTimeOut
		, bool bLoggingEnabled = false
		)
	{
		StoppingDistanceOut = 0.f;
		SimulationTimeOut = 0.f;
		
		float MaxSimTime = 15.0f;
		float VehicleMass = 1300.f;
		float VehicleMassPerWheel = 1300.f / 4.f;

		Wheel.SetWheelLoadForce(VehicleMassPerWheel * Gravity);
		Wheel.SetMassPerWheel(VehicleMassPerWheel);

		// Road speed
		FVector Velocity = FVector(VehicleSpeed, 0.f, 0.f);

		// wheel rolling speed matches road speed
		Wheel.SetMatchingSpeed(Velocity.X);

		if (bLoggingEnabled)
		{
			UE_LOGF(LogChaos, Warning, "--------------------START---------------------");
		}

		while (SimulationTimeOut < MaxSimTime)
		{
			// rolling speed matches road speed
			Wheel.SetVehicleGroundSpeed(Velocity);
			Wheel.Simulate(DeltaTime);

			// deceleration from brake, F = m * a, a = F / m, v = dt * F / m
			Velocity += DeltaTime * Wheel.GetForceFromFriction() / VehicleMassPerWheel;
			StoppingDistanceOut += Velocity.X * DeltaTime;

			if (bLoggingEnabled)
			{
				UE_LOGF(LogChaos, Warning, "Wheel.GetForceFromFriction() %ls", *Wheel.GetForceFromFriction().ToString());
			}

			if (FMath::Abs(Velocity.X) < 0.05f)
			{
				Velocity.X = 0.f;
				break; // break out early if already stopped
			}

			SimulationTimeOut += DeltaTime;
		}

		if (bLoggingEnabled)
		{
			UE_LOGF(LogChaos, Warning, "---------------------END----------------------");
		}


	}

	void SimulateAccelerating(FSimpleWheelSim& Wheel
		, const float Gravity
		, float FinalVehicleSpeed
		, float DeltaTime
		, float& DistanceTravelledOut
		, float& SimulationTimeOut
		)
	{
		DistanceTravelledOut = 0.f;
		SimulationTimeOut = 0.f;

		float MaxSimTime = 15.0f;
		float VehicleMass = 1300.f;
		float VehicleMassPerWheel = VehicleMass / 4.f;

		Wheel.SetWheelLoadForce(VehicleMassPerWheel * Gravity);
		Wheel.SetMassPerWheel(VehicleMassPerWheel);

		// Road speed
		FVector Velocity = FVector(0.f, 0.f, 0.f);

		// start from stationary
		Wheel.SetMatchingSpeed(Velocity.X);

		while (SimulationTimeOut < MaxSimTime)
		{
			Wheel.SetVehicleGroundSpeed(Velocity);
			Wheel.Simulate(DeltaTime);

			Velocity += DeltaTime * Wheel.GetForceFromFriction() / VehicleMassPerWheel;
			DistanceTravelledOut += Velocity.X * DeltaTime;

			SimulationTimeOut += DeltaTime;

			if (FMath::Abs(Velocity.X) >= FinalVehicleSpeed)
			{
				break; // time is up
			}

		}
	}

	GTEST_TEST(AllTraits, VehicleTest_WheelBrakingLongitudinalSlip)
	{
		FSimpleWheelConfig Setup;
		Setup.ABSEnabled = false;
		Setup.TractionControlEnabled = false;
		Setup.BrakeEnabled = true;
		Setup.EngineEnabled = true;
		Setup.WheelRadius = 0.3f;
		Setup.FrictionMultiplier = 1.0f;
		Setup.CorneringStiffness = 1000.0f;
		Setup.SideSlipModifier = 0.7f;

		FSimpleWheelSim Wheel(&Setup);

		// Google braking distance at 30mph says 14m (not interested in the thinking distance part)
		// So using a range 10-20 to ensure we are in the correct ballpark.
		// If specified more accurately in the test, then modifying the code would break the test all the time.

		// units meters
		float Gravity = 9.8f;
		float StoppingDistanceTolerance = 0.5f;
		float DeltaTime = 1.f / 30.f;
		float StoppingDistanceA = 0.f;
		float SimulationTime = 0.0f;
		Wheel.SetSurfaceFriction(RealWorldConsts::DryRoadFriction());

		// reasonably ideal stopping distance - traveling forwards
		Wheel.SetBrakeTorque(650);
		SimulateBraking(Wheel, Gravity, MPHToMS(30.f), DeltaTime, StoppingDistanceA, SimulationTime);
		EXPECT_GT(StoppingDistanceA, 10.f);
		EXPECT_LT(StoppingDistanceA, 20.f);

		// traveling backwards stops just the same
		float StoppingDistanceReverseDir = 0.f;
		Wheel.SetBrakeTorque(650);
		SimulateBraking(Wheel, Gravity, MPHToMS(-30.f), DeltaTime, StoppingDistanceReverseDir, SimulationTime);
		EXPECT_GT(StoppingDistanceReverseDir, -20.f);
		EXPECT_LT(StoppingDistanceReverseDir, -10.f);
		EXPECT_LT(StoppingDistanceA - FMath::Abs(StoppingDistanceReverseDir), StoppingDistanceTolerance);

		// Changing to units of Cm should yield the same results
		float MToCm = 100.0f;
		float StoppingDistanceCm = 0.f;
		Wheel.SetBrakeTorque(650 * MToCm * MToCm);
		Wheel.SetWheelRadius(0.3f * MToCm);
		SimulateBraking(Wheel, Gravity * MToCm, MPHToCmS(30.f), DeltaTime, StoppingDistanceCm, SimulationTime);
		EXPECT_NEAR(StoppingDistanceCm, StoppingDistanceA * MToCm, 1.0f);

		// Similar results with different delta time
		float StoppingDistanceDiffDT = 0.f;
		Wheel.SetWheelRadius(0.3f);
		SimulateBraking(Wheel, Gravity, MPHToMS(30.f), DeltaTime * 0.25f, StoppingDistanceDiffDT, SimulationTime);
		EXPECT_LT(StoppingDistanceA - StoppingDistanceDiffDT, StoppingDistanceTolerance);

		// barely touching the brake - going to take longer to stop
		float StoppingDistanceLightBraking = 0.f;
		Wheel.SetBrakeTorque(150);
		SimulateBraking(Wheel, Gravity, MPHToMS(30.f), DeltaTime, StoppingDistanceLightBraking, SimulationTime);
		EXPECT_GT(StoppingDistanceLightBraking, StoppingDistanceA);

		// locking the wheels / too much brake torque -> dynamic friction rather than static friction -> going to take longer to stop
		float StoppingDistanceTooHeavyBreaking = 0.f;
		Wheel.SetBrakeTorque(5000);
		SimulateBraking(Wheel, Gravity, MPHToMS(30.f), DeltaTime, StoppingDistanceTooHeavyBreaking, SimulationTime);
		EXPECT_GT(StoppingDistanceTooHeavyBreaking, StoppingDistanceA);

		// Would have locked the wheels but ABS prevents skidding
		Wheel.ABSEnabled = true;
		float StoppingDistanceTooHeavyBreakingABS = 0.f;
		Wheel.SetBrakeTorque(5000);
		SimulateBraking(Wheel, Gravity, MPHToMS(30.f), DeltaTime, StoppingDistanceTooHeavyBreakingABS, SimulationTime);
		EXPECT_LT(StoppingDistanceTooHeavyBreakingABS, StoppingDistanceTooHeavyBreaking);
		Wheel.ABSEnabled = false;

		// lower initial speed - stops more quickly
		float StoppingDistanceLowerSpeed = 0.f;
		Wheel.SetBrakeTorque(650);
		SimulateBraking(Wheel, Gravity, MPHToMS(20.f), DeltaTime, StoppingDistanceLowerSpeed, SimulationTime);
		EXPECT_LT(StoppingDistanceLowerSpeed, StoppingDistanceA);

		// higher initial speed - stops more slowly
		float StoppingDistanceHigherSpeed = 0.f;
		Wheel.SetBrakeTorque(650);
		SimulateBraking(Wheel, Gravity, MPHToMS(60.f), DeltaTime, StoppingDistanceHigherSpeed, SimulationTime);
		EXPECT_GT(StoppingDistanceHigherSpeed, StoppingDistanceA);

		// slippy surface - stops more slowly
		float StoppingDistanceLowFriction = 0.f;
		Wheel.SetSurfaceFriction(0.3f);
		Wheel.SetBrakeTorque(650);
		SimulateBraking(Wheel, Gravity, MPHToMS(30.f), DeltaTime, StoppingDistanceLowFriction, SimulationTime);
		EXPECT_GT(StoppingDistanceLowFriction, StoppingDistanceA);
	}

	GTEST_TEST(AllTraits, VehicleTest_WheelAcceleratingLongitudinalSlip)
	{
		FSimpleWheelConfig Setup;
		Setup.ABSEnabled = false;
		Setup.TractionControlEnabled = false;
		Setup.BrakeEnabled = true;
		Setup.EngineEnabled = true;
		Setup.WheelRadius = 0.3f;
		Setup.FrictionMultiplier = 1.0f;
		Setup.CorneringStiffness = 1000.0f;
		Setup.SideSlipModifier = 0.7f;

		FSimpleWheelSim Wheel(&Setup);

		// There could be one frame extra computation on the acceleration since the last frame of brake is not using the full 
		// amount of torque, it's clearing the last remaining velocity without pushing the vehicle back in the opposite direction
		// Hence a slightly larger tolerance for the result
		float AccelerationResultsTolerance = 1.0f; // meters

		// units meters
		float Gravity = 9.8f;
		float DeltaTime = 1.f / 30.f;

		float StoppingDistanceA = 0.f;
		float SimulationTimeBrake = 0.0f;
		Wheel.SetSurfaceFriction(RealWorldConsts::DryRoadFriction());

		// How far & what time does it take to stop from 30MPH to rest
		Wheel.SetBrakeTorque(650);
		SimulateBraking(Wheel, Gravity, MPHToMS(30.0f), DeltaTime, StoppingDistanceA, SimulationTimeBrake);

		// How far and what time does it take to accelerate from rest to 30MPH
		float SimulationTimeAccel = 0.0f;
		float DrivingDistanceA = 0.f;
		Wheel.SetDriveTorque(650);
		SimulateAccelerating(Wheel, Gravity, MPHToMS(30.0f), DeltaTime, DrivingDistanceA, SimulationTimeAccel);

		// 0-30 MPH and 30-0 MPH should be the same if there's no slipping and accel torque was same as the brake torque run
		EXPECT_LT(DrivingDistanceA - StoppingDistanceA, AccelerationResultsTolerance);
		EXPECT_LT(SimulationTimeAccel - SimulationTimeBrake, AccelerationResultsTolerance);

		// same range as braking from 30MPH
		EXPECT_GT(DrivingDistanceA, 10.f);
		EXPECT_LT(DrivingDistanceA, 20.f);

		// Unreal units cm - Note for the same results the radius needs to remain at 0.3m and not also be scaled to 30(cm)
		float SimulationTimeAccelCM = 0.0f;
		float MToCm = 100.0f;
		float DrivingDistanceCM = 0.f;
		Wheel.SetDriveTorque(650.0f * MToCm * MToCm);
		Wheel.SetWheelRadius(0.3f * MToCm);
		SimulateAccelerating(Wheel, Gravity * MToCm, MPHToCmS(30.0f), DeltaTime, DrivingDistanceCM, SimulationTimeAccelCM);
		EXPECT_GT(DrivingDistanceCM, 10.f * MToCm);
		EXPECT_LT(DrivingDistanceCM, 20.f * MToCm);
		EXPECT_NEAR(DrivingDistanceCM, DrivingDistanceA * MToCm, AccelerationResultsTolerance);

		float SimulationTimeAccelSpin = 0.0f;
		float DrivingDistanceWheelspin = 0.f;
		Wheel.SetWheelRadius(0.3f);
		Wheel.SetDriveTorque(5000); // definitely cause wheel spin
		SimulateAccelerating(Wheel, Gravity, 30.0f, DeltaTime, DrivingDistanceWheelspin, SimulationTimeAccelSpin);

		// drives further to reach the same speed
		EXPECT_GT(DrivingDistanceWheelspin, DrivingDistanceA);

		// takes longer to reach the same speed
		EXPECT_GT(SimulationTimeAccelSpin, SimulationTimeAccel);

		// Enable traction control should be better than both of the above
		float SimulationTimeAccelTC = 0.0f;
		float DrivingDistanceTC = 0.f;
		Wheel.TractionControlEnabled = true;
		Wheel.SetDriveTorque(5000); // definitely cause wheel spin
		SimulateAccelerating(Wheel, Gravity, MPHToMS(30.0f), DeltaTime, DrivingDistanceTC, SimulationTimeAccelTC);

		// reaches target speed in a shorter distance
		EXPECT_LT(DrivingDistanceTC, DrivingDistanceWheelspin);

		// reaches speed quicker with TC on when wheel would be slipping from drive torque
		EXPECT_LT(SimulationTimeAccelTC, SimulationTimeAccelSpin);

	}

	GTEST_TEST(AllTraits, DISABLED_VehicleTest_WheelLateralSlip)
	{
		FSimpleWheelConfig Setup;
		FSimpleWheelSim Wheel(&Setup);
	}

	GTEST_TEST(AllTraits, VehicleTest_WheelRolling)
	{
		FSimpleWheelConfig Setup;
		FSimpleWheelSim Wheel(&Setup);

		float DeltaTime = 1.f / 30.f;
		float MaxSimTime = 10.0f;
		float Tolerance = 0.1f; // wheel friction losses slow wheel speed

		//------------------------------------------------------------------
		// Car is moving FORWARDS - with AMPLE friction we would expect an initially 
		// static rolling wheel to speed up and match the vehicle speed
		FVector VehicleGroundSpeed(10.0f, 0.f, 0.f); // X is forwards
		Wheel.SetVehicleGroundSpeed(VehicleGroundSpeed);
		Wheel.SetSurfaceFriction(1.0f);	// Some wheel/ground friction
		Wheel.SetWheelLoadForce(250.f); // wheel pressed into the ground, to give it grip
		Wheel.Omega = 0.f;

		// initially wheel is static
		EXPECT_LT(Wheel.GetAngularVelocity(), SMALL_NUMBER);

		// after some time, the wheel picks up speed to match the vehicle speed
		float SimulatedTime = 0.f;
		while (SimulatedTime < MaxSimTime)
		{
			Wheel.Simulate(DeltaTime);
			SimulatedTime += DeltaTime;
		}

		// there's enough grip to cause the wheel to spin and match the vehivle speed
		float WheelGroundSpeed = Wheel.GetAngularVelocity() * Wheel.GetEffectiveRadius();
		EXPECT_LT(VehicleGroundSpeed.X - WheelGroundSpeed, Tolerance);
		EXPECT_LT(VehicleGroundSpeed.X - WheelGroundSpeed, Tolerance);
		EXPECT_GT(Wheel.GetAngularVelocity(), 0.f); // +ve spin on it

		//------------------------------------------------------------------
		// Car is moving BACKWARDS - with AMPLE friction we would expect an initially 
		// static rolling wheel to speed up and match the vehicle speed
		VehicleGroundSpeed.Set(-10.0f, 0.f, 0.f); // X is -ve not travelling backwards
		Wheel.SetVehicleGroundSpeed(VehicleGroundSpeed);
		Wheel.SetSurfaceFriction(1.0f);	// Some wheel/ground friction
		Wheel.SetWheelLoadForce(250.f); // wheel pressed into the ground, to give it grip
		Wheel.Omega = 0.f;

		// initially wheel is static
		EXPECT_LT(Wheel.GetAngularVelocity(), SMALL_NUMBER);

		// after some time, the wheel picks up speed to match the vehicle speed
		SimulatedTime = 0.f;
		while (SimulatedTime < MaxSimTime)
		{
			Wheel.Simulate(DeltaTime);
			SimulatedTime += DeltaTime;
		}

		// there's enough grip to cause the wheel to spin and match the vehicle speed
		WheelGroundSpeed = Wheel.GetAngularVelocity() * Wheel.GetEffectiveRadius();
		EXPECT_LT(VehicleGroundSpeed.X - WheelGroundSpeed, Tolerance);
		EXPECT_LT(VehicleGroundSpeed.X - Wheel.GetWheelGroundSpeed(), Tolerance);
		EXPECT_LT(Wheel.GetAngularVelocity(), 0.f); // -ve spin on it

		//------------------------------------------------------------------
		// Car is moving FROWARDS - with NO friction we would expect an initially 
		// static wheel to NOT speed up to match the vehicle speed
		Wheel.SetVehicleGroundSpeed(VehicleGroundSpeed);
		Wheel.SetSurfaceFriction(0.0f);	// No wheel/ground friction
		Wheel.SetWheelLoadForce(250.f); // wheel pressed into the ground, to give it grip
		Wheel.Omega = 0.f;

		// initially wheel is static
		EXPECT_LT(Wheel.GetAngularVelocity(), SMALL_NUMBER);

		// after some time, the wheel picks up speed to match the vehicle speed
		SimulatedTime = 0.f;
		while (SimulatedTime < MaxSimTime)
		{
			Wheel.Simulate(DeltaTime);
			SimulatedTime += DeltaTime;
		}

		WheelGroundSpeed = Wheel.GetAngularVelocity() * Wheel.GetEffectiveRadius();

		// wheel is just sliding there's no friction to make it spin
		EXPECT_LT(WheelGroundSpeed, SMALL_NUMBER);

	}

	// Suspension

	float SumSprungMasses(TArray<float>& SprungMasses)
	{
		float Sum = 0.f;
		for (int I = 0; I < SprungMasses.Num(); I++)
		{
			Sum += SprungMasses[I];
		}
		return Sum;
	}

	GTEST_TEST(AllTraits, VehicleTest_SuspensionSprungMassesTwoWheels)
	{
		float TotalMass = 1000.f;
		float Tolerance = 0.01f;

		{
			// simple 1 Wheels unstable as offset from COM
			TArray<FVector> MassSpringPositions;
			TArray<float> OutSprungMasses;

			MassSpringPositions.Add(FVector(200, 0, 0));

			FSuspensionUtility::ComputeSprungMasses(MassSpringPositions, TotalMass, OutSprungMasses);

			EXPECT_EQ(MassSpringPositions.Num(), OutSprungMasses.Num());
			EXPECT_LT(FMath::Abs(OutSprungMasses[0] - 1000.f), Tolerance);
			EXPECT_LT(FMath::Abs(SumSprungMasses(OutSprungMasses) - TotalMass), Tolerance);
		}

		{
			// simple 2 Wheels equally spaced around COM
			TArray<FVector> MassSpringPositions;
			TArray<float> OutSprungMasses;

			MassSpringPositions.Add(FVector(200, 0, 0));
			MassSpringPositions.Add(FVector(-200, 0, 0));

			FSuspensionUtility::ComputeSprungMasses(MassSpringPositions, TotalMass, OutSprungMasses);

			EXPECT_EQ(MassSpringPositions.Num(), OutSprungMasses.Num());
			EXPECT_LT(FMath::Abs(OutSprungMasses[0] - 500.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[1] - 500.f), Tolerance);
			EXPECT_LT(FMath::Abs(SumSprungMasses(OutSprungMasses) - TotalMass), Tolerance);
		}

		{
			// simple 2 Wheels equally spaced around COM
			TArray<FVector> MassSpringPositions;
			TArray<float> OutSprungMasses;

			MassSpringPositions.Add(FVector(200, 0, 50));
			MassSpringPositions.Add(FVector(-200, 0, -50));

			FSuspensionUtility::ComputeSprungMasses(MassSpringPositions, TotalMass, OutSprungMasses);

			EXPECT_EQ(MassSpringPositions.Num(), OutSprungMasses.Num());
			EXPECT_LT(FMath::Abs(OutSprungMasses[0] - 500.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[1] - 500.f), Tolerance);
			EXPECT_LT(FMath::Abs(SumSprungMasses(OutSprungMasses) - TotalMass), Tolerance);
		}

		{
			// simple 2 Wheels equally spaced around COM
			TArray<FVector> MassSpringPositions;
			TArray<float> OutSprungMasses;

			MassSpringPositions.Add(FVector(200, 0, 0));
			MassSpringPositions.Add(FVector(0, 0, 0));

			FSuspensionUtility::ComputeSprungMasses(MassSpringPositions, TotalMass, OutSprungMasses);

			EXPECT_EQ(MassSpringPositions.Num(), OutSprungMasses.Num());
			EXPECT_LT(FMath::Abs(OutSprungMasses[0] - 1000.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[1] - 0.f), Tolerance);
			EXPECT_LT(FMath::Abs(SumSprungMasses(OutSprungMasses) - TotalMass), Tolerance);
		}

		{
			// simple 2 Wheels equally spaced around COM
			TArray<FVector> MassSpringPositions;
			TArray<float> OutSprungMasses;

			MassSpringPositions.Add(FVector(200, 0, 0));
			MassSpringPositions.Add(FVector(-100, 0, 0));

			FSuspensionUtility::ComputeSprungMasses(MassSpringPositions, TotalMass, OutSprungMasses);

			EXPECT_EQ(MassSpringPositions.Num(), OutSprungMasses.Num());
			EXPECT_LT(FMath::Abs(OutSprungMasses[0] - TotalMass * 2.f / 3.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[1] - TotalMass * 1.f / 3.f), Tolerance);
			EXPECT_LT(FMath::Abs(SumSprungMasses(OutSprungMasses) - TotalMass), Tolerance);
		}
	}

	GTEST_TEST(AllTraits, VehicleTest_SuspensionSprungMassesThreeWheels)
	{
		float TotalMass = 1000.f;
		float Tolerance = 0.01f;

		{
			// simple 3 Wheels equally spaced around COM
			TArray<FVector> MassSpringPositions;
			TArray<float> OutSprungMasses;

			MassSpringPositions.Add(FVector(200, 0, 0));
			MassSpringPositions.Add(FVector(-200, -100, 0));
			MassSpringPositions.Add(FVector(-200, 100, 0));

			FSuspensionUtility::ComputeSprungMasses(MassSpringPositions, TotalMass, OutSprungMasses);

			EXPECT_EQ(MassSpringPositions.Num(), OutSprungMasses.Num());
			EXPECT_LT(FMath::Abs(OutSprungMasses[0] - 500), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[1] - 250), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[2] - 250), Tolerance);
			EXPECT_LT(FMath::Abs(SumSprungMasses(OutSprungMasses) - TotalMass), Tolerance);
		}

	}

	GTEST_TEST(AllTraits, VehicleTest_SuspensionSprungMassesFourWheels)
	{
		float TotalMass = 1000.f;
		float Tolerance = 0.1f;

		{
			// simple 4 Wheels equally spaced around COM
			TArray<FVector> MassSpringPositions;
			TArray<float> OutSprungMasses;

			MassSpringPositions.Add(FVector(200, 0, 0));
			MassSpringPositions.Add(FVector(-200, 0, 0));
			MassSpringPositions.Add(FVector(200, -100, 0));
			MassSpringPositions.Add(FVector(-200, 100, 0));

			FSuspensionUtility::ComputeSprungMasses(MassSpringPositions, TotalMass, OutSprungMasses);

			EXPECT_EQ(MassSpringPositions.Num(), OutSprungMasses.Num());

			EXPECT_LT(FMath::Abs(OutSprungMasses[0] - 250.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[1] - 250.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[2] - 250.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[3] - 250.f), Tolerance);
			EXPECT_LT(FMath::Abs(SumSprungMasses(OutSprungMasses) - TotalMass), Tolerance);
		}

		{
			// simple 4 Wheels all weight on rear COM
			TArray<FVector> MassSpringPositions;
			TArray<float> OutSprungMasses;

			MassSpringPositions.Add(FVector(0, 0, 0));
			MassSpringPositions.Add(FVector(-200, 0, 0));
			MassSpringPositions.Add(FVector(0, -100, 0));
			MassSpringPositions.Add(FVector(-200, 100, 0));

			FSuspensionUtility::ComputeSprungMasses(MassSpringPositions, TotalMass, OutSprungMasses);

			EXPECT_EQ(MassSpringPositions.Num(), OutSprungMasses.Num());

			EXPECT_LT(FMath::Abs(OutSprungMasses[0] - 500.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[1] - 0.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[2] - 250.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[3] - 250.f), Tolerance);
			EXPECT_LT(FMath::Abs(SumSprungMasses(OutSprungMasses) - TotalMass), Tolerance);
		}

		{
			// 4 Wheels all weight on rear COM
			TArray<FVector> MassSpringPositions;
			TArray<float> OutSprungMasses;

			MassSpringPositions.Add(FVector(100, 0, 0));
			MassSpringPositions.Add(FVector(-200, 0, 0));
			MassSpringPositions.Add(FVector(100, -100, 0));
			MassSpringPositions.Add(FVector(-200, 100, 0));

			FSuspensionUtility::ComputeSprungMasses(MassSpringPositions, TotalMass, OutSprungMasses);

			EXPECT_EQ(MassSpringPositions.Num(), OutSprungMasses.Num());

			//for (int i = 0; i < 4; i++)
			//{
			//	UE_LOGF(LogChaos, Warning, "SM = %f", OutSprungMasses[i]);
			//}
			EXPECT_LT(FMath::Abs(OutSprungMasses[0] - TotalMass * 1.f / 3.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[1] - TotalMass * 1.f / 6.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[2] - TotalMass * 1.f / 4.f), Tolerance);
			EXPECT_LT(FMath::Abs(OutSprungMasses[3] - TotalMass * 1.f / 4.f), Tolerance);
			EXPECT_LT(FMath::Abs(SumSprungMasses(OutSprungMasses) - TotalMass), Tolerance);
		}
	}

	float PlaneZPos = 1.0f;
	bool RayCastPlane(FVec3& RayStart, FVec3& Direction, float Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal)
	{
		TPlane<FReal, 3> Plane(FVec3(1), FVec3(0, 0, PlaneZPos));
		int32 FaceIndex;

		return Plane.Raycast(RayStart, Direction, Length, 0, OutTime, OutPosition, OutNormal, FaceIndex);
	}

	void AddForceAtPosition(FPBDRigidsEvolutionGBF& Evolution, TPBDRigidParticleHandle<FReal, 3>* Rigid, const FVector& InForce, const FVector& InPosition)
	{
		const Chaos::FVec3 WorldCOM = FParticleUtilitiesGT::GetCoMWorldPosition(Rigid);

		Evolution.SetParticleObjectState(Rigid, EObjectStateType::Dynamic);

		const Chaos::FVec3 WorldTorque = Chaos::FVec3::CrossProduct(InPosition - WorldCOM, InForce);
		Rigid->AddForce(InForce);
		Rigid->AddTorque(WorldTorque);

	}

	FVector WorldVelocityAtPoint(TPBDRigidParticleHandle<FReal, 3>* Rigid, const FVector& InPoint)
	{
		check(Rigid);
		const Chaos::FVec3 COM = Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(Rigid);
		const Chaos::FVec3 Diff = InPoint - COM;
		const Chaos::FVec3 V = Rigid->GetV();
		const Chaos::FVec3 W = Rigid->GetW();
		return V - Chaos::FVec3::CrossProduct(Diff, W);
	}

	// #todo: break out vehicle simulation setup so it can be used across number of tests
	GTEST_TEST(AllEvolutions, VehicleTest_SuspensionSpringLoad)
	{
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs Particles(UniqueIndices);
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;
		FPBDRigidsEvolutionGBF Evolution(Particles, PhysicalMaterials);

		float BodyMass = 1000.0f;
		float Gravity = FMath::Abs(Evolution.GetGravityForces().GetAcceleration(0).Z);

		FSimpleSuspensionConfig Setup;
		Setup.MaxLength = 20.0f;
		Setup.SpringRate = (2.0f * BodyMass * Gravity / 4.0f) / Setup.MaxLength;
		Setup.SpringPreload = 0.f;
		Setup.RaycastSafetyMargin = 0.f;
		Setup.SuspensionSmoothing = 0;
		Setup.ReboundDamping = 0.f;		// calculated later
		Setup.CompressionDamping = 0.f; // calculated later

		TArray<FSimpleSuspensionSim> Suspensions;
		for (int SpringIdx = 0; SpringIdx < 4; SpringIdx++)
		{
			FSimpleSuspensionSim Suspension(&Setup);
			Suspensions.Add(Suspension);
		}

		float HalfLength = 100.f;
		float HalfWidth = 50.f;
		TArray<FVector> LocalSpringPositions;
		LocalSpringPositions.Add(FVector( HalfLength, -HalfWidth, 0.f));
		LocalSpringPositions.Add(FVector( HalfLength,  HalfWidth, 0.f));
		LocalSpringPositions.Add(FVector(-HalfLength, -HalfWidth, 0.f));
		LocalSpringPositions.Add(FVector(-HalfLength,  HalfWidth, 0.f));

		for (int SpringIdx = 0; SpringIdx < 4; SpringIdx++)
		{
			Suspensions[SpringIdx].SetLocalRestingPosition(LocalSpringPositions[SpringIdx]);
		}

		//////////////////////////////////////////////////////////////////////////
		TArray<float> OutSprungMasses;
		FSuspensionUtility::ComputeSprungMasses(LocalSpringPositions, BodyMass, OutSprungMasses);

		TArray<FSuspensionTrace> Traces;
		Traces.SetNum(4);

		for (int SpringIdx = 0; SpringIdx < 4; SpringIdx++)
		{
			float NaturalFrequency = FSuspensionUtility::ComputeNaturalFrequency(Setup.SpringRate, OutSprungMasses[SpringIdx]);
			float Damping = FSuspensionUtility::ComputeCriticalDamping(Setup.SpringRate, OutSprungMasses[SpringIdx]);
			Setup.ReboundDamping = Damping;
			Setup.CompressionDamping = Damping;
		}

		float WheelRadius = 2.0f;
		//const float SuspendedDisplacement = SprungMass * Gravity / Setup.SpringRate;
		//UE_LOGF(LogChaos, Warning, "SuspendedDisplacement %.1f cm", SuspendedDisplacement);

		//////////////////////////////////////////////////////////////////////////

		auto Dynamic = Evolution.CreateDynamicParticles(1)[0];

		TUniquePtr<FChaosPhysicsMaterial> PhysicsMaterial = MakeUnique<FChaosPhysicsMaterial>();
		PhysicsMaterial->SleepCounterThreshold = 2;

		Chaos::FImplicitObjectPtr Box(new Chaos::FSphere(FVec3(0, 0, 0), 50));
		Dynamic->SetGeometry(Box);

		Evolution.SetPhysicsMaterial(Dynamic, MakeSerializable(PhysicsMaterial));

		Dynamic->SetX(FVec3(10, 10, 20));
		Dynamic->M() = BodyMass;
		Dynamic->InvM() = 1.0f / BodyMass;
		Dynamic->I() = TVec3<FRealSingle>(100000.0f);
		Dynamic->InvI() = TVec3<FRealSingle>(1.0f / 100000.0f);

		Evolution.EnableParticle(Dynamic);

		float AccumulatedTime = 0.f;
		const FReal Dt = 1 / 30.f;
		for (int i = 0; i < 500; ++i)
		{
			// latest body transform
			const FTransform BodyTM(Dynamic->GetR(), Dynamic->GetX());

			for (int SpringIdx = 0; SpringIdx < 4; SpringIdx++)
			{
				FSimpleSuspensionSim& Suspension = Suspensions[SpringIdx];
				FSuspensionTrace& Trace = Traces[SpringIdx];
				Suspension.UpdateWorldRaycastLocation(BodyTM, WheelRadius, Trace);

				// raycast
				FVec3 Start = Trace.Start;
				FVec3 Dir = Trace.TraceDir();
				FReal CurrentLength = Suspension.Setup().MaxLength;

				FVec3 Position(0,0,0);
				FVec3 Normal(0,0,0);
				bool bHit = RayCastPlane(Start, Dir, Trace.Length(), CurrentLength, Position, Normal);

				Suspension.SetSuspensionLength(CurrentLength, WheelRadius);
				Suspension.SetLocalVelocityFromWorld(BodyTM, WorldVelocityAtPoint(Dynamic, Trace.Start));
				Suspension.Simulate(Dt); // ComputeSuspensionForces

				if (bHit)
				{
					FVector SusForce = Suspension.GetSuspensionForceVector(BodyTM);
					AddForceAtPosition(Evolution, Dynamic, SusForce, Start);
				}
			}

			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
			AccumulatedTime += Dt;
		}

		float Tolerance = 0.5f; // half cm
		float ExpectedRestingPosition = (10.f + PlaneZPos + WheelRadius);
		EXPECT_LT(Dynamic->GetX().Z - ExpectedRestingPosition, Tolerance);
	}

	GTEST_TEST(AllTraits, VehicleTest_WheelAcceleratingLongitudinalSlip_VaryingDelta)
	{
		FSimpleWheelConfig Setup;
		Setup.ABSEnabled = false;
		Setup.TractionControlEnabled = false;
		Setup.BrakeEnabled = true;
		Setup.EngineEnabled = true;
		Setup.WheelRadius = 0.3f;
		Setup.FrictionMultiplier = 1.0f;
		Setup.CorneringStiffness = 1000.0f;
		Setup.SideSlipModifier = 0.7f;

		FSimpleWheelSim Wheel(&Setup);

		// units meters
		float Gravity = 9.8f;
		float VehicleMassPerWheel = 100.f;
		float ResultsTolerance = 0.01f;

		Wheel.SetSurfaceFriction(RealWorldConsts::DryRoadFriction());
		Wheel.SetWheelLoadForce(VehicleMassPerWheel * Gravity);
		Wheel.SetMassPerWheel(VehicleMassPerWheel);

		// Road speed
		FVector Velocity = FVector(10.f, 0.f, 0.f);

		// start from stationary
		Wheel.SetMatchingSpeed(Velocity.X);
		Wheel.SetVehicleGroundSpeed(Velocity);
		Wheel.SetDriveTorque(100.0f);

		float DeltaTime = 1.f / 30.f;
		Wheel.Simulate(DeltaTime);
		FVector ForceGenerated30FPS = Wheel.GetForceFromFriction();

		Wheel.Simulate(DeltaTime);
		FVector ForceGenerated30FPS_2 = Wheel.GetForceFromFriction();

		DeltaTime = 1.f / 60.f;
		Wheel.Simulate(DeltaTime);

		FVector ForceGenerated60FPS = Wheel.GetForceFromFriction();

		DeltaTime = 1.f / 50.f;
		Wheel.Simulate(DeltaTime);

		FVector ForceGenerated50FPS = Wheel.GetForceFromFriction();

		EXPECT_NEAR(ForceGenerated30FPS.X, ForceGenerated30FPS_2.X, ResultsTolerance);
		EXPECT_NEAR(ForceGenerated30FPS.X, ForceGenerated60FPS.X, ResultsTolerance);
		EXPECT_NEAR(ForceGenerated30FPS.X, ForceGenerated60FPS.X, ResultsTolerance);


		Wheel.SetMatchingSpeed(Velocity.X);
		Wheel.SetVehicleGroundSpeed(Velocity);
		Wheel.SetDriveTorque(100.0f);
	}

	GTEST_TEST(AllTraits, VehicleTest_Suspension_VaryingDelta)
	{
		FSimpleSuspensionConfig Setup;
		Setup.MaxLength = 20.0f;
		Setup.DampingRatio = 0.5f;
		Setup.SpringPreload = 100.0f;
		Setup.SpringRate = 100.0f;
		Setup.SuspensionAxis = FVector(0.f, 0.f, -1.f);
		Setup.RestingForce = 50.0f;

		FSimpleSuspensionSim Suspension(&Setup);

		// units meters
		float Gravity = 9.8f;
		float VehicleMassPerWheel = 100.f;
		float WheelRadius = 32.0f;
		float CurrentLength = 5.0f;

		float DeltaTime = 1.0f / 30.0f;
		float ResultsTolerance = 0.01f;

		Suspension.SetSuspensionLength(CurrentLength, WheelRadius);
		Suspension.SetLocalVelocity(FVector(0.f,0.f,-8.f));

		// first one has to adjust for sudden change in length
		Suspension.Simulate(DeltaTime);

		Suspension.Simulate(DeltaTime);
		float Force30FPS = Suspension.GetSuspensionForce();

		Suspension.Simulate(DeltaTime);
		float Force30FPS_2 = Suspension.GetSuspensionForce();

		DeltaTime = 60.0f;
		Suspension.Simulate(DeltaTime);
		float Force60FPS = Suspension.GetSuspensionForce();

		DeltaTime = 50.0f; 
		Suspension.Simulate(DeltaTime);
		float Force50FPS = Suspension.GetSuspensionForce();

		EXPECT_NEAR(Force30FPS, Force30FPS_2, ResultsTolerance);
		EXPECT_NEAR(Force30FPS, Force60FPS, ResultsTolerance);
		EXPECT_NEAR(Force30FPS, Force50FPS, ResultsTolerance);

	}
}


