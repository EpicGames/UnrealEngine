// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#pragma warning(disable:4121)
#pragma warning(disable:4668)
#pragma warning(disable:4996)
#include "gtest/gtest.h"
#include "gmock/gmock.h"
#pragma warning(default:4121)
#pragma warning(default:4668)
#pragma warning(default:4996)


#include "CoreMinimal.h"
DEFINE_LOG_CATEGORY_STATIC(LogTest, Verbose, All);
