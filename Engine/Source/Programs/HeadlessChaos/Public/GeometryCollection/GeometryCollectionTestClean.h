// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCollectionTest.h"

namespace GeometryCollectionTest
{	
	void  TestDeleteCoincidentVertices();

	void  TestDeleteCoincidentVertices2();

	void  TestDeleteZeroAreaFaces();

	void  TestDeleteHiddenFaces();

	void  TestFillHoles();
}
