// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCollectionTest.h"

namespace GeometryCollectionTest
{	
	void  CheckClassTypes();

	void  CheckIncrementMask();

	void  Creation();

	void  Empty();

	void  AppendTransformHierarchy();

	void  ContiguousElementsTest();

	void  DeleteFromEnd();

	void  DeleteFromStart();

	void  DeleteFromMiddle();

	void  DeleteBranch();

	void  DeleteRootLeafMiddle();

	void  DeleteEverything();

	void  ParentTransformTest();

	void  ReindexMaterialsTest();

	void  AttributeTransferTest();

	void  AttributeDependencyTest();

	void IntListReindexOnDeletionTest();

	void IntListSelfDependencyTest();

	void AppendManagedArrayCollectionTest();

	void AppendTransformCollectionTest();

	void CollectionCycleTest();

}
