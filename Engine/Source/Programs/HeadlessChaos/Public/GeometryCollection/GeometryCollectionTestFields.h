// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCollectionTest.h"

namespace GeometryCollectionTest
{
	void Fields_NoiseSample();
	void Fields_RadialIntMask();
	void Fields_RadialFalloff();
	void Fields_PlaneFalloff();
	void Fields_UniformVector();
	void Fields_RaidalVector();
	void  Fields_SumVectorFullMult();
	void  Fields_SumVectorFullDiv();
	void  Fields_SumVectorFullAdd();
	void  Fields_SumVectorFullSub();
	void  Fields_SumVectorLeftSide();
	void  Fields_SumVectorRightSide();
	void  Fields_SumScalar();
	void  Fields_SumScalarRightSide();
	void  Fields_SumScalarLeftSide();
	void  Fields_Culling();
	void  Fields_SerializeAPI();
}
