// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosSolversModule.h"
#include "Chaos/ParticleHandle.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollectionProxyData.h"
#include "PhysicsProxy/PhysicsProxies.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "Chaos/Defines.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "PhysicsInterfaceTypesCore.h"

namespace GeometryCollectionTest
{
	using namespace Chaos;

	enum WrapperType
	{
		RigidBody,
		GeometryCollection
	};

	enum ESimplicialType
	{
		Chaos_Simplicial_Box,
		Chaos_Simplicial_Sphere,
		Chaos_Simplicial_GriddleBox,
		Chaos_Simplicial_Tetrahedron,
		Chaos_Simplicial_Imported_Sphere,
		Chaos_Simplicial_None
	};

	struct WrapperBase
	{
		WrapperType Type;
		WrapperBase(WrapperType TypeIn) : Type(TypeIn) {}
		virtual ~WrapperBase() {}
		template<class AS_T> AS_T* As() { return AS_T::StaticType() == Type ? static_cast<AS_T*>(this) : nullptr; }
		template<class AS_T> const AS_T* As() const { return AS_T::StaticType() == Type ? static_cast<const AS_T*>(this) : nullptr; }
	};

	struct FGeometryCollectionWrapper : public WrapperBase
	{
		FGeometryCollectionWrapper();
		//FGeometryCollectionWrapper(
		//	TSharedPtr<FGeometryCollection> RestCollectionIn,
		//	TSharedPtr<FGeometryDynamicCollection> DynamicCollectionIn,
		//	FGeometryCollectionPhysicsProxy* PhysObjectIn)
		//	: WrapperBase(WrapperType::GeometryCollection)
		//	, RestCollection(RestCollectionIn)
		//	, DynamicCollection(DynamicCollectionIn)
		//	, PhysObject(PhysObjectIn) {}
		static WrapperType StaticType() { return WrapperType::GeometryCollection; }
		TSharedPtr<FGeometryCollection> RestCollection;
		TSharedPtr<FGeometryDynamicCollection> DynamicCollection;
		FGeometryCollectionPhysicsProxy* PhysObject;
		FChaosUserData UserData;
	};

	struct RigidBodyWrapper : public WrapperBase
	{
		RigidBodyWrapper(
			TSharedPtr<Chaos::FChaosPhysicsMaterial> PhysicalMaterialIn,
			FPhysicsActorHandle ParticleIn)
			: WrapperBase(WrapperType::RigidBody)
			, PhysicalMaterial(PhysicalMaterialIn)
			, Particle(ParticleIn) {}
		static WrapperType StaticType() { return WrapperType::RigidBody; }
		TSharedPtr<Chaos::FChaosPhysicsMaterial> PhysicalMaterial;
		FPhysicsActorHandle Particle;
	};


	struct CreationParameters
	{
		FTransform RootTransform = FTransform::Identity;
		/**
		 * Implicit box uses Scale X, Y, Z for dimensions.
		 * Implicit sphere uses Scale X for radius.
		 */
		FVector3f InitialLinearVelocity = FVector3f::ZeroVector;
		EObjectStateTypeEnum DynamicState = EObjectStateTypeEnum::Chaos_Object_Dynamic;
		bool Simulating = true;
		FReal Mass = 1.0;
		bool bMassAsDensity = false;
		ECollisionTypeEnum CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		ESimplicialType SimplicialType = ESimplicialType::Chaos_Simplicial_Sphere;
		EImplicitTypeEnum ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Sphere;
		float CollisionParticleFraction = 1.0f;
		EInitialVelocityTypeEnum InitialVelocityType = EInitialVelocityTypeEnum::Chaos_Initial_Velocity_None;
		TArray<FTransform> NestedTransforms;
		bool EnableClustering = true;
		FTransform GeomTransform = FTransform::Identity;
		TSharedPtr<FGeometryCollection> RestCollection = nullptr;
		int32 MaxClusterLevel = 100;
		int32 MaxSimulatedLevel = 100;
		TArray<float> DamageThreshold = { 1000.0 };
		Chaos::FClusterCreationParameters::EConnectionMethod ClusterConnectionMethod = Chaos::FClusterCreationParameters::EConnectionMethod::PointImplicit;
		int32 CollisionGroup = 0;
		int32 MinLevelSetResolution = 5;
		int32 MaxLevelSetResolution = 10;
		int32 ClusterGroupIndex = 0;
	};


	enum GeometryType {
		GeometryCollectionWithSingleRigid,
		RigidFloor,
		GeometryCollectionWithSuppliedRestCollection
	};


	template <GeometryType>
	struct TNewSimulationObject
	{
		static WrapperBase* Init(const CreationParameters Params = CreationParameters());
	};

	template<> WrapperBase* TNewSimulationObject<GeometryType::GeometryCollectionWithSingleRigid>::Init(const CreationParameters);
	template<> WrapperBase* TNewSimulationObject<GeometryType::RigidFloor>::Init(const CreationParameters);
	template<> WrapperBase* TNewSimulationObject<GeometryType::GeometryCollectionWithSuppliedRestCollection>::Init(const CreationParameters);

	struct FrameworkParameters
	{
		FReal Dt = 1.0 / 60.0;
		FReal AsyncDt = -1.0;
		Chaos::EThreadingMode ThreadingMode = Chaos::EThreadingMode::SingleThread;
	};

	class FFramework
	{
	public:

		FFramework(FrameworkParameters Properties = FrameworkParameters());
		virtual ~FFramework();

		void AddSimulationObject(WrapperBase* Object);
		void RemoveSimulationObject(WrapperBase* Object);
		void Initialize();
		void Advance();
		FReal Dt;
		FChaosSolversModule* Module = FChaosSolversModule::GetModule();
		Chaos::FPBDRigidsSolver* Solver;
		TArray< WrapperBase* > PhysicsObjects;
	};

}