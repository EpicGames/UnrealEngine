// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCollectionTest.h"

namespace GeometryCollectionTest
{	
	
	void BuildProximity();
	
	void GeometryDeleteFromStart();

	void GeometryDeleteFromEnd();

	void GeometryDeleteFromMiddle();

	void GeometryDeleteMultipleFromMiddle();

	void GeometryDeleteRandom();

	void GeometryDeleteRandom2();

	void GeometryDeleteAll();

	void GeometrySwapFlat();

	void TestFracturedGeometry();
	
}
