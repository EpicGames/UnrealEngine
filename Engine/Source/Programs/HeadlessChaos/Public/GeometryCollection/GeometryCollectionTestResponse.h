// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace GeometryCollectionTest
{
	class ExampleResponse
	{
	public:
		ExampleResponse() : ErrorFlag(false) {}
		virtual ~ExampleResponse() {}
		virtual void ExpectTrue(bool, FString Reason = "");
		virtual bool HasError() { return ErrorFlag; }
		TArray<FString> Reasons;
		bool ErrorFlag;
	};

}