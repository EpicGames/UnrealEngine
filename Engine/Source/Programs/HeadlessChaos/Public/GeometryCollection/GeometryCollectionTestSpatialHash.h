// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCollectionTest.h"

namespace GeometryCollectionTest
{	

	void GetClosestPointsTest1();
	
	void GetClosestPointsTest2();

	void GetClosestPointsTest3();

	void GetClosestPointTest();

	void HashTableUpdateTest();

	void HashTablePressureTest();

}
