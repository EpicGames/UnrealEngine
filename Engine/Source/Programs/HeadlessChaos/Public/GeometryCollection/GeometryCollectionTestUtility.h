// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosSolversModule.h"
#include "Chaos/Serializable.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
//#include "GeometryCollection/GeometryCollection.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Templates/SharedPointer.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"

class FGeometryCollection;
class FGeometryDynamicCollection;
struct FSimulationParameters;

namespace GeometryCollectionTest {

	TSharedPtr<FGeometryDynamicCollection> GeometryCollectionToGeometryDynamicCollection(TSharedPtr<const FGeometryCollection> InputCollection, EObjectStateTypeEnum DynamicStateDefault = EObjectStateTypeEnum::Chaos_Object_Dynamic);

	TSharedPtr<FGeometryCollection>	CreateClusteredBody(FVector Position);
	TSharedPtr<FGeometryCollection>	CreateClusteredBody_OnePartent_FourBodies(FVector Position);
	TSharedPtr<FGeometryCollection>	CreateClusteredBody_TwoParents_TwoBodies(FVector Position);
	TSharedPtr<FGeometryCollection>	CreateClusteredBody_TwoParents_TwoBodiesB(FVector Position);
	TSharedPtr<FGeometryCollection>	CreateClusteredBody_FourParents_OneBody(FVector Position);
	TSharedPtr<FGeometryCollection>	CreateClusteredBody_TwoByTwo_ThreeTransform(FVector Position);
	TSharedPtr<FGeometryCollection>	CreateClusteredBody_ThreeByTwo_ThreeTransform(FVector Position);
	TSharedPtr<FGeometryCollection>	CreateClusteredBody_FracturedGeometry(FVector Position = FVector(0));

	void InitMaterialToZero(Chaos::FChaosPhysicsMaterial * PhysicalMaterial);

	/*
	using FInitFunc = TFunction<void(FSimulationParameters&)>;
	FGeometryCollectionPhysicsProxy* RigidBodySetup(
		TUniquePtr<Chaos::FChaosPhysicsMaterial> &PhysicalMaterial,
		TSharedPtr<FGeometryCollection> &RestCollection,
		TSharedPtr<FGeometryDynamicCollection> &DynamicCollection,
		FInitFunc CustomFunc = nullptr
	);
	*/
}
