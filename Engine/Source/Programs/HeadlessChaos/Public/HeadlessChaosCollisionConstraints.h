// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "Chaos/Box.h"
#include "Chaos/Evolution/SolverBodyContainer.h"
#include "Chaos/Evolution/ConstraintGroupSolver.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDCollisionConstraintsContact.h"
#include "Chaos/CollisionResolution.h"
#include "Chaos/Collision/SpatialAccelerationCollisionDetector.h"
#include "Chaos/CollisionResolutionUtil.h"
#include "Chaos/Plane.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"
#include "Chaos/Vector.h"
#include "Chaos/PBDRigidsSOAs.h"

namespace Chaos
{	

/**
 * Test collision constraints.
 */
class FPBDCollisionConstraintAccessor
{
public:
	using FCollisionConstraints = FPBDCollisionConstraints;
	using FConstraintContainerHandle = FPBDCollisionConstraintHandle;
	using FConstraintHandleAllocator = TConstraintHandleAllocator<FPBDCollisionConstraint>;
	using FConstraintHandleID = TPair<const FGeometryParticleHandle*, const FGeometryParticleHandle*>;
	using FCollisionDetector = FSpatialAccelerationCollisionDetector;
	using FAccelerationStructure = TBoundingVolume<FAccelerationStructureHandle>;

	FPBDCollisionConstraintAccessor()
		: EmptyParticles(UniqueIndices)
		, SpatialAcceleration(EmptyParticles.GetNonDisabledView())
		, CollisionConstraints(EmptyParticles, EmptyCollided, EmptyPhysicsMaterials, EmptyUniquePhysicsMaterials, nullptr)
		, BroadPhase(EmptyParticles)
		, CollisionDetector(BroadPhase, CollisionConstraints)
		, ConstraintSolver(Iterations)
	{
		CollisionConstraints.SetContainerId(0);

		CollisionConstraints.SetCullDistance(FReal(1));

		ConstraintSolver.SetConstraintSolver(CollisionConstraints.GetContainerId(), CollisionConstraints.CreateSceneSolver(0));
	}

	FPBDCollisionConstraintAccessor(const FPBDRigidsSOAs& InParticles, TArrayCollectionArray<bool>& Collided, const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>& PerParticleMaterials, const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>& PerParticleUniqueMaterials,
		const int32 PushOutIterations, const int32 PushOutPairIterations) 
		: EmptyParticles(UniqueIndices)
		, SpatialAcceleration(InParticles.GetNonDisabledView())
		, CollisionConstraints(InParticles, Collided, PerParticleMaterials, PerParticleUniqueMaterials, nullptr)
		, BroadPhase(InParticles)
		, CollisionDetector(BroadPhase, CollisionConstraints)
		, ConstraintSolver(Iterations)
	{
		CollisionConstraints.SetContainerId(0);

		CollisionConstraints.SetCullDistance(FReal(1));

		ConstraintSolver.SetConstraintSolver(CollisionConstraints.GetContainerId(), CollisionConstraints.CreateSceneSolver(0));
	}

	virtual ~FPBDCollisionConstraintAccessor() {}
	
	void ComputeConstraints(FReal Dt)
	{
		CollisionDetector.GetBroadPhase().SetSpatialAcceleration(&SpatialAcceleration);

		FCollisionDetectorSettings DetectorSettings = CollisionConstraints.GetDetectorSettings();
		DetectorSettings.bFilteringEnabled = true;
		DetectorSettings.bDeferNarrowPhase = false;
		DetectorSettings.bAllowManifolds = true;
		CollisionConstraints.SetDetectorSettings(DetectorSettings);
		
		CollisionDetector.DetectCollisions(Dt, nullptr);
		
		CollisionDetector.GetCollisionContainer().GetConstraintAllocator().SortConstraintsHandles();
	}

	void Update(FPBDCollisionConstraint& Constraint)
	{
		if (!Constraint.GetCCDEnabled())
		{
			// Dt is not important for the tests that use this function
			const FReal Dt = FReal(1) / FReal(30);

			FRigidTransform3 ShapeWorldTransform0 = Constraint.GetShapeRelativeTransform0() * Collisions::GetTransform(Constraint.GetParticle0());
			FRigidTransform3 ShapeWorldTransform1 = Constraint.GetShapeRelativeTransform1() * Collisions::GetTransform(Constraint.GetParticle1());
			Constraint.SetShapeWorldTransforms(ShapeWorldTransform0, ShapeWorldTransform1);

			Constraint.ResetPhi(TNumericLimits<FReal>::Max());
			Collisions::UpdateConstraint(Constraint, ShapeWorldTransform0, ShapeWorldTransform1,Dt);
		}
	}


	void UpdateLevelsetConstraint(FPBDCollisionConstraint& Constraint)
	{
		// Dt is not important for the tests that use this function
		const FReal Dt = FReal(1) / FReal(30);

		FRigidTransform3 WorldTransform0 = Constraint.GetShapeRelativeTransform0() * Collisions::GetTransform(Constraint.GetParticle0());
		FRigidTransform3 WorldTransform1 = Constraint.GetShapeRelativeTransform1() * Collisions::GetTransform(Constraint.GetParticle1());

		Constraint.ResetManifold();
		Collisions::UpdateLevelsetLevelsetConstraint(WorldTransform0, WorldTransform1, FReal(1 / 30.0f), Constraint);
	}

	int32 NumConstraints() const
	{
		return CollisionConstraints.NumConstraints();
	}

	FPBDCollisionConstraint& GetConstraint(int32 Index)
	{
		if (Index < CollisionConstraints.NumConstraints())
		{
			return GetConstraintHandle(Index)->GetContact();
		}
		return EmptyConstraint;
	}

	const FConstraintContainerHandle* GetConstraintHandle(int32 ConstraintIndex) const
	{
		return CollisionConstraints.GetConstraintHandles()[ConstraintIndex];
	}

	FConstraintContainerHandle* GetConstraintHandle(int32 ConstraintIndex)
	{
		return CollisionConstraints.GetConstraintHandles()[ConstraintIndex];
	}

	void Apply(const FReal Dt, const int32 NumIts)
	{
		Iterations.SetNumPositionIterations(NumIts);
		ConstraintSolver.SetIterationSettings(Iterations);
		ConstraintSolver.ApplyPositionConstraints(Dt);
	}

	void ApplyPushOut(const FReal Dt, int32 NumIts)
	{
		Iterations.SetNumVelocityIterations(NumIts);
		ConstraintSolver.SetIterationSettings(Iterations);
		ConstraintSolver.ApplyVelocityConstraints(Dt);
	}

	void GatherInput(FReal Dt)
	{
		ConstraintSolver.Reset();
		ConstraintSolver.AddConstraintsAndBodies();
		ConstraintSolver.GatherBodies(Dt);
		ConstraintSolver.GatherConstraints(Dt);
	}

	void ScatterOutput(FReal Dt)
	{
		ConstraintSolver.ScatterConstraints(Dt);
		ConstraintSolver.ScatterBodies(Dt);
	}

	FPBDCollisionConstraint EmptyConstraint;
	FParticleUniqueIndicesMultithreaded UniqueIndices;
	FPBDRigidsSOAs EmptyParticles;
	TArrayCollectionArray<bool> EmptyCollided;
	TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>> EmptyPhysicsMaterials;
	TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>> EmptyUniquePhysicsMaterials;

	FAccelerationStructure SpatialAcceleration;
	FCollisionConstraints CollisionConstraints;
	FSpatialAccelerationBroadPhase BroadPhase;
	FCollisionDetector CollisionDetector;
	Private::FPBDSceneConstraintGroupSolver ConstraintSolver;
	Private::FIterationSettings Iterations;
};
}
