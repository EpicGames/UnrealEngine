// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "HeadlessChaosTestUtility.h"

namespace ChaosTest {

	void GridBPTest();

	void GridBPEarlyExitTest();

	void GridBPTest2();

	void AABBTreeTest();

	void AABBTreeTestDynamic();

	void AABBTreeDirtyTreeTest();

	void AABBTreeDirtyGridTest();

	void AABBTreeTimesliceTest();

	void DoForSweepIntersectCellsImpTest();

	void AABBTreeDirtyGridFunctionsWithEdgeCase();

	void BroadphaseCollectionTest();

	void BoundingVolumeNoBoundsTest();

	void SpatialAccelerationDirtyAndGlobalQueryStrestTest();
}