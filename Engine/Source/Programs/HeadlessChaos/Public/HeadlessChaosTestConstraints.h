// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "HeadlessChaosTestUtility.h"
#include "Modules/ModuleManager.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"

namespace ChaosTest 
{
	/**
	 * Base class for constraint tests. Provides a basic sim with no builtin constraint support.
	 */
	template <typename TEvolution>
	class FConstraintsTest
	{
	public:

		FConstraintsTest(const int32 NumIterations, const FReal Gravity)
			: SOAs(UniqueIndices)
			, Evolution(SOAs, PhysicalMaterials)
		{
			PhysicalMaterial = MakeUnique<FChaosPhysicsMaterial>();
			PhysicalMaterial->Friction = 0;
			PhysicalMaterial->Restitution = 0;
			PhysicalMaterial->SleepingLinearThreshold = 0;
			PhysicalMaterial->SleepingAngularThreshold = 0;
			PhysicalMaterial->DisabledLinearThreshold = 0;
			PhysicalMaterial->DisabledAngularThreshold = 0;
			PhysicalMaterial->SleepCounterThreshold = std::numeric_limits<int32>::max();

			Evolution.SetNumPositionIterations(NumIterations);
			Evolution.SetNumVelocityIterations(1);
			Evolution.SetNumProjectionIterations(1);
			Evolution.GetGravityForces().SetAcceleration(Gravity * FVec3(0, 0, -1), 0);
		}

		virtual ~FConstraintsTest()
		{
		}

		// By default the tests are set up to disable sleeping. Undo this by giving the physical material a non-zero sleep threshold
		void EnableSleeping()
		{
			// These are the defaults from PhysicalMaterial.cpp
			PhysicalMaterial->SleepingLinearThreshold = 1;
			PhysicalMaterial->SleepingAngularThreshold = 0.05;
			PhysicalMaterial->SleepCounterThreshold = 4;
		}

		FGeometryParticleHandle* AddParticleBox(const FVec3& Position, const FRotation3& Rotation, const FVec3& Size, FReal Mass)
		{
			FPBDRigidParticleHandle* Particle = AppendDynamicParticleBox(SOAs, Size);

			if (Mass == 0)
			{
				Evolution.SetParticleObjectState(Particle, EObjectStateType::Kinematic);
			}

			ResetParticle(Particle, Position, Rotation, FVec3(0), FVec3(0));

			if(Particle->ObjectState() == EObjectStateType::Dynamic)
			{
				// NOTE: particle was created with unit mass
				Particle->M() = Particle->M() * Mass;
				Particle->I() = Particle->I() * Mass;
				Particle->InvM() = Particle->InvM() * ((FReal)1 / Mass);
				Particle->InvI() = Particle->InvI() * ((FReal)1 / Mass);
			}
			Evolution.SetPhysicsMaterial(Particle, MakeSerializable(PhysicalMaterial));

			Evolution.EnableParticle(Particle);

			return Particle;
		}

		void ResetParticle(FGeometryParticleHandle* Particle, const FVec3& Position, const FRotation3& Rotation, const FVec3& Velocity, const FVec3& AngularVelocity)
		{
			Particle->SetX(Position);
			Particle->SetR(Rotation);
			if (FKinematicGeometryParticleHandle* KinParticle = Particle->CastToKinematicParticle())
			{
				KinParticle->SetV(Velocity);
				KinParticle->SetW(AngularVelocity);
			}
			if (FPBDRigidParticleHandle* PBDParticle = Particle->CastToRigidParticle())
			{
				PBDParticle->SetP(Position);
				PBDParticle->SetQ(Rotation);
			}
		}

		// Solver state
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs SOAs;
		TEvolution Evolution;
		TUniquePtr<FChaosPhysicsMaterial> PhysicalMaterial;
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;

		FGeometryParticleHandle* GetParticle(const int32 Idx)
		{
			return SOAs.GetParticleHandles().Handle(Idx).Get();
		}
	};

}