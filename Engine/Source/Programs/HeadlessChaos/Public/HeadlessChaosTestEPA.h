// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HeadlessChaosTestUtility.h"
#include "Modules/ModuleManager.h"

namespace ChaosTest {

	void EPAInitTest();

	void EPASimpleTest();
}