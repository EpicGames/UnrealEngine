// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Box.h"
#include "Chaos/Collision/PBDCollisionConstraint.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"
#include "HeadlessChaosTestUtility.h"

namespace ChaosTest 
{
	/**
	 * Base class for evolution tests
	 */
	class FEvolutionTest
	{
	public:

		FEvolutionTest()
			: SOAs(UniqueIndices)
			, Evolution(SOAs, PhysicalMaterials)
		{
			PhysicalMaterial = MakeUnique<FChaosPhysicsMaterial>();
			PhysicalMaterial->Friction = 0;
			PhysicalMaterial->Restitution = 0;
			PhysicalMaterial->DisabledLinearThreshold = 0;
			PhysicalMaterial->DisabledAngularThreshold = 0;
			EnableSleeping();

			Dt = 1.0 / 30.0;
		}

		virtual ~FEvolutionTest()
		{
		}

		FPBDRigidsEvolutionGBF& GetEvolution()
		{
			return Evolution;
		}

		FPBDCollisionConstraints& GetCollisionConstraints()
		{
			return Evolution.GetCollisionConstraints();
		}

		// Get a particle. The index is the order in which they were created by the AddParticleXXX methods
		FGeometryParticleHandle* GetParticle(const int32 ParticleIndex)
		{
			return Particles[ParticleIndex];
		}

		// Set up the physical material to allow sleeping (sleeping is enabled by default)
		void EnableSleeping()
		{
			// These are the defaults from PhysicalMaterial.cpp
			PhysicalMaterial->SleepingLinearThreshold = 1;
			PhysicalMaterial->SleepingAngularThreshold = 0.05;
			PhysicalMaterial->SleepCounterThreshold = 4;
			OnMaterialChanged();
		}

		// Set up the physical material to disable sleeping (sleeping is enabled by default)
		void DisableSleeping()
		{
			// These are the defaults from PhysicalMaterial.cpp
			PhysicalMaterial->SleepingLinearThreshold = 0;
			PhysicalMaterial->SleepingAngularThreshold = 0;
			PhysicalMaterial->SleepCounterThreshold = std::numeric_limits<int32>::max();
			OnMaterialChanged();
		}

		void Tick()
		{
			Evolution.AdvanceOneTimeStep(Dt);
			Evolution.EndFrame(Dt);
		}

		bool TickUntilSleep(const int32 MaxTicks = 1000)
		{
			int32 TickIndex = 0;
			while (!AreAllParticlesAsleep() && (TickIndex++ < MaxTicks))
			{
				Tick();
			}

			return AreAllParticlesAsleep();
		}

		bool AreAllParticlesAsleep() const
		{
			for (FGeometryParticleHandle* Particle : Particles)
			{
				if (FPBDRigidParticleHandle* Rigid = Particle->CastToRigidParticle())
				{
					if (Rigid->IsDynamic() && !Rigid->IsSleeping())
					{
						return false;
					}
				}
			}
			return true;
		}

		// Create a dynamic or kinematic particle depending on the mass
		FPBDRigidParticleHandle* AddParticleBox(const FVec3& Position, const FRotation3& Rotation, const FVec3& Size, FReal Mass)
		{
			FPBDRigidParticleHandle* Particle = AppendDynamicParticleBox(SOAs, Size);

			if (Mass == 0)
			{
				Evolution.SetParticleObjectState(Particle, EObjectStateType::Kinematic);
			}

			ResetParticle(Particle, Position, Rotation, FVec3(0), FVec3(0));

			if(Particle->ObjectState() == EObjectStateType::Dynamic)
			{
				// NOTE: particle was created with unit mass
				Particle->M() = Particle->M() * Mass;
				Particle->I() = Particle->I() * Mass;
				Particle->InvM() = Particle->InvM() * ((FReal)1 / Mass);
				Particle->InvI() = Particle->InvI() * ((FReal)1 / Mass);
			}
			Evolution.SetPhysicsMaterial(Particle, MakeSerializable(PhysicalMaterial));

			Evolution.EnableParticle(Particle);

			Particles.Add(Particle);

			return Particle;
		}

		// Add a static particle, e.g., for the floor
		FGeometryParticleHandle* AddStaticParticleBox(const FVec3& Position, const FRotation3& Rotation, const FVec3& Size)
		{
			FGeometryParticleHandle* Particle = AppendStaticParticleBox(SOAs, Size);

			ResetParticle(Particle, Position, Rotation, FVec3(0), FVec3(0));

			Evolution.SetPhysicsMaterial(Particle, MakeSerializable(PhysicalMaterial));

			Evolution.EnableParticle(Particle);

			Particles.Add(Particle);

			return Particle;
		}

		// Add a static particle for the floor consisting of a box with the specified size and the upper surface at Z=0
		FGeometryParticleHandle* AddParticleFloor(const FVec3& Size)
		{
			return AddStaticParticleBox(FVec3(0, 0, -0.5 * Size.Z), FRotation3::FromIdentity(), Size);
		}

		void ResetParticle(FGeometryParticleHandle* Particle, const FVec3& Position, const FRotation3& Rotation, const FVec3& Velocity, const FVec3& AngularVelocity)
		{
			Particle->SetX(Position);
			Particle->SetR(Rotation);
			if (FKinematicGeometryParticleHandle* KinParticle = Particle->CastToKinematicParticle())
			{
				KinParticle->SetV(Velocity);
				KinParticle->SetW(AngularVelocity);
			}
			if (FPBDRigidParticleHandle* PBDParticle = Particle->CastToRigidParticle())
			{
				PBDParticle->SetP(Position);
				PBDParticle->SetQ(Rotation);
			}
		}

		void OnMaterialChanged()
		{
			// When we change the material so any collisions using it must reacquire materials properties
			GetCollisionConstraints().GetConstraintAllocator().VisitCollisions(
				[](FPBDCollisionConstraint& Collision)
				{
					Collision.ClearMaterialProperties();
					return ECollisionVisitorResult::Continue;
				});
		}

		// Solver state
		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FPBDRigidsSOAs SOAs;
		FPBDRigidsEvolutionGBF Evolution;
		TUniquePtr<FChaosPhysicsMaterial> PhysicalMaterial;
		THandleArray<FChaosPhysicsMaterial> PhysicalMaterials;

		TArray<FGeometryParticleHandle*> Particles;

		FReal Dt;
	};

}