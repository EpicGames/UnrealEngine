// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HeadlessChaosTestUtility.h"
#include "Modules/ModuleManager.h"

namespace ChaosTest {

	/**/
	void SimplexLine();

	void SimplexTriangle();

	void SimplexTetrahedron();

	void GJKSphereSphereTest();

	void GJKSphereBoxTest();

	void GJKSphereCapsuleTest();

	void GJKSphereConvexTest();

	void GJKSphereScaledSphereTest();

	void GJKSphereSphereSweep();

	void GJKSphereBoxSweep();

	void GJKSphereCapsuleSweep();

	void GJKSphereConvexSweep();

	void GJKSphereScaledSphereSweep();

	void GJKSphereTransformedSphereSweep();

	void GJKBoxCapsuleSweep();

	void GJKBoxBoxSweep();

	void GJKCapsuleConvexInitialOverlapSweep();

	void GJKLargeDistanceCapsuleSweep();

	void GJKConvexCapsule();
}