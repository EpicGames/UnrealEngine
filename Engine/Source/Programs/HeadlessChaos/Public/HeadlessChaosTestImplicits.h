// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HeadlessChaosTestUtility.h"
#include "Modules/ModuleManager.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidsEvolution.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/Sphere.h"
#include "Chaos/Utilities.h"

namespace ChaosTest {

	/**/
	void ImplicitCube();

	/**/
	void ImplicitPlane();

	/**/
	void ImplicitTetrahedron();

	/**/
	void ImplicitSphere();

	/**/
	void ImplicitCylinder();

	/**/
	void ImplicitTaperedCylinder();

	/**/
	void ImplicitTaperedCapsule();

	/**/
	void ImplicitCapsule();

	/**/
	void ImplicitScaled();

	/**/
	void ImplicitTransformed();

	/**/
	void ImplicitIntersection();

	/**/
	void ImplicitUnion();

	/**/
	void ImplicitLevelset();

	/**/
	void RasterizationImplicit();

	void RasterizationImplicitWithHole();

	void ConvexHull();

	void ConvexHull2();

	void Simplify();

	void ImplicitScaled2();

	void UpdateImplicitUnion();

}