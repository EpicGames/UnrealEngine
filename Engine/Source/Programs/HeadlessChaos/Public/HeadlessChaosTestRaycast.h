// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "HeadlessChaosTestUtility.h"

namespace ChaosTest {

	void SphereRaycast();

	void PlaneRaycast();

	void CylinderRaycast();

	void TaperedCylinderRaycast();

	void CapsuleRaycast();

	void CapsuleRaycastFastLargeDistance();

	void CapsuleRaycastMissWithEndPointOnBounds();

	void TriangleRaycast();

	void TriangleRaycastDenegerated();

	void BoxRaycast();

	void VectorizedAABBRaycast();

	void ScaledRaycast();

	void TransformedRaycast();

	void UnionRaycast();

	void IntersectionRaycast();
}