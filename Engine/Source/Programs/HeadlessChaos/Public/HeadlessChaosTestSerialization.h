// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "HeadlessChaosTestUtility.h"

namespace ChaosTest {

	void SimpleTypesSerialization();

	void SimpleObjectsSerialization();

	void SharedObjectsSerialization();

	void GraphSerialization();

	void ObjectUnionSerialization();

	void ParticleSerialization();

	void BVHSerialization();

	void RigidParticlesSerialization();

	void BVHParticlesSerialization();

	void GeometryCollectionSerialization();

	void EvolutionPerfHarness();

	void HeightFieldSerialization();

}