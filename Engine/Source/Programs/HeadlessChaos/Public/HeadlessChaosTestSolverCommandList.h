// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace ChaosTest
{
    template <typename T>
    void CommandListTest();
}
