// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace ChaosTest
{
    void SingleParticleProxySingleThreadTest();

	void SingleParticleProxyTaskGraphTest();

	void SingleParticleProxyWakeEventPropergationTest();
}
