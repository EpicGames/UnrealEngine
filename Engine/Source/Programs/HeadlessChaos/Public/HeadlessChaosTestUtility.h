// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HeadlessChaos.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/Vector.h"
#include "Math/Vector.h"

#include "Chaos/Box.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/Plane.h"
#include "Chaos/Sphere.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/Levelset.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Utilities.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "Chaos/Collision/CollisionApplyType.h"
#include "Chaos/Collision/CollisionContext.h"

namespace Chaos
{
	class FPBDRigidsSOAs;
}

namespace ChaosTest {
	
	using namespace Chaos;

	MATCHER_P2(VectorNear, V, Tolerance, "") { return arg.Equals(V, Tolerance); }

	// Expects each component of the vector is within T of its corresponding component in A. 
#define EXPECT_VECTOR_NEAR(A,B,T) EXPECT_THAT(A, VectorNear(B, T)) << *FString("Expected: " + B.ToString() + "\nActual:   " + A.ToString());
	// Default comparison to KINDA_SMALL_NUMBER.
#define EXPECT_VECTOR_NEAR_DEFAULT(A,B) EXPECT_THAT(A, VectorNear(B, KINDA_SMALL_NUMBER)) << *FString("Expected: " + B.ToString() + "\nActual:   " + A.ToString())
	// Print an additional error string if the expect fails.
#define EXPECT_VECTOR_NEAR_ERR(A,B,T,E) EXPECT_THAT(A, VectorNear(B, T)) << *(FString("Expected: " + B.ToString() + "\nActual:   " + A.ToString() + "\n") + E);

	// Similar to EXPECT_VECTOR_NEAR_DEFAULT but only prints the component(s) that are wrong, and prints with more precision.
#define EXPECT_VECTOR_FLOAT_EQ(A, B) EXPECT_FLOAT_EQ(A.X, B.X); EXPECT_FLOAT_EQ(A.Y, B.Y); EXPECT_FLOAT_EQ(A.Z, B.Z);
	// Print an additional error string if the expect fails. 
#define EXPECT_VECTOR_FLOAT_EQ_ERR(A, B, E) EXPECT_FLOAT_EQ(A.X, B.X) << *E; EXPECT_FLOAT_EQ(A.Y, B.Y) << *E; EXPECT_FLOAT_EQ(A.Z, B.Z) << *E;

	/**/
	int32 AppendAnalyticSphere(FPBDRigidParticles& InParticles, FReal Scale = (FReal)1);

	/**/
	int32 AppendAnalyticBox(FPBDRigidParticles& InParticles, FVec3 Scale = FVec3(1));

	/**/
	int32 AppendParticleBox(FPBDRigidParticles& InParticles, FVec3 Scale = FVec3(1), TArray<TVec3<int32>>* OutElements = nullptr);

	FPBDRigidParticleHandle* AppendDynamicParticleBox(FPBDRigidsSOAs& SOAs, const FVec3& Scale = FVec3(1), TArray<TVec3<int32>>* OutElements = nullptr);

	FPBDRigidParticleHandle* AppendDynamicParticleBoxMargin(FPBDRigidsSOAs& SOAs, const FVec3& Scale, FReal Margin, TArray<TVec3<int32>>* OutElements = nullptr);

	FPBDRigidParticleHandle* AppendDynamicParticleSphere(FPBDRigidsSOAs& SOAs, const FVec3& Scale = FVec3(1), TArray<TVec3<int32>>* OutElements = nullptr);

	FPBDRigidParticleHandle* AppendDynamicParticleCylinder(FPBDRigidsSOAs& SOAs, const FVec3& Scale = FVec3(1), TArray<TVec3<int32>>* OutElements = nullptr);
	
	FPBDRigidParticleHandle* AppendDynamicParticleTaperedCylinder(FPBDRigidsSOAs& SOAs, const FVec3& Scale = FVec3(1), TArray<TVec3<int32>>* OutElements = nullptr);

	FGeometryParticleHandle* AppendStaticParticleBox(FPBDRigidsSOAs& SOAs, const FVec3& Scale = FVec3(1), TArray<TVec3<int32>>* OutElements = nullptr);

	FPBDRigidParticleHandle* AppendClusteredParticleBox(FPBDRigidsSOAs& SOAs, const FVec3& Scale = FVec3(1), TArray<TVec3<int32>>* OutElements = nullptr);
		
	/**/
	int32 AppendStaticAnalyticFloor(FPBDRigidParticles& InParticles);

	FKinematicGeometryParticleHandle* AppendStaticAnalyticFloor(FPBDRigidsSOAs& SOAs);

	FKinematicGeometryParticleHandle* AppendStaticConvexFloor(FPBDRigidsSOAs& SOAs);
		
	/**/
	FLevelSet ConstructLevelset(FParticles& SurfaceParticles, TArray<TVec3<int32>> & Elements);

	/**/
	void AppendDynamicParticleConvexBox(FPBDRigidParticleHandle& InParticles, const FVec3& Scale, FReal Margin);
	
	FPBDRigidParticleHandle* AppendDynamicParticleConvexBox(FPBDRigidsSOAs& SOAs, const FVec3& Scale = FVec3(1));

	FPBDRigidParticleHandle* AppendDynamicParticleConvexBoxMargin(FPBDRigidsSOAs& SOAs, const FVec3& Scale, FReal Margin);

	/**/
	FVec3 ObjectSpacePoint(FPBDRigidParticles& InParticles, const int32 Index, const FVec3& WorldSpacePoint);

	/**/
	FReal PhiWithNormal(FPBDRigidParticles& InParticles, const int32 Index, const FVec3& WorldSpacePoint, FVec3& Normal);

	/**/
	FReal SignedDistance(FPBDRigidParticles& InParticles, const int32 Index, const FVec3& WorldSpacePoint);

	/**/
	FVec3  ObjectSpacePoint(FGeometryParticleHandle& Particle, const FVec3& WorldSpacePoint);

	/**/
	FReal PhiWithNormal(FGeometryParticleHandle& Particle, const FVec3& WorldSpacePoint, FVec3& Normal);

	/**/
	FReal SignedDistance(FGeometryParticleHandle& Particle, const FVec3& WorldSpacePoint);

	/**
	 * Return a random normalized axis.
	 * @note: not spherically distributed - actually calculates a point on a box and normalizes.
	 */
	FVec3 RandAxis();


	/**/
	inline FVec3 RandomVector(FReal MinValue, FReal MaxValue)
	{
		FVec3 V;
		V.X = FMath::RandRange(MinValue, MaxValue);
		V.Y = FMath::RandRange(MinValue, MaxValue);
		V.Z = FMath::RandRange(MinValue, MaxValue);
		return V;
	}

	/**/
	inline FMatrix33 RandomMatrix(FReal MinValue, FReal MaxValue)
	{
		return FMatrix33(
			RandomVector(MinValue, MaxValue),
			RandomVector(MinValue, MaxValue),
			RandomVector(MinValue, MaxValue));
	}

	/**/
	inline FRotation3 RandomRotation(FReal XMax, FReal YMax, FReal ZMax)
	{
		FReal X = FMath::DegreesToRadians(FMath::RandRange(-XMax, XMax));
		FReal Y = FMath::DegreesToRadians(FMath::RandRange(-YMax, YMax));
		FReal Z = FMath::DegreesToRadians(FMath::RandRange(-ZMax, ZMax));
		return FRotation3::FromAxisAngle(FVec3(1, 0, 0), X) * FRotation3::FromAxisAngle(FVec3(0, 1, 0), Y) * FRotation3::FromAxisAngle(FVec3(0, 0, 1), Z);
	}

	/**/
	void SetParticleSimDataToCollide(TArray< Chaos::FGeometryParticle* > ParticleArray);
	void SetParticleSimDataToCollide(TArray< Chaos::FGeometryParticleHandle* > ParticleArray);


	/**
	 * Set settings on Evolution to those used by the tests
	 */
	template<typename T_Evolution>
	void InitEvolutionSettings(T_Evolution& Evolution)
	{
		// Settings used for unit tests
		const float CullDistance = 3.0f;
		FCollisionDetectorSettings DetectorSettings = Evolution.GetCollisionConstraints().GetDetectorSettings();
		DetectorSettings.BoundsExpansion = CullDistance;
		DetectorSettings.bDeferNarrowPhase = false;
		DetectorSettings.bAllowManifolds = true;
		Evolution.GetCollisionConstraints().SetDetectorSettings(DetectorSettings);
	}

	template<typename T_SOLVER>
	void InitSolverSettings(T_SOLVER& Solver)
	{
		InitEvolutionSettings(*Solver->GetEvolution());
	}


	template <typename TParticle>
	void SetCubeInertiaTensor(TParticle& Particle, float Dimension, float Mass)
	{
		float Element = Mass * Dimension * Dimension / 6.0f;
		float InvElement = 1.f / Element;
		Particle.SetI(TVec3<FRealSingle>(Element));
		Particle.SetInvI(TVec3<FRealSingle>(InvElement));
	}

	template <typename TParticle>
	void SetBoxInertiaTensor(TParticle& Particle, const FVec3& Dimensions, float Mass)
	{
		float ElementX = Mass * (Dimensions.Y * Dimensions.Y + Dimensions.Z * Dimensions.Z) / 12.0f;
		float ElementY = Mass * (Dimensions.X * Dimensions.X + Dimensions.Z * Dimensions.Z) / 12.0f;
		float ElementZ = Mass * (Dimensions.Y * Dimensions.Y + Dimensions.X * Dimensions.X) / 12.0f;
		float InvElementX = 1.f / ElementX;
		float InvElementY = 1.f / ElementY;
		float InvElementZ = 1.f / ElementX;
		Particle.SetI(TVec3<FRealSingle>(ElementX, ElementY, ElementZ));
		Particle.SetInvI(TVec3<FRealSingle>(InvElementX, InvElementY, InvElementZ));
	}

	template <typename TParticle>
	void SetSphereInertiaTensor(TParticle& Particle, float Radius, float Mass)
	{
		float Element = 2.0f * Mass * Radius * Radius / 5.0f;
		float InvElement = 1.f / Element;
		Particle.SetI(TVec3<FRealSingle>(Element));
		Particle.SetInvI(TVec3<FRealSingle>(InvElement));
	}

	extern FImplicitConvex3 CreateConvexBox(const FConvex::FVec3Type& BoxSize, const FReal Margin);
	extern FImplicitConvex3 CreateConvexBox(const FConvex::FVec3Type& BoxMin, const FConvex::FVec3Type& BoxMax, const FReal Margin);
	extern TImplicitObjectInstanced<FImplicitConvex3> CreateInstancedConvexBox(const FConvex::FVec3Type& BoxSize, const FReal Margin);
	extern TImplicitObjectScaled<FImplicitConvex3> CreateScaledConvexBox(const FConvex::FVec3Type& BoxSize, const FVec3 BoxScale, const FReal Margin);
}
