// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Resource/BoxGeometry.h"
namespace GeometryCollectionTest
{
	const TArray<float> BoxGeometry::RawVertexArray = {
																-50.000000, -50.000000, -50.000000,
																50.000000, -50.000000, -50.000000,
																50.000000, -50.000000, 50.000000,
																-50.000000, -50.000000, 50.000000,
																-50.000000, 50.000000, -50.000000,
																50.000000, 50.000000, -50.000000,
																50.000000, 50.000000, 50.000000,
																-50.000000, 50.000000, 50.000000,
	};

	const TArray<int32> BoxGeometry::RawIndicesArray = {
															1, 5, 4,
															2, 6, 5,
															3, 7, 6,
															0, 4, 7,
															2, 1, 0,
															5, 6, 7,
															7, 4, 5,
															0, 3, 2,
															7, 3, 0,
															6, 2, 3,
															5, 1, 2,
															4, 0, 1
	};
}