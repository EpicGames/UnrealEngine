// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/Real.h"
#include "Chaos/Array.h"
namespace GeometryCollectionTest
{

	class CubeGeometry
	{

	public:
		CubeGeometry() {}
		~CubeGeometry() {}

		static const TArray<float>	RawVertexArray;
		static const TArray<int32>	RawIndicesArray;
	};


	const TArray<float> CubeGeometry::RawVertexArray = {
															50.000000, -50.000000, 50.000000,
															25.000000, -50.000000, 50.000000,
															0.000000, -50.000000, 50.000000,
															-25.000000, -50.000000, 50.000000,
															-50.000000, -50.000000, 50.000000,
															50.000000, -25.000000, 50.000000,
															25.000000, -25.000000, 50.000000,
															0.000000, -25.000000, 50.000000,
															-25.000000, -25.000000, 50.000000,
															-50.000000, -25.000000, 50.000000,
															50.000000, 0.000000, 50.000000,
															25.000000, 0.000000, 50.000000,
															0.000000, 0.000000, 50.000000,
															-25.000000, 0.000000, 50.000000,
															-50.000000, 0.000000, 50.000000,
															50.000000, 25.000000, 50.000000,
															25.000000, 25.000000, 50.000000,
															0.000000, 25.000000, 50.000000,
															-25.000000, 25.000000, 50.000000,
															-50.000000, 25.000000, 50.000000,
															50.000000, 50.000000, 50.000000,
															25.000000, 50.000000, 50.000000,
															0.000000, 50.000000, 50.000000,
															-25.000000, 50.000000, 50.000000,
															-50.000000, 50.000000, 50.000000,
															-50.000000, -50.000000, -50.000000,
															-25.000000, -50.000000, -50.000000,
															0.000000, -50.000000, -50.000000,
															25.000000, -50.000000, -50.000000,
															50.000000, -50.000000, -50.000000,
															-50.000000, -25.000000, -50.000000,
															-25.000000, -25.000000, -50.000000,
															0.000000, -25.000000, -50.000000,
															25.000000, -25.000000, -50.000000,
															50.000000, -25.000000, -50.000000,
															-50.000000, 0.000000, -50.000000,
															-25.000000, 0.000000, -50.000000,
															0.000000, 0.000000, -50.000000,
															25.000000, 0.000000, -50.000000,
															50.000000, 0.000000, -50.000000,
															-50.000000, 25.000000, -50.000000,
															-25.000000, 25.000000, -50.000000,
															0.000000, 25.000000, -50.000000,
															25.000000, 25.000000, -50.000000,
															50.000000, 25.000000, -50.000000,
															-50.000000, 50.000000, -50.000000,
															-25.000000, 50.000000, -50.000000,
															0.000000, 50.000000, -50.000000,
															25.000000, 50.000000, -50.000000,
															50.000000, 50.000000, -50.000000,
															-50.000000, 50.000000, -25.000000,
															-25.000000, 50.000000, -25.000000,
															0.000000, 50.000000, -25.000000,
															25.000000, 50.000000, -25.000000,
															50.000000, 50.000000, -25.000000,
															-50.000000, 50.000000, 0.000000,
															-25.000000, 50.000000, 0.000000,
															0.000000, 50.000000, 0.000000,
															25.000000, 50.000000, 0.000000,
															50.000000, 50.000000, 0.000000,
															-50.000000, 50.000000, 25.000000,
															-25.000000, 50.000000, 25.000000,
															0.000000, 50.000000, 25.000000,
															25.000000, 50.000000, 25.000000,
															50.000000, 50.000000, 25.000000,
															50.000000, -50.000000, -25.000000,
															25.000000, -50.000000, -25.000000,
															0.000000, -50.000000, -25.000000,
															-25.000000, -50.000000, -25.000000,
															-50.000000, -50.000000, -25.000000,
															50.000000, -50.000000, 0.000000,
															25.000000, -50.000000, 0.000000,
															0.000000, -50.000000, 0.000000,
															-25.000000, -50.000000, 0.000000,
															-50.000000, -50.000000, 0.000000,
															50.000000, -50.000000, 25.000000,
															25.000000, -50.000000, 25.000000,
															0.000000, -50.000000, 25.000000,
															-25.000000, -50.000000, 25.000000,
															-50.000000, -50.000000, 25.000000,
															50.000000, -25.000000, -25.000000,
															50.000000, -25.000000, 0.000000,
															50.000000, -25.000000, 25.000000,
															50.000000, 0.000000, -25.000000,
															50.000000, 0.000000, 0.000000,
															50.000000, 0.000000, 25.000000,
															50.000000, 25.000000, -25.000000,
															50.000000, 25.000000, 0.000000,
															50.000000, 25.000000, 25.000000,
															-50.000000, -25.000000, 25.000000,
															-50.000000, -25.000000, 0.000000,
															-50.000000, -25.000000, -25.000000,
															-50.000000, 0.000000, 25.000000,
															-50.000000, 0.000000, 0.000000,
															-50.000000, 0.000000, -25.000000,
															-50.000000, 25.000000, 25.000000,
															-50.000000, 25.000000, 0.000000,
															-50.000000, 25.000000, -25.000000
	};

	const TArray<int32> CubeGeometry::RawIndicesArray = {
															0, 1, 6,
															1, 2, 7,
															2, 3, 8,
															3, 4, 9,
															5, 6, 11,
															6, 7, 12,
															7, 8, 13,
															8, 9, 14,
															10, 11, 16,
															11, 12, 17,
															12, 13, 18,
															13, 14, 19,
															15, 16, 21,
															16, 17, 22,
															17, 18, 23,
															18, 19, 24,
															25, 26, 31,
															26, 27, 32,
															27, 28, 33,
															28, 29, 34,
															30, 31, 36,
															31, 32, 37,
															32, 33, 38,
															33, 34, 39,
															35, 36, 41,
															36, 37, 42,
															37, 38, 43,
															38, 39, 44,
															40, 41, 46,
															41, 42, 47,
															42, 43, 48,
															43, 44, 49,
															45, 46, 51,
															46, 47, 52,
															47, 48, 53,
															48, 49, 54,
															50, 51, 56,
															51, 52, 57,
															52, 53, 58,
															53, 54, 59,
															55, 56, 61,
															56, 57, 62,
															57, 58, 63,
															58, 59, 64,
															60, 61, 23,
															61, 62, 22,
															62, 63, 21,
															63, 64, 20,
															29, 28, 66,
															28, 27, 67,
															27, 26, 68,
															26, 25, 69,
															65, 66, 71,
															66, 67, 72,
															67, 68, 73,
															68, 69, 74,
															70, 71, 76,
															71, 72, 77,
															72, 73, 78,
															73, 74, 79,
															75, 76, 1,
															76, 77, 2,
															77, 78, 3,
															78, 79, 4,
															29, 65, 80,
															65, 70, 81,
															70, 75, 82,
															75, 0, 5,
															34, 80, 83,
															80, 81, 84,
															81, 82, 85,
															82, 5, 10,
															39, 83, 86,
															83, 84, 87,
															84, 85, 88,
															85, 10, 15,
															44, 86, 54,
															86, 87, 59,
															87, 88, 64,
															88, 15, 20,
															4, 79, 89,
															79, 74, 90,
															74, 69, 91,
															69, 25, 30,
															9, 89, 92,
															89, 90, 93,
															90, 91, 94,
															91, 30, 35,
															14, 92, 95,
															92, 93, 96,
															93, 94, 97,
															94, 35, 40,
															19, 95, 60,
															95, 96, 55,
															96, 97, 50,
															97, 40, 45,
															45, 50, 97,
															50, 55, 96,
															55, 60, 95,
															60, 24, 19,
															40, 97, 94,
															97, 96, 93,
															96, 95, 92,
															95, 19, 14,
															35, 94, 91,
															94, 93, 90,
															93, 92, 89,
															92, 14, 9,
															30, 91, 69,
															91, 90, 74,
															90, 89, 79,
															89, 9, 4,
															20, 64, 88,
															64, 59, 87,
															59, 54, 86,
															54, 49, 44,
															15, 88, 85,
															88, 87, 84,
															87, 86, 83,
															86, 44, 39,
															10, 85, 82,
															85, 84, 81,
															84, 83, 80,
															83, 39, 34,
															5, 82, 75,
															82, 81, 70,
															81, 80, 65,
															80, 34, 29,
															4, 3, 78,
															3, 2, 77,
															2, 1, 76,
															1, 0, 75,
															79, 78, 73,
															78, 77, 72,
															77, 76, 71,
															76, 75, 70,
															74, 73, 68,
															73, 72, 67,
															72, 71, 66,
															71, 70, 65,
															69, 68, 26,
															68, 67, 27,
															67, 66, 28,
															66, 65, 29,
															20, 21, 63,
															21, 22, 62,
															22, 23, 61,
															23, 24, 60,
															64, 63, 58,
															63, 62, 57,
															62, 61, 56,
															61, 60, 55,
															59, 58, 53,
															58, 57, 52,
															57, 56, 51,
															56, 55, 50,
															54, 53, 48,
															53, 52, 47,
															52, 51, 46,
															51, 50, 45,
															49, 48, 43,
															48, 47, 42,
															47, 46, 41,
															46, 45, 40,
															44, 43, 38,
															43, 42, 37,
															42, 41, 36,
															41, 40, 35,
															39, 38, 33,
															38, 37, 32,
															37, 36, 31,
															36, 35, 30,
															34, 33, 28,
															33, 32, 27,
															32, 31, 26,
															31, 30, 25,
															24, 23, 18,
															23, 22, 17,
															22, 21, 16,
															21, 20, 15,
															19, 18, 13,
															18, 17, 12,
															17, 16, 11,
															16, 15, 10,
															14, 13, 8,
															13, 12, 7,
															12, 11, 6,
															11, 10, 5,
															9, 8, 3,
															8, 7, 2,
															7, 6, 1,
															6, 5, 0
	};
}