// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Resource/CylinderGeometry.h"
namespace GeometryCollectionTest
{
	const TArray<float> CylinderGeometry::RawVertexArray = {
																100.000000f, 0.000000f, 100.000000f,
																95.105652f, 30.901701f, 100.000000f,
																80.901703f, 58.778526f, 100.000000f,
																58.778526f, 80.901703f, 100.000000f,
																30.901697f, 95.105652f, 100.000000f,
																-0.000004f, 100.000000f, 100.000000f,
																-30.901703f, 95.105652f, 100.000000f,
																-58.778519f, 80.901703f, 100.000000f,
																-80.901703f, 58.778519f, 100.000000f,
																-95.105659f, 30.901680f, 100.000000f,
																-100.000000f, -0.000009f, 100.000000f,
																-95.105652f, -30.901697f, 100.000000f,
																-80.901695f, -58.778538f, 100.000000f,
																-58.778507f, -80.901711f, 100.000000f,
																-30.901709f, -95.105652f, 100.000000f,
																0.000001f, -100.000000f, 100.000000f,
																30.901712f, -95.105652f, 100.000000f,
																58.778549f, -80.901680f, 100.000000f,
																80.901726f, -58.778496f, 100.000000f,
																95.105652f, -30.901695f, 100.000000f,
																100.000000f, 0.000000f, 60.000004f,
																95.105652f, 30.901701f, 60.000004f,
																80.901703f, 58.778526f, 60.000004f,
																58.778526f, 80.901703f, 60.000004f,
																30.901697f, 95.105652f, 60.000004f,
																-0.000004f, 100.000000f, 60.000004f,
																-30.901703f, 95.105652f, 60.000004f,
																-58.778519f, 80.901703f, 60.000004f,
																-80.901703f, 58.778519f, 60.000004f,
																-95.105659f, 30.901680f, 60.000004f,
																-100.000000f, -0.000009f, 60.000004f,
																-95.105652f, -30.901697f, 60.000004f,
																-80.901695f, -58.778538f, 60.000004f,
																-58.778507f, -80.901711f, 60.000004f,
																-30.901709f, -95.105652f, 60.000004f,
																0.000001f, -100.000000f, 60.000004f,
																30.901712f, -95.105652f, 60.000004f,
																58.778549f, -80.901680f, 60.000004f,
																80.901726f, -58.778496f, 60.000004f,
																95.105652f, -30.901695f, 60.000004f,
																100.000000f, 0.000000f, 20.000002f,
																95.105652f, 30.901701f, 20.000002f,
																80.901703f, 58.778526f, 20.000002f,
																58.778526f, 80.901703f, 20.000002f,
																30.901697f, 95.105652f, 20.000002f,
																-0.000004f, 100.000000f, 20.000002f,
																-30.901703f, 95.105652f, 20.000002f,
																-58.778519f, 80.901703f, 20.000002f,
																-80.901703f, 58.778519f, 20.000002f,
																-95.105659f, 30.901680f, 20.000002f,
																-100.000000f, -0.000009f, 20.000002f,
																-95.105652f, -30.901697f, 20.000002f,
																-80.901695f, -58.778538f, 20.000002f,
																-58.778507f, -80.901711f, 20.000002f,
																-30.901709f, -95.105652f, 20.000002f,
																0.000001f, -100.000000f, 20.000002f,
																30.901712f, -95.105652f, 20.000002f,
																58.778549f, -80.901680f, 20.000002f,
																80.901726f, -58.778496f, 20.000002f,
																95.105652f, -30.901695f, 20.000002f,
																100.000000f, -0.000000f, -19.999998f,
																95.105652f, 30.901701f, -19.999998f,
																80.901703f, 58.778526f, -19.999998f,
																58.778526f, 80.901703f, -19.999998f,
																30.901697f, 95.105652f, -19.999998f,
																-0.000004f, 100.000000f, -19.999998f,
																-30.901703f, 95.105652f, -19.999998f,
																-58.778519f, 80.901703f, -19.999998f,
																-80.901703f, 58.778519f, -19.999998f,
																-95.105659f, 30.901680f, -19.999998f,
																-100.000000f, -0.000009f, -19.999998f,
																-95.105652f, -30.901697f, -19.999998f,
																-80.901695f, -58.778538f, -19.999998f,
																-58.778507f, -80.901711f, -19.999998f,
																-30.901709f, -95.105652f, -19.999998f,
																0.000001f, -100.000000f, -19.999998f,
																30.901712f, -95.105652f, -19.999998f,
																58.778549f, -80.901680f, -19.999998f,
																80.901726f, -58.778496f, -19.999998f,
																95.105652f, -30.901695f, -19.999998f,
																100.000000f, -0.000000f, -60.000004f,
																95.105652f, 30.901701f, -60.000004f,
																80.901703f, 58.778526f, -60.000004f,
																58.778526f, 80.901703f, -60.000004f,
																30.901697f, 95.105652f, -60.000004f,
																-0.000004f, 100.000000f, -60.000004f,
																-30.901703f, 95.105652f, -60.000004f,
																-58.778519f, 80.901703f, -60.000004f,
																-80.901703f, 58.778519f, -60.000004f,
																-95.105659f, 30.901680f, -60.000004f,
																-100.000000f, -0.000009f, -60.000004f,
																-95.105652f, -30.901697f, -60.000004f,
																-80.901695f, -58.778538f, -60.000004f,
																-58.778507f, -80.901711f, -60.000004f,
																-30.901709f, -95.105652f, -60.000004f,
																0.000001f, -100.000000f, -60.000004f,
																30.901712f, -95.105652f, -60.000004f,
																58.778549f, -80.901680f, -60.000004f,
																80.901726f, -58.778496f, -60.000004f,
																95.105652f, -30.901695f, -60.000004f,
																100.000000f, -0.000000f, -100.000000f,
																95.105652f, 30.901701f, -100.000000f,
																80.901703f, 58.778526f, -100.000000f,
																58.778526f, 80.901703f, -100.000000f,
																30.901697f, 95.105652f, -100.000000f,
																-0.000004f, 100.000000f, -100.000000f,
																-30.901703f, 95.105652f, -100.000000f,
																-58.778519f, 80.901703f, -100.000000f,
																-80.901703f, 58.778519f, -100.000000f,
																-95.105659f, 30.901680f, -100.000000f,
																-100.000000f, -0.000009f, -100.000000f,
																-95.105652f, -30.901697f, -100.000000f,
																-80.901695f, -58.778538f, -100.000000f,
																-58.778507f, -80.901711f, -100.000000f,
																-30.901709f, -95.105652f, -100.000000f,
																0.000001f, -100.000000f, -100.000000f,
																30.901712f, -95.105652f, -100.000000f,
																58.778549f, -80.901680f, -100.000000f,
																80.901726f, -58.778496f, -100.000000f,
																95.105652f, -30.901695f, -100.000000f,
																100.000000f, 0.0000003f, 100.000000f,
																95.105652f, 30.9017013f, 100.000000f,
																80.901703f, 58.7785263f, 100.000000f,
																58.778526f, 80.9017033f, 100.000000f,
																30.901697f, 95.1056523f, 100.000000f,
																-0.000004f, 100.0000003f, 100.000000f,
																-30.901703f, 95.1056523f, 100.000000f,
																-58.778519f, 80.9017033f, 100.000000f,
																-80.901703f, 58.7785193f, 100.000000f,
																-95.105659f, 30.9016803f, 100.000000f,
																-100.000000f, -0.0000093f, 100.000000f,
																-95.105652f, -30.9016973f, 100.000000f,
																-80.901695f, -58.7785383f, 100.000000f,
																-58.778507f, -80.9017113f, 100.000000f,
																-30.901709f, -95.1056523f, 100.000000f,
																0.000001f, -100.0000003f, 100.000000f,
																30.901712f, -95.1056523f, 100.000000f,
																58.778549f, -80.9016803f, 100.000000f,
																80.901726f, -58.7784963f, 100.000000f,
																95.105652f, -30.9016953f, 100.000000f,
																100.000000f, 0.0000003f, 60.000004f,
																95.105652f, 30.9017013f, 60.000004f,
																80.901703f, 58.7785263f, 60.000004f,
																58.778526f, 80.9017033f, 60.000004f,
																30.901697f, 95.1056523f, 60.000004f,
																-0.000004f, 100.0000003f, 60.000004f,
																-30.901703f, 95.1056523f, 60.000004f,
																-58.778519f, 80.9017033f, 60.000004f,
																-80.901703f, 58.7785193f, 60.000004f,
																-95.105659f, 30.9016803f, 60.000004f,
																-100.000000f, -0.0000093f, 60.000004f,
																-95.105652f, -30.9016973f, 60.000004f,
																-80.901695f, -58.7785383f, 60.000004f,
																-58.778507f, -80.9017113f, 60.000004f,
																-30.901709f, -95.1056523f, 60.000004f,
																0.000001f, -100.0000003f, 60.000004f,
																30.901712f, -95.1056523f, 60.000004f,
																58.778549f, -80.9016803f, 60.000004f,
																80.901726f, -58.7784963f, 60.000004f,
																95.105652f, -30.9016953f, 60.000004f,
																100.000000f, 0.0000003f, 20.000002f,
																95.105652f, 30.9017013f, 20.000002f,
																80.901703f, 58.7785263f, 20.000002f,
																58.778526f, 80.9017033f, 20.000002f,
																30.901697f, 95.1056523f, 20.000002f,
																-0.000004f, 100.0000003f, 20.000002f,
																-30.901703f, 95.1056523f, 20.000002f,
																-58.778519f, 80.9017033f, 20.000002f,
																-80.901703f, 58.7785193f, 20.000002f,
																-95.105659f, 30.9016803f, 20.000002f,
																-100.000000f, -0.0000093f, 20.000002f,
																-95.105652f, -30.9016973f, 20.000002f,
																-80.901695f, -58.7785383f, 20.000002f,
																-58.778507f, -80.9017113f, 20.000002f,
																-30.901709f, -95.1056523f, 20.000002f,
																0.000001f, -100.0000003f, 20.000002f,
																30.901712f, -95.1056523f, 20.000002f,
																58.778549f, -80.9016803f, 20.000002f,
																80.901726f, -58.7784963f, 20.000002f,
																95.105652f, -30.9016953f, 20.000002f,
																100.000000f, -0.0000003f, -19.999998f,
																95.105652f, 30.9017013f, -19.999998f,
																80.901703f, 58.7785263f, -19.999998f,
																58.778526f, 80.9017033f, -19.999998f,
																30.901697f, 95.1056523f, -19.999998f,
																-0.000004f, 100.0000003f, -19.999998f,
																-30.901703f, 95.1056523f, -19.999998f,
																-58.778519f, 80.9017033f, -19.999998f,
																-80.901703f, 58.7785193f, -19.999998f,
																-95.105659f, 30.9016803f, -19.999998f,
																-100.000000f, -0.0000093f, -19.999998f,
																-95.105652f, -30.9016973f, -19.999998f,
																-80.901695f, -58.7785383f, -19.999998f,
																-58.778507f, -80.9017113f, -19.999998f,
																-30.901709f, -95.1056523f, -19.999998f,
																0.000001f, -100.0000003f, -19.999998f,
																30.901712f, -95.1056523f, -19.999998f,
																58.778549f, -80.9016803f, -19.999998f,
																80.901726f, -58.7784963f, -19.999998f,
																95.105652f, -30.9016953f, -19.999998f,
																100.000000f, -0.0000003f, -60.000004f,
																95.105652f, 30.9017013f, -60.000004f,
																80.901703f, 58.7785263f, -60.000004f,
																58.778526f, 80.9017033f, -60.000004f,
																30.901697f, 95.1056523f, -60.000004f,
																-0.000004f, 100.0000003f, -60.000004f,
																-30.901703f, 95.1056523f, -60.000004f,
																-58.778519f, 80.9017033f, -60.000004f,
																-80.901703f, 58.7785193f, -60.000004f,
																-95.105659f, 30.9016803f, -60.000004f,
																-100.000000f, -0.0000093f, -60.000004f,
																-95.105652f, -30.9016973f, -60.000004f,
																-80.901695f, -58.7785383f, -60.000004f,
																-58.778507f, -80.9017113f, -60.000004f,
																-30.901709f, -95.1056523f, -60.000004f,
																0.000001f, -100.0000003f, -60.000004f,
																30.901712f, -95.1056523f, -60.000004f,
																58.778549f, -80.9016803f, -60.000004f,
																80.901726f, -58.7784963f, -60.000004f,
																95.105652f, -30.9016953f, -60.000004f,
																100.000000f, -0.0000003f, -100.000000f,
																95.105652f, 30.9017013f, -100.000000f,
																80.901703f, 58.7785263f, -100.000000f,
																58.778526f, 80.9017033f, -100.000000f,
																30.901697f, 95.1056523f, -100.000000f,
																-0.000004f, 100.0000003f, -100.000000f,
																-30.901703f, 95.1056523f, -100.000000f,
																-58.778519f, 80.9017033f, -100.000000f,
																-80.901703f, 58.7785193f, -100.000000f,
																-95.105659f, 30.9016803f, -100.000000f,
																-100.000000f, -0.0000093f, -100.000000f,
																-95.105652f, -30.9016973f, -100.000000f,
																-80.901695f, -58.7785383f, -100.000000f,
																-58.778507f, -80.9017113f, -100.000000f,
																-30.901709f, -95.1056523f, -100.000000f,
																0.000001f, -100.0000003f, -100.000000f,
																30.901712f, -95.1056523f, -100.000000f,
																58.778549f, -80.9016803f, -100.000000f,
																80.901726f, -58.7784963f, -100.000000f,
																95.105652f, -30.9016953f, -100.000000f,
																100.000000f,       0.0000f, 100.000000f,
																95.105652f,      30.9017f, 100.000000f,
																80.901703f,      58.7785f, 100.000000f,
																58.778526f,      80.9017f, 100.000000f,
																30.901697f,      95.1057f, 100.000000f,
																-0.000004f,     100.0000f, 100.000000f,
																-30.901703f,      95.1057f, 100.000000f,
																-58.778519f,      80.9017f, 100.000000f,
																-80.901703f,      58.7785f, 100.000000f,
																-95.105659f,      30.9017f, 100.000000f,
																-100.000000f,      -0.0000f, 100.000000f,
																-95.105652f,     -30.9017f, 100.000000f,
																-80.901695f,     -58.7785f, 100.000000f,
																-58.778507f,     -80.9017f, 100.000000f,
																-30.901709f,     -95.1057f, 100.000000f,
																0.000001f,    -100.0000f, 100.000000f,
																30.901712f,     -95.1057f, 100.000000f,
																58.778549f,     -80.9017f, 100.000000f,
																80.901726f,     -58.7785f, 100.000000f,
																95.105652f,     -30.9017f, 100.000000f,
																100.000000f,       0.0000f, 60.000004f,
																95.105652f,      30.9017f, 60.000004f,
																80.901703f,      58.7785f, 60.000004f,
																58.778526f,      80.9017f, 60.000004f,
																30.901697f,      95.1057f, 60.000004f,
																-0.000004f,     100.0000f, 60.000004f,
																-30.901703f,      95.1057f, 60.000004f,
																-58.778519f,      80.9017f, 60.000004f,
																-80.901703f,      58.7785f, 60.000004f,
																-95.105659f,      30.9017f, 60.000004f,
																-100.000000f,      -0.0000f, 60.000004f,
																-95.105652f,     -30.9017f, 60.000004f,
																-80.901695f,     -58.7785f, 60.000004f,
																-58.778507f,     -80.9017f, 60.000004f,
																-30.901709f,     -95.1057f, 60.000004f,
																0.000001f,    -100.0000f, 60.000004f,
																30.901712f,     -95.1057f, 60.000004f,
																58.778549f,     -80.9017f, 60.000004f,
																80.901726f,     -58.7785f, 60.000004f,
																95.105652f,     -30.9017f, 60.000004f,
																100.000000f,       0.0000f, 20.000002f,
																95.105652f,      30.9017f, 20.000002f,
																80.901703f,      58.7785f, 20.000002f,
																58.778526f,      80.9017f, 20.000002f,
																30.901697f,      95.1057f, 20.000002f,
																-0.000004f,     100.0000f, 20.000002f,
																-30.901703f,      95.1057f, 20.000002f,
																-58.778519f,      80.9017f, 20.000002f,
																-80.901703f,      58.7785f, 20.000002f,
																-95.105659f,      30.9017f, 20.000002f,
																-100.000000f,      -0.0000f, 20.000002f,
																-95.105652f,     -30.9017f, 20.000002f,
																-80.901695f,     -58.7785f, 20.000002f,
																-58.778507f,     -80.9017f, 20.000002f,
																-30.901709f,     -95.1057f, 20.000002f,
																0.000001f,    -100.0000f, 20.000002f,
																30.901712f,     -95.1057f, 20.000002f,
																58.778549f,     -80.9017f, 20.000002f,
																80.901726f,     -58.7785f, 20.000002f,
																95.105652f,     -30.9017f, 20.000002f,
																100.000000f,      -0.0000f, -19.999998f,
																95.105652f,      30.9017f, -19.999998f,
																80.901703f,      58.7785f, -19.999998f,
																58.778526f,      80.9017f, -19.999998f,
																30.901697f,      95.1057f, -19.999998f,
																-0.000004f,     100.0000f, -19.999998f,
																-30.901703f,      95.1057f, -19.999998f,
																-58.778519f,      80.9017f, -19.999998f,
																-80.901703f,      58.7785f, -19.999998f,
																-95.105659f,      30.9017f, -19.999998f,
																-100.000000f,      -0.0000f, -19.999998f,
																-95.105652f,     -30.9017f, -19.999998f,
																-80.901695f,     -58.7785f, -19.999998f,
																-58.778507f,     -80.9017f, -19.999998f,
																-30.901709f,     -95.1057f, -19.999998f,
																0.000001f,    -100.0000f, -19.999998f,
																30.901712f,     -95.1057f, -19.999998f,
																58.778549f,     -80.9017f, -19.999998f,
																80.901726f,     -58.7785f, -19.999998f,
																95.105652f,     -30.9017f, -19.999998f,
																100.000000f,      -0.0000f, -60.000004f,
																95.105652f,      30.9017f, -60.000004f,
																80.901703f,      58.7785f, -60.000004f,
																58.778526f,      80.9017f, -60.000004f,
																30.901697f,      95.1057f, -60.000004f,
																-0.000004f,     100.0000f, -60.000004f,
																-30.901703f,      95.1057f, -60.000004f,
																-58.778519f,      80.9017f, -60.000004f,
																-80.901703f,      58.7785f, -60.000004f,
																-95.105659f,      30.9017f, -60.000004f,
																-100.000000f,      -0.0000f, -60.000004f,
																-95.105652f,     -30.9017f, -60.000004f,
																-80.901695f,     -58.7785f, -60.000004f,
																-58.778507f,     -80.9017f, -60.000004f,
																-30.901709f,     -95.1057f, -60.000004f,
																0.000001f,    -100.0000f, -60.000004f,
																30.901712f,     -95.1057f, -60.000004f,
																58.778549f,     -80.9017f, -60.000004f,
																80.901726f,     -58.7785f, -60.000004f,
																95.105652f,     -30.9017f, -60.000004f,
																100.000000f,      -0.0000f, -100.000000f,
																95.105652f,      30.9017f, -100.000000f,
																80.901703f,      58.7785f, -100.000000f,
																58.778526f,      80.9017f, -100.000000f,
																30.901697f,      95.1057f, -100.000000f,
																-0.000004f,     100.0000f, -100.000000f,
																-30.901703f,      95.1057f, -100.000000f,
																-58.778519f,      80.9017f, -100.000000f,
																-80.901703f,      58.7785f, -100.000000f,
																-95.105659f,      30.9017f, -100.000000f,
																-100.000000f,      -0.0000f, -100.000000f,
																-95.105652f,     -30.9017f, -100.000000f,
																-80.901695f,     -58.7785f, -100.000000f,
																-58.778507f,     -80.9017f, -100.000000f,
																-30.901709f,     -95.1057f, -100.000000f,
																0.000001f,    -100.0000f, -100.000000f,
																30.901712f,     -95.1057f, -100.000000f,
																58.778549f,     -80.9017f, -100.000000f,
																80.901726f,     -58.7785f, -100.000000f,
																95.105652f,     -30.9017f, -100.000000f,
																100.000000f, 0.000000f, 100.000000f,
																95.105652f, 30.901701f, 100.000000f,
																80.901703f, 58.778526f, 100.000000f,
																58.778526f, 80.901703f, 100.000000f,
																30.901697f, 95.105652f, 100.000000f,
																-0.000004f, 100.000000f, 100.000000f,
																-30.901703f, 95.105652f, 100.000000f,
																-58.778519f, 80.901703f, 100.000000f,
																-80.901703f, 58.778519f, 100.000000f,
																-95.105659f, 30.901680f, 100.000000f,
																-100.000000f, -0.000009f, 100.000000f,
																-95.105652f, -30.901697f, 100.000000f,
																-80.901695f, -58.778538f, 100.000000f,
																-58.778507f, -80.901711f, 100.000000f,
																-30.901709f, -95.105652f, 100.000000f,
																0.000001f, -100.000000f, 100.000000f,
																30.901712f, -95.105652f, 100.000000f,
																58.778549f, -80.901680f, 100.000000f,
																80.901726f, -58.778496f, 100.000000f,
																95.105652f, -30.901695f, 100.000000f,
																100.000000f, 0.000000f, 60.000004f,
																95.105652f, 30.901701f, 60.000004f,
																80.901703f, 58.778526f, 60.000004f,
																58.778526f, 80.901703f, 60.000004f,
																30.901697f, 95.105652f, 60.000004f,
																-0.000004f, 100.000000f, 60.000004f,
																-30.901703f, 95.105652f, 60.000004f,
																-58.778519f, 80.901703f, 60.000004f,
																-80.901703f, 58.778519f, 60.000004f,
																-95.105659f, 30.901680f, 60.000004f,
																-100.000000f, -0.000009f, 60.000004f,
																-95.105652f, -30.901697f, 60.000004f,
																-80.901695f, -58.778538f, 60.000004f,
																-58.778507f, -80.901711f, 60.000004f,
																-30.901709f, -95.105652f, 60.000004f,
																0.000001f, -100.000000f, 60.000004f,
																30.901712f, -95.105652f, 60.000004f,
																58.778549f, -80.901680f, 60.000004f,
																80.901726f, -58.778496f, 60.000004f,
																95.105652f, -30.901695f, 60.000004f,
																100.000000f, 0.000000f, 20.000002f,
																95.105652f, 30.901701f, 20.000002f,
																80.901703f, 58.778526f, 20.000002f,
																58.778526f, 80.901703f, 20.000002f,
																30.901697f, 95.105652f, 20.000002f,
																-0.000004f, 100.000000f, 20.000002f,
																-30.901703f, 95.105652f, 20.000002f,
																-58.778519f, 80.901703f, 20.000002f,
																-80.901703f, 58.778519f, 20.000002f,
																-95.105659f, 30.901680f, 20.000002f,
																-100.000000f, -0.000009f, 20.000002f,
																-95.105652f, -30.901697f, 20.000002f,
																-80.901695f, -58.778538f, 20.000002f,
																-58.778507f, -80.901711f, 20.000002f,
																-30.901709f, -95.105652f, 20.000002f,
																0.000001f, -100.000000f, 20.000002f,
																30.901712f, -95.105652f, 20.000002f,
																58.778549f, -80.901680f, 20.000002f,
																80.901726f, -58.778496f, 20.000002f,
																95.105652f, -30.901695f, 20.000002f,
																100.000000f, -0.000000f, -19.999998f,
																95.105652f, 30.901701f, -19.999998f,
																80.901703f, 58.778526f, -19.999998f,
																58.778526f, 80.901703f, -19.999998f,
																30.901697f, 95.105652f, -19.999998f,
																-0.000004f, 100.000000f, -19.999998f,
																-30.901703f, 95.105652f, -19.999998f,
																-58.778519f, 80.901703f, -19.999998f,
																-80.901703f, 58.778519f, -19.999998f,
																-95.105659f, 30.901680f, -19.999998f,
																-100.000000f, -0.000009f, -19.999998f,
																-95.105652f, -30.901697f, -19.999998f,
																-80.901695f, -58.778538f, -19.999998f,
																-58.778507f, -80.901711f, -19.999998f,
																-30.901709f, -95.105652f, -19.999998f,
																0.000001f, -100.000000f, -19.999998f,
																30.901712f, -95.105652f, -19.999998f,
																58.778549f, -80.901680f, -19.999998f,
																80.901726f, -58.778496f, -19.999998f,
																95.105652f, -30.901695f, -19.999998f,
																100.000000f, -0.000000f, -60.000004f,
																95.105652f, 30.901701f, -60.000004f,
																80.901703f, 58.778526f, -60.000004f,
																58.778526f, 80.901703f, -60.000004f,
																30.901697f, 95.105652f, -60.000004f,
																-0.000004f, 100.000000f, -60.000004f,
																-30.901703f, 95.105652f, -60.000004f,
																-58.778519f, 80.901703f, -60.000004f,
																-80.901703f, 58.778519f, -60.000004f,
																-95.105659f, 30.901680f, -60.000004f,
																-100.000000f, -0.000009f, -60.000004f,
																-95.105652f, -30.901697f, -60.000004f,
																-80.901695f, -58.778538f, -60.000004f,
																-58.778507f, -80.901711f, -60.000004f,
																-30.901709f, -95.105652f, -60.000004f,
																0.000001f, -100.000000f, -60.000004f,
																30.901712f, -95.105652f, -60.000004f,
																58.778549f, -80.901680f, -60.000004f,
																80.901726f, -58.778496f, -60.000004f,
																95.105652f, -30.901695f, -60.000004f,
																100.000000f, -0.000000f, -100.000000f,
																95.105652f, 30.901701f, -100.000000f,
																80.901703f, 58.778526f, -100.000000f,
																58.778526f, 80.901703f, -100.000000f,
																30.901697f, 95.105652f, -100.000000f,
																-0.000004f, 100.000000f, -100.000000f,
																-30.901703f, 95.105652f, -100.000000f,
																-58.778519f, 80.901703f, -100.000000f,
																-80.901703f, 58.778519f, -100.000000f,
																-95.105659f, 30.901680f, -100.000000f,
																-100.000000f, -0.000009f, -100.000000f,
																-95.105652f, -30.901697f, -100.000000f,
																-80.901695f, -58.778538f, -100.000000f,
																-58.778507f, -80.901711f, -100.000000f,
																-30.901709f, -95.105652f, -100.000000f,
																0.000001f, -100.000000f, -100.000000f,
																30.901712f, -95.105652f, -100.000000f,
																58.778549f, -80.901680f, -100.000000f,
																80.901726f, -58.778496f, -100.000000f,
																95.105652f, -30.901695f, -100.000000f
	};

	const TArray<int32> CylinderGeometry::RawIndicesArray = {
																10, 9, 8,
																109, 110, 111,
																0, 1, 21,
																1, 2, 22,
																2, 3, 23,
																3, 4, 24,
																4, 5, 25,
																5, 6, 26,
																6, 7, 27,
																7, 8, 28,
																8, 9, 29,
																9, 10, 30,
																10, 11, 31,
																11, 12, 32,
																12, 13, 33,
																13, 14, 34,
																14, 15, 35,
																15, 16, 36,
																16, 17, 37,
																17, 18, 38,
																18, 19, 39,
																19, 0, 20,
																20, 21, 41,
																21, 22, 42,
																22, 23, 43,
																23, 24, 44,
																24, 25, 45,
																25, 26, 46,
																26, 27, 47,
																27, 28, 48,
																28, 29, 49,
																29, 30, 50,
																30, 31, 51,
																31, 32, 52,
																32, 33, 53,
																33, 34, 54,
																34, 35, 55,
																35, 36, 56,
																36, 37, 57,
																37, 38, 58,
																38, 39, 59,
																39, 20, 40,
																40, 41, 61,
																41, 42, 62,
																42, 43, 63,
																43, 44, 64,
																44, 45, 65,
																45, 46, 66,
																46, 47, 67,
																47, 48, 68,
																48, 49, 69,
																49, 50, 70,
																50, 51, 71,
																51, 52, 72,
																52, 53, 73,
																53, 54, 74,
																54, 55, 75,
																55, 56, 76,
																56, 57, 77,
																57, 58, 78,
																58, 59, 79,
																59, 40, 60,
																60, 61, 81,
																61, 62, 82,
																62, 63, 83,
																63, 64, 84,
																64, 65, 85,
																65, 66, 86,
																66, 67, 87,
																67, 68, 88,
																68, 69, 89,
																69, 70, 90,
																70, 71, 91,
																71, 72, 92,
																72, 73, 93,
																73, 74, 94,
																74, 75, 95,
																75, 76, 96,
																76, 77, 97,
																77, 78, 98,
																78, 79, 99,
																79, 60, 80,
																80, 81, 101,
																81, 82, 102,
																82, 83, 103,
																83, 84, 104,
																84, 85, 105,
																85, 86, 106,
																86, 87, 107,
																87, 88, 108,
																88, 89, 109,
																89, 90, 110,
																90, 91, 111,
																91, 92, 112,
																92, 93, 113,
																93, 94, 114,
																94, 95, 115,
																95, 96, 116,
																96, 97, 117,
																97, 98, 118,
																98, 99, 119,
																99, 80, 100,
																100, 119, 99,
																119, 118, 98,
																118, 117, 97,
																117, 116, 96,
																116, 115, 95,
																115, 114, 94,
																114, 113, 93,
																113, 112, 92,
																112, 111, 91,
																111, 110, 90,
																110, 109, 89,
																109, 108, 88,
																108, 107, 87,
																107, 106, 86,
																106, 105, 85,
																105, 104, 84,
																104, 103, 83,
																103, 102, 82,
																102, 101, 81,
																101, 100, 80,
																80, 99, 79,
																99, 98, 78,
																98, 97, 77,
																97, 96, 76,
																96, 95, 75,
																95, 94, 74,
																94, 93, 73,
																93, 92, 72,
																92, 91, 71,
																91, 90, 70,
																90, 89, 69,
																89, 88, 68,
																88, 87, 67,
																87, 86, 66,
																86, 85, 65,
																85, 84, 64,
																84, 83, 63,
																83, 82, 62,
																82, 81, 61,
																81, 80, 60,
																60, 79, 59,
																79, 78, 58,
																78, 77, 57,
																77, 76, 56,
																76, 75, 55,
																75, 74, 54,
																74, 73, 53,
																73, 72, 52,
																72, 71, 51,
																71, 70, 50,
																70, 69, 49,
																69, 68, 48,
																68, 67, 47,
																67, 66, 46,
																66, 65, 45,
																65, 64, 44,
																64, 63, 43,
																63, 62, 42,
																62, 61, 41,
																61, 60, 40,
																40, 59, 39,
																59, 58, 38,
																58, 57, 37,
																57, 56, 36,
																56, 55, 35,
																55, 54, 34,
																54, 53, 33,
																53, 52, 32,
																52, 51, 31,
																51, 50, 30,
																50, 49, 29,
																49, 48, 28,
																48, 47, 27,
																47, 46, 26,
																46, 45, 25,
																45, 44, 24,
																44, 43, 23,
																43, 42, 22,
																42, 41, 21,
																41, 40, 20,
																20, 39, 19,
																39, 38, 18,
																38, 37, 17,
																37, 36, 16,
																36, 35, 15,
																35, 34, 14,
																34, 33, 13,
																33, 32, 12,
																32, 31, 11,
																31, 30, 10,
																30, 29, 9,
																29, 28, 8,
																28, 27, 7,
																27, 26, 6,
																26, 25, 5,
																25, 24, 4,
																24, 23, 3,
																23, 22, 2,
																22, 21, 1,
																21, 20, 0,
																100, 101, 119,
																101, 102, 119,
																102, 118, 119,
																102, 103, 118,
																103, 117, 118,
																103, 104, 117,
																104, 116, 117,
																104, 105, 116,
																105, 115, 116,
																105, 106, 115,
																106, 114, 115,
																106, 107, 114,
																107, 113, 114,
																107, 108, 113,
																108, 112, 113,
																108, 109, 112,
																109, 111, 112,
																19, 18, 0,
																18, 17, 0,
																17, 1, 0,
																17, 16, 1,
																16, 2, 1,
																16, 15, 2,
																15, 3, 2,
																15, 14, 3,
																14, 4, 3,
																14, 13, 4,
																13, 5, 4,
																13, 12, 5,
																12, 6, 5,
																12, 11, 6,
																11, 7, 6,
																11, 10, 7,
																10, 8, 7
	};
}