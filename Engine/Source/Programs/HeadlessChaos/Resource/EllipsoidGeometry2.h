// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/Real.h"
#include "Chaos/Array.h"
namespace GeometryCollectionTest
{
	class EllipsoidGeometry2
	{

	public:
		EllipsoidGeometry2() {}
		~EllipsoidGeometry2() {}

		static const TArray<float>	RawVertexArray;
		static const TArray<int32>	RawIndicesArray;
	};
}