// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Resource/EllipsoidGeometry3.h"
namespace GeometryCollectionTest
{
	const TArray<float> EllipsoidGeometry3::RawVertexArray = {
																0.000000f, 0.000000f, -250.000000f,
																0.000000f, 21.962393f, -243.896149f,
																31.331211f, 6.786763f, -243.896179f,
																0.000000f, 42.852455f, -225.882553f,
																33.167919f, 30.434498f, -231.635529f,
																61.132645f, 13.242155f, -225.882553f,
																0.000000f, 61.650002f, -196.838699f,
																33.754868f, 51.224689f, -207.208115f,
																62.645508f, 37.231178f, -207.208115f,
																87.948944f, 19.050928f, -196.838715f,
																0.000000f, 77.437073f, -158.183014f,
																33.167923f, 69.261757f, -171.644058f,
																62.645519f, 57.482807f, -175.917664f,
																88.558289f, 42.432804f, -171.644073f,
																110.470528f, 23.929405f, -158.183014f,
																0.000000f, 89.442734f, -111.803284f,
																31.331219f, 84.223846f, -124.249191f,
																61.132652f, 74.892166f, -130.627930f,
																87.948944f, 61.903389f, -130.627930f,
																110.470528f, 45.891808f, -124.249191f,
																127.597626f, 27.639368f, -111.803299f,
																19.363752f, -17.767946f, -243.896179f,
																53.666817f, -11.624928f, -231.635529f,
																37.782051f, -34.668358f, -225.882553f,
																83.507187f, -5.572543f, -207.208176f,
																72.471931f, -28.214546f, -207.208176f,
																54.355442f, -49.875896f, -196.838745f,
																109.057220f, 0.373371f, -171.644119f,
																101.362587f, -21.956450f, -175.917679f,
																87.899979f, -43.036842f, -171.644119f,
																68.274551f, -62.647907f, -158.183044f,
																129.834320f, 6.161456f, -124.249229f,
																125.731049f, -15.617439f, -130.627975f,
																115.488129f, -36.633751f, -130.627975f,
																99.605789f, -55.861145f, -124.249229f,
																78.859680f, -72.360695f, -111.803314f,
																-19.363752f, -17.767946f, -243.896179f,
																0.000000f, -37.619091f, -231.635529f,
																-37.782051f, -34.668358f, -225.882553f,
																17.855400f, -54.668720f, -207.208176f,
																-17.855400f, -54.668720f, -207.208176f,
																-54.355442f, -49.875896f, -196.838745f,
																34.233131f, -69.031013f, -171.644119f,
																0.000000f, -71.052620f, -175.917664f,
																-34.233131f, -69.031013f, -171.644119f,
																-68.274551f, -62.647907f, -158.183044f,
																48.910797f, -80.415871f, -124.249229f,
																16.573383f, -84.544273f, -130.627960f,
																-16.573383f, -84.544273f, -130.627960f,
																-48.910797f, -80.415871f, -124.249229f,
																-78.859680f, -72.360695f, -111.803314f,
																-31.331211f, 6.786763f, -243.896179f,
																-53.666817f, -11.624928f, -231.635529f,
																-61.132645f, 13.242155f, -225.882553f,
																-72.471931f, -28.214546f, -207.208176f,
																-83.507187f, -5.572543f, -207.208176f,
																-87.948944f, 19.050928f, -196.838715f,
																-87.899979f, -43.036842f, -171.644119f,
																-101.362587f, -21.956450f, -175.917679f,
																-109.057220f, 0.373371f, -171.644119f,
																-110.470528f, 23.929405f, -158.183014f,
																-99.605789f, -55.861145f, -124.249229f,
																-115.488129f, -36.633751f, -130.627975f,
																-125.731049f, -15.617439f, -130.627975f,
																-129.834320f, 6.161456f, -124.249229f,
																-127.597626f, 27.639368f, -111.803299f,
																-33.167919f, 30.434498f, -231.635529f,
																-62.645508f, 37.231178f, -207.208115f,
																-33.754868f, 51.224689f, -207.208115f,
																-88.558289f, 42.432804f, -171.644073f,
																-62.645519f, 57.482807f, -175.917664f,
																-33.167923f, 69.261757f, -171.644058f,
																-110.470528f, 45.891808f, -124.249191f,
																-87.948944f, 61.903389f, -130.627930f,
																-61.132652f, 74.892166f, -130.627930f,
																-31.331219f, 84.223846f, -124.249191f,
																-19.363754f, 95.205017f, -69.343246f,
																-53.666817f, 88.071297f, -77.596321f,
																-37.782051f, 96.318359f, -23.496904f,
																-83.507156f, 76.625198f, -80.205582f,
																-72.471924f, 86.750999f, -29.576611f,
																-54.355431f, 92.728340f, 23.496904f,
																-109.057175f, 61.242340f, -77.596321f,
																-101.362549f, 72.757484f, -29.576611f,
																-87.899948f, 80.655952f, 19.471691f,
																-68.274536f, 84.610298f, 69.343254f,
																-129.834290f, 41.697357f, -69.343254f,
																-125.730988f, 53.719284f, -23.496904f,
																-115.488075f, 63.118046f, 23.496904f,
																-99.605759f, 69.434662f, 69.343254f,
																-78.859680f, 72.360695f, 111.803314f,
																-141.801758f, 17.142641f, -69.343254f,
																-142.225113f, -6.811248f, -77.596336f,
																-149.081604f, 5.808771f, -23.496910f,
																-135.117477f, -29.268211f, -80.205620f,
																-146.152695f, -19.142406f, -29.576614f,
																-149.081604f, -5.808771f, 23.496910f,
																-121.067886f, -50.221455f, -77.596336f,
																-135.117447f, -41.784412f, -29.576618f,
																-142.225098f, -30.807846f, 19.471691f,
																-141.801758f, -17.142641f, 69.343254f,
																-99.605759f, -69.434662f, -69.343254f,
																-115.488075f, -63.118046f, -23.496904f,
																-125.730988f, -53.719284f, 23.496904f,
																-129.834290f, -41.697357f, 69.343254f,
																-127.597626f, -27.639368f, 111.803299f,
																-68.274536f, -84.610298f, -69.343254f,
																-34.233124f, -92.280876f, -77.596329f,
																-54.355431f, -92.728340f, -23.496904f,
																0.000000f, -94.713943f, -80.205605f,
																-17.855396f, -98.581642f, -29.576611f,
																-37.782051f, -96.318359f, 23.496904f,
																34.233124f, -92.280876f, -77.596329f,
																17.855396f, -98.581642f, -29.576611f,
																0.000000f, -99.696220f, 19.471687f,
																-19.363754f, -95.205017f, 69.343246f,
																68.274536f, -84.610298f, -69.343254f,
																54.355431f, -92.728340f, -23.496904f,
																37.782051f, -96.318359f, 23.496904f,
																19.363754f, -95.205017f, 69.343246f,
																0.000000f, -89.442734f, 111.803284f,
																99.605759f, -69.434662f, -69.343254f,
																121.067886f, -50.221455f, -77.596336f,
																115.488075f, -63.118046f, -23.496904f,
																135.117477f, -29.268211f, -80.205620f,
																135.117447f, -41.784412f, -29.576618f,
																125.730988f, -53.719284f, 23.496904f,
																142.225113f, -6.811248f, -77.596336f,
																146.152695f, -19.142406f, -29.576614f,
																142.225098f, -30.807846f, 19.471691f,
																129.834290f, -41.697357f, 69.343254f,
																141.801758f, 17.142641f, -69.343254f,
																149.081604f, 5.808771f, -23.496910f,
																149.081604f, -5.808771f, 23.496910f,
																141.801758f, -17.142641f, 69.343254f,
																127.597626f, -27.639368f, 111.803299f,
																129.834290f, 41.697357f, -69.343254f,
																109.057175f, 61.242340f, -77.596321f,
																125.730988f, 53.719284f, -23.496904f,
																83.507156f, 76.625198f, -80.205582f,
																101.362549f, 72.757484f, -29.576611f,
																115.488075f, 63.118046f, 23.496904f,
																53.666817f, 88.071297f, -77.596321f,
																72.471924f, 86.750999f, -29.576611f,
																87.899948f, 80.655952f, 19.471691f,
																99.605759f, 69.434662f, 69.343254f,
																19.363754f, 95.205017f, -69.343246f,
																37.782051f, 96.318359f, -23.496904f,
																54.355431f, 92.728340f, 23.496904f,
																68.274536f, 84.610298f, 69.343254f,
																78.859680f, 72.360695f, 111.803314f,
																0.000000f, 99.696220f, -19.471687f,
																-17.855396f, 98.581635f, 29.576611f,
																17.855396f, 98.581642f, 29.576611f,
																-34.233124f, 92.280876f, 77.596329f,
																0.000000f, 94.713943f, 80.205605f,
																34.233124f, 92.280876f, 77.596329f,
																-48.910797f, 80.415871f, 124.249229f,
																-16.573383f, 84.544273f, 130.627960f,
																16.573383f, 84.544273f, 130.627960f,
																48.910797f, 80.415871f, 124.249229f,
																-142.225098f, 30.807846f, -19.471691f,
																-146.152695f, 19.142406f, 29.576618f,
																-135.117432f, 41.784401f, 29.576611f,
																-142.225128f, 6.811248f, 77.596344f,
																-135.117477f, 29.268208f, 80.205612f,
																-121.067886f, 50.221455f, 77.596336f,
																-129.834320f, -6.161456f, 124.249229f,
																-125.731049f, 15.617439f, 130.627975f,
																-115.488129f, 36.633751f, 130.627975f,
																-99.605789f, 55.861145f, 124.249229f,
																-87.899940f, -80.655945f, -19.471687f,
																-72.471924f, -86.750999f, 29.576611f,
																-101.362549f, -72.757484f, 29.576611f,
																-53.666817f, -88.071297f, 77.596321f,
																-83.507156f, -76.625198f, 80.205582f,
																-109.057175f, -61.242340f, 77.596321f,
																-31.331219f, -84.223846f, 124.249191f,
																-61.132652f, -74.892166f, 130.627930f,
																-87.948944f, -61.903389f, 130.627930f,
																-110.470528f, -45.891808f, 124.249191f,
																87.899948f, -80.655952f, -19.471691f,
																101.362549f, -72.757484f, 29.576611f,
																72.471924f, -86.750999f, 29.576611f,
																109.057175f, -61.242340f, 77.596321f,
																83.507156f, -76.625198f, 80.205582f,
																53.666817f, -88.071297f, 77.596321f,
																110.470528f, -45.891808f, 124.249191f,
																87.948944f, -61.903389f, 130.627930f,
																61.132652f, -74.892166f, 130.627930f,
																31.331219f, -84.223846f, 124.249191f,
																142.225113f, 30.807844f, -19.471691f,
																135.117447f, 41.784412f, 29.576618f,
																146.152695f, 19.142406f, 29.576614f,
																121.067886f, 50.221455f, 77.596336f,
																135.117477f, 29.268211f, 80.205620f,
																142.225128f, 6.811248f, 77.596344f,
																99.605789f, 55.861145f, 124.249229f,
																115.488129f, 36.633751f, 130.627975f,
																125.731049f, 15.617439f, 130.627975f,
																129.834320f, -6.161456f, 124.249229f,
																0.000000f, 0.000000f, 250.000000f,
																19.363752f, 17.767946f, 243.896179f,
																-19.363752f, 17.767946f, 243.896179f,
																37.782051f, 34.668358f, 225.882553f,
																0.000000f, 37.619091f, 231.635529f,
																-37.782051f, 34.668358f, 225.882553f,
																54.355442f, 49.875896f, 196.838745f,
																17.855400f, 54.668720f, 207.208176f,
																-17.855400f, 54.668720f, 207.208176f,
																-54.355442f, 49.875896f, 196.838745f,
																68.274551f, 62.647907f, 158.183044f,
																34.233131f, 69.031013f, 171.644119f,
																0.000000f, 71.052620f, 175.917664f,
																-34.233131f, 69.031013f, 171.644119f,
																-68.274551f, 62.647907f, 158.183044f,
																31.331211f, -6.786763f, 243.896179f,
																61.132645f, -13.242155f, 225.882553f,
																53.666817f, 11.624928f, 231.635529f,
																87.948944f, -19.050928f, 196.838715f,
																83.507187f, 5.572543f, 207.208176f,
																72.471931f, 28.214546f, 207.208176f,
																110.470528f, -23.929405f, 158.183014f,
																109.057220f, -0.373371f, 171.644119f,
																101.362587f, 21.956450f, 175.917679f,
																87.899979f, 43.036842f, 171.644119f,
																0.000000f, -21.962393f, 243.896149f,
																0.000000f, -42.852455f, 225.882553f,
																33.167919f, -30.434498f, 231.635529f,
																0.000000f, -61.650002f, 196.838699f,
																33.754868f, -51.224689f, 207.208115f,
																62.645508f, -37.231178f, 207.208115f,
																0.000000f, -77.437073f, 158.183014f,
																33.167923f, -69.261757f, 171.644058f,
																62.645519f, -57.482807f, 175.917664f,
																88.558289f, -42.432804f, 171.644073f,
																-31.331211f, -6.786763f, 243.896179f,
																-61.132645f, -13.242155f, 225.882553f,
																-33.167919f, -30.434498f, 231.635529f,
																-87.948944f, -19.050928f, 196.838715f,
																-62.645508f, -37.231178f, 207.208115f,
																-33.754868f, -51.224689f, 207.208115f,
																-110.470528f, -23.929405f, 158.183014f,
																-88.558289f, -42.432804f, 171.644073f,
																-62.645519f, -57.482807f, 175.917664f,
																-33.167923f, -69.261757f, 171.644058f,
																-53.666817f, 11.624928f, 231.635529f,
																-72.471931f, 28.214546f, 207.208176f,
																-83.507187f, 5.572543f, 207.208176f,
																-87.899979f, 43.036842f, 171.644119f,
																-101.362587f, 21.956450f, 175.917679f,
																-109.057220f, -0.373371f, 171.644119f
	};

	const TArray<int32> EllipsoidGeometry3::RawIndicesArray = {
																2, 1, 0,
																4, 3, 1,
																5, 4, 2,
																7, 6, 3,
																8, 7, 4,
																9, 8, 5,
																11, 10, 6,
																12, 11, 7,
																13, 12, 8,
																14, 13, 9,
																16, 15, 10,
																17, 16, 11,
																18, 17, 12,
																19, 18, 13,
																20, 19, 14,
																2, 4, 1,
																4, 7, 3,
																5, 8, 4,
																7, 11, 6,
																8, 12, 7,
																9, 13, 8,
																11, 16, 10,
																12, 17, 11,
																13, 18, 12,
																14, 19, 13,
																21, 2, 0,
																22, 5, 2,
																23, 22, 21,
																24, 9, 5,
																25, 24, 22,
																26, 25, 23,
																27, 14, 9,
																28, 27, 24,
																29, 28, 25,
																30, 29, 26,
																31, 20, 14,
																32, 31, 27,
																33, 32, 28,
																34, 33, 29,
																35, 34, 30,
																21, 22, 2,
																22, 24, 5,
																23, 25, 22,
																24, 27, 9,
																25, 28, 24,
																26, 29, 25,
																27, 31, 14,
																28, 32, 27,
																29, 33, 28,
																30, 34, 29,
																36, 21, 0,
																37, 23, 21,
																38, 37, 36,
																39, 26, 23,
																40, 39, 37,
																41, 40, 38,
																42, 30, 26,
																43, 42, 39,
																44, 43, 40,
																45, 44, 41,
																46, 35, 30,
																47, 46, 42,
																48, 47, 43,
																49, 48, 44,
																50, 49, 45,
																36, 37, 21,
																37, 39, 23,
																38, 40, 37,
																39, 42, 26,
																40, 43, 39,
																41, 44, 40,
																42, 46, 30,
																43, 47, 42,
																44, 48, 43,
																45, 49, 44,
																51, 36, 0,
																52, 38, 36,
																53, 52, 51,
																54, 41, 38,
																55, 54, 52,
																56, 55, 53,
																57, 45, 41,
																58, 57, 54,
																59, 58, 55,
																60, 59, 56,
																61, 50, 45,
																62, 61, 57,
																63, 62, 58,
																64, 63, 59,
																65, 64, 60,
																51, 52, 36,
																52, 54, 38,
																53, 55, 52,
																54, 57, 41,
																55, 58, 54,
																56, 59, 55,
																57, 61, 45,
																58, 62, 57,
																59, 63, 58,
																60, 64, 59,
																1, 51, 0,
																66, 53, 51,
																3, 66, 1,
																67, 56, 53,
																68, 67, 66,
																6, 68, 3,
																69, 60, 56,
																70, 69, 67,
																71, 70, 68,
																10, 71, 6,
																72, 65, 60,
																73, 72, 69,
																74, 73, 70,
																75, 74, 71,
																15, 75, 10,
																1, 66, 51,
																66, 67, 53,
																3, 68, 66,
																67, 69, 56,
																68, 70, 67,
																6, 71, 68,
																69, 72, 60,
																70, 73, 69,
																71, 74, 70,
																10, 75, 71,
																76, 75, 15,
																77, 74, 75,
																78, 77, 76,
																79, 73, 74,
																80, 79, 77,
																81, 80, 78,
																82, 72, 73,
																83, 82, 79,
																84, 83, 80,
																85, 84, 81,
																86, 65, 72,
																87, 86, 82,
																88, 87, 83,
																89, 88, 84,
																90, 89, 85,
																76, 77, 75,
																77, 79, 74,
																78, 80, 77,
																79, 82, 73,
																80, 83, 79,
																81, 84, 80,
																82, 86, 72,
																83, 87, 82,
																84, 88, 83,
																85, 89, 84,
																91, 64, 65,
																92, 63, 64,
																93, 92, 91,
																94, 62, 63,
																95, 94, 92,
																96, 95, 93,
																97, 61, 62,
																98, 97, 94,
																99, 98, 95,
																100, 99, 96,
																101, 50, 61,
																102, 101, 97,
																103, 102, 98,
																104, 103, 99,
																105, 104, 100,
																91, 92, 64,
																92, 94, 63,
																93, 95, 92,
																94, 97, 62,
																95, 98, 94,
																96, 99, 95,
																97, 101, 61,
																98, 102, 97,
																99, 103, 98,
																100, 104, 99,
																106, 49, 50,
																107, 48, 49,
																108, 107, 106,
																109, 47, 48,
																110, 109, 107,
																111, 110, 108,
																112, 46, 47,
																113, 112, 109,
																114, 113, 110,
																115, 114, 111,
																116, 35, 46,
																117, 116, 112,
																118, 117, 113,
																119, 118, 114,
																120, 119, 115,
																106, 107, 49,
																107, 109, 48,
																108, 110, 107,
																109, 112, 47,
																110, 113, 109,
																111, 114, 110,
																112, 116, 46,
																113, 117, 112,
																114, 118, 113,
																115, 119, 114,
																121, 34, 35,
																122, 33, 34,
																123, 122, 121,
																124, 32, 33,
																125, 124, 122,
																126, 125, 123,
																127, 31, 32,
																128, 127, 124,
																129, 128, 125,
																130, 129, 126,
																131, 20, 31,
																132, 131, 127,
																133, 132, 128,
																134, 133, 129,
																135, 134, 130,
																121, 122, 34,
																122, 124, 33,
																123, 125, 122,
																124, 127, 32,
																125, 128, 124,
																126, 129, 125,
																127, 131, 31,
																128, 132, 127,
																129, 133, 128,
																130, 134, 129,
																136, 19, 20,
																137, 18, 19,
																138, 137, 136,
																139, 17, 18,
																140, 139, 137,
																141, 140, 138,
																142, 16, 17,
																143, 142, 139,
																144, 143, 140,
																145, 144, 141,
																146, 15, 16,
																147, 146, 142,
																148, 147, 143,
																149, 148, 144,
																150, 149, 145,
																136, 137, 19,
																137, 139, 18,
																138, 140, 137,
																139, 142, 17,
																140, 143, 139,
																141, 144, 140,
																142, 146, 16,
																143, 147, 142,
																144, 148, 143,
																145, 149, 144,
																146, 76, 15,
																151, 78, 76,
																147, 151, 146,
																152, 81, 78,
																153, 152, 151,
																148, 153, 147,
																154, 85, 81,
																155, 154, 152,
																156, 155, 153,
																149, 156, 148,
																157, 90, 85,
																158, 157, 154,
																159, 158, 155,
																160, 159, 156,
																150, 160, 149,
																146, 151, 76,
																151, 152, 78,
																147, 153, 151,
																152, 154, 81,
																153, 155, 152,
																148, 156, 153,
																154, 157, 85,
																155, 158, 154,
																156, 159, 155,
																149, 160, 156,
																86, 91, 65,
																161, 93, 91,
																87, 161, 86,
																162, 96, 93,
																163, 162, 161,
																88, 163, 87,
																164, 100, 96,
																165, 164, 162,
																166, 165, 163,
																89, 166, 88,
																167, 105, 100,
																168, 167, 164,
																169, 168, 165,
																170, 169, 166,
																90, 170, 89,
																86, 161, 91,
																161, 162, 93,
																87, 163, 161,
																162, 164, 96,
																163, 165, 162,
																88, 166, 163,
																164, 167, 100,
																165, 168, 164,
																166, 169, 165,
																89, 170, 166,
																101, 106, 50,
																171, 108, 106,
																102, 171, 101,
																172, 111, 108,
																173, 172, 171,
																103, 173, 102,
																174, 115, 111,
																175, 174, 172,
																176, 175, 173,
																104, 176, 103,
																177, 120, 115,
																178, 177, 174,
																179, 178, 175,
																180, 179, 176,
																105, 180, 104,
																101, 171, 106,
																171, 172, 108,
																102, 173, 171,
																172, 174, 111,
																173, 175, 172,
																103, 176, 173,
																174, 177, 115,
																175, 178, 174,
																176, 179, 175,
																104, 180, 176,
																116, 121, 35,
																181, 123, 121,
																117, 181, 116,
																182, 126, 123,
																183, 182, 181,
																118, 183, 117,
																184, 130, 126,
																185, 184, 182,
																186, 185, 183,
																119, 186, 118,
																187, 135, 130,
																188, 187, 184,
																189, 188, 185,
																190, 189, 186,
																120, 190, 119,
																116, 181, 121,
																181, 182, 123,
																117, 183, 181,
																182, 184, 126,
																183, 185, 182,
																118, 186, 183,
																184, 187, 130,
																185, 188, 184,
																186, 189, 185,
																119, 190, 186,
																131, 136, 20,
																191, 138, 136,
																132, 191, 131,
																192, 141, 138,
																193, 192, 191,
																133, 193, 132,
																194, 145, 141,
																195, 194, 192,
																196, 195, 193,
																134, 196, 133,
																197, 150, 145,
																198, 197, 194,
																199, 198, 195,
																200, 199, 196,
																135, 200, 134,
																131, 191, 136,
																191, 192, 138,
																132, 193, 191,
																192, 194, 141,
																193, 195, 192,
																133, 196, 193,
																194, 197, 145,
																195, 198, 194,
																196, 199, 195,
																134, 200, 196,
																203, 202, 201,
																205, 204, 202,
																206, 205, 203,
																208, 207, 204,
																209, 208, 205,
																210, 209, 206,
																212, 211, 207,
																213, 212, 208,
																214, 213, 209,
																215, 214, 210,
																160, 150, 211,
																159, 160, 212,
																158, 159, 213,
																157, 158, 214,
																90, 157, 215,
																203, 205, 202,
																205, 208, 204,
																206, 209, 205,
																208, 212, 207,
																209, 213, 208,
																210, 214, 209,
																212, 160, 211,
																213, 159, 212,
																214, 158, 213,
																215, 157, 214,
																202, 216, 201,
																218, 217, 216,
																204, 218, 202,
																220, 219, 217,
																221, 220, 218,
																207, 221, 204,
																223, 222, 219,
																224, 223, 220,
																225, 224, 221,
																211, 225, 207,
																200, 135, 222,
																199, 200, 223,
																198, 199, 224,
																197, 198, 225,
																150, 197, 211,
																202, 218, 216,
																218, 220, 217,
																204, 221, 218,
																220, 223, 219,
																221, 224, 220,
																207, 225, 221,
																223, 200, 222,
																224, 199, 223,
																225, 198, 224,
																211, 197, 225,
																216, 226, 201,
																228, 227, 226,
																217, 228, 216,
																230, 229, 227,
																231, 230, 228,
																219, 231, 217,
																233, 232, 229,
																234, 233, 230,
																235, 234, 231,
																222, 235, 219,
																190, 120, 232,
																189, 190, 233,
																188, 189, 234,
																187, 188, 235,
																135, 187, 222,
																216, 228, 226,
																228, 230, 227,
																217, 231, 228,
																230, 233, 229,
																231, 234, 230,
																219, 235, 231,
																233, 190, 232,
																234, 189, 233,
																235, 188, 234,
																222, 187, 235,
																226, 236, 201,
																238, 237, 236,
																227, 238, 226,
																240, 239, 237,
																241, 240, 238,
																229, 241, 227,
																243, 242, 239,
																244, 243, 240,
																245, 244, 241,
																232, 245, 229,
																180, 105, 242,
																179, 180, 243,
																178, 179, 244,
																177, 178, 245,
																120, 177, 232,
																226, 238, 236,
																238, 240, 237,
																227, 241, 238,
																240, 243, 239,
																241, 244, 240,
																229, 245, 241,
																243, 180, 242,
																244, 179, 243,
																245, 178, 244,
																232, 177, 245,
																236, 203, 201,
																246, 206, 203,
																237, 246, 236,
																247, 210, 206,
																248, 247, 246,
																239, 248, 237,
																249, 215, 210,
																250, 249, 247,
																251, 250, 248,
																242, 251, 239,
																170, 90, 215,
																169, 170, 249,
																168, 169, 250,
																167, 168, 251,
																105, 167, 242,
																236, 246, 203,
																246, 247, 206,
																237, 248, 246,
																247, 249, 210,
																248, 250, 247,
																239, 251, 248,
																249, 170, 215,
																250, 169, 249,
																251, 168, 250,
																242, 167, 251
	};

}