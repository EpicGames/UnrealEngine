// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Resource/FracturedGeometry.h"
#include "Chaos/Box.h"
#include "Math/RandomStream.h"

namespace GeometryCollectionTest 
{

	const TArray<float> FracturedGeometry::RawVertexArray = {
													50, 0, -50,
													50, 50, -50,
													0, 50, -50,
													0, 0, -50,
													-50, 50, -50,
													50, -50, -50,
													-50, 0, -50,
													0, -50, -50,
													-50, -50, -50,
													50, 50, 50,
													50, 1.22465e-14f, 50,
													0, 50, 50,
													0, 1.22465e-14f, 50,
													50, -50, 50,
													-50, 50, 50,
													0, -50, 50,
													-50, 1.22465e-14f, 50,
													-50, -50, 50,
													50, 50, -50,
													50, 50, -3.06162e-15f,
													0, 50, -50,
													0, 50, -3.06162e-15f,
													50, 50, 50,
													-50, 50, -50,
													0, 50, 50,
													-50, 50, -3.06162e-15f,
													-50, 50, 50,
													50, -50, -50,
													50, -50, 3.06162e-15f,
													50, 0, -50,
													50, 6.12323e-15f, -1.8747e-31f,
													50, -50, 50,
													50, 50, -50,
													50, 1.22465e-14f, 50,
													50, 50, -3.06162e-15f,
													50, 50, 50,
													-50, -50, -50,
													-50, -50, 3.06162e-15f,
													0, -50, -50,
													0, -50, 3.06162e-15f,
													-50, -50, 50,
													50, -50, -50,
													0, -50, 50,
													50, -50, 3.06162e-15f,
													50, -50, 50,
													-50, 50, -50,
													-50, 50, -3.06162e-15f,
													-50, 0, -50,
													-50, 6.12323e-15f, -1.8747e-31f,
													-50, 50, 50,
													-50, -50, -50,
													-50, 1.22465e-14f, 50,
													-50, -50, 3.06162e-15f,
													-50, -50, 50,
													16.2172f, 8.6841f, 15.7825f,
													-10.3678f, 35.2691f, 15.7825f,
													24.0997f, 35.2691f, 15.7825f,
													9.27464f, -14.7309f, 15.7825f,
													-10.3678f, -14.7309f, 15.7825f,
													3.62653f, -28.7252f, 15.7825f,
													5.3612f, -27.9296f, 15.7825f,
													-10.3678f, -35.1439f, 15.7825f,
													15.9054f, 35.2691f, -7.94424f,
													2.08283f, 35.2691f, -21.7668f,
													11.8691f, 35.2691f, -19.6313f,
													-10.3678f, 35.2691f, 15.7825f,
													-10.3678f, 35.2691f, -24.4836f,
													24.0997f, 35.2691f, 15.7825f,
													-10.3678f, 35.2691f, 15.7825f,
													-10.3678f, 2.16495f, -17.3216f,
													-10.3678f, 35.2691f, -24.4836f,
													-10.3678f, -14.7309f, 15.7825f,
													-10.3678f, -14.7309f, -13.6663f,
													-10.3678f, -33.1022f, -2.58878f,
													-10.3678f, -32.2934f, -9.86669f,
													-10.3678f, -35.1439f, 15.7825f,
													-3.61541f, -29.2878f, -9.04352f,
													11.8691f, 35.2691f, -19.6313f,
													-10.3678f, -32.2934f, -9.86669f,
													-10.3678f, -14.7309f, -13.6663f,
													-10.3678f, 2.16495f, -17.3216f,
													-10.3678f, 35.2691f, -24.4836f,
													2.08283f, 35.2691f, -21.7668f,
													5.3612f, -27.9296f, 15.7825f,
													9.27464f, -14.7309f, 15.7825f,
													-3.61541f, -29.2878f, -9.04352f,
													11.8691f, 35.2691f, -19.6313f,
													15.9054f, 35.2691f, -7.94424f,
													24.0997f, 35.2691f, 15.7825f,
													16.2172f, 8.6841f, 15.7825f,
													-10.3678f, -35.1439f, 15.7825f,
													3.62653f, -28.7252f, 15.7825f,
													-10.3678f, -33.1022f, -2.58878f,
													-10.3678f, -32.2934f, -9.86669f,
													-3.61541f, -29.2878f, -9.04352f,
													5.3612f, -27.9296f, 15.7825f,
													-0.0144844f, -15.9719f, -42.5147f,
													-4.03455f, 9.82732f, -42.5147f,
													-27.0941f, -15.9719f, -42.5147f,
													-27.0941f, 9.35167f, -42.5147f,
													-14.633f, 13.3885f, -42.5147f,
													-27.0941f, -15.9719f, -42.5147f,
													-27.0941f, -15.9719f, 7.4853f,
													-0.0144844f, -15.9719f, -42.5147f,
													8.77472f, -15.9719f, -28.3835f,
													22.9059f, -15.9719f, -5.6636f,
													22.9059f, -15.9719f, 7.4853f,
													6.25886f, -15.9719f, 24.1323f,
													-27.0941f, -15.9719f, 29.5568f,
													27.9481f, -15.9719f, 2.44314f,
													22.9059f, -15.9719f, 21.4249f,
													31.0841f, -15.9719f, 7.4853f,
													38.2065f, -15.9719f, 18.9365f,
													-27.0941f, 9.35167f, -42.5147f,
													-27.0941f, 11.456f, -15.0869f,
													-27.0941f, -15.9719f, -42.5147f,
													-27.0941f, -15.9719f, 7.4853f,
													-27.0941f, 13.1877f, 7.4853f,
													-27.0941f, 5.07368f, 28.5308f,
													-27.0941f, 14.7661f, 28.0583f,
													-27.0941f, -15.9719f, 29.5568f,
													-27.0941f, 14.7661f, 28.0583f,
													-27.0941f, 13.1877f, 7.4853f,
													-11.9324f, 19.4711f, 25.3631f,
													-27.0941f, 11.456f, -15.0869f,
													-2.6789f, 21.8609f, 17.4398f,
													-14.633f, 13.3885f, -42.5147f,
													-27.0941f, 9.35167f, -42.5147f,
													-0.0144844f, -15.9719f, -42.5147f,
													8.77472f, -15.9719f, -28.3835f,
													-4.03455f, 9.82732f, -42.5147f,
													22.9059f, -15.9719f, -5.6636f,
													27.9481f, -15.9719f, 2.44314f,
													31.0841f, -15.9719f, 7.4853f,
													20.3276f, 9.79925f, -3.35256f,
													36.7772f, -7.517f, 18.7568f,
													38.2065f, -15.9719f, 18.9365f,
													-2.6789f, 21.8609f, 17.4398f,
													-14.633f, 13.3885f, -42.5147f,
													5.96455f, 19.9955f, 22.4268f,
													25.0912f, 12.9919f, 19.6575f,
													20.3276f, 9.79925f, -3.35256f,
													-4.03455f, 9.82732f, -42.5147f,
													-2.6789f, 21.8609f, 17.4398f,
													5.96455f, 19.9955f, 22.4268f,
													-11.9324f, 19.4711f, 25.3631f,
													20.3276f, 9.79925f, -3.35256f,
													36.7772f, -7.517f, 18.7568f,
													25.0912f, 12.9919f, 19.6575f,
													-27.0941f, -15.9719f, 29.5568f,
													-27.0941f, 5.07368f, 28.5308f,
													6.25886f, -15.9719f, 24.1323f,
													22.9059f, -15.9719f, 21.4249f,
													38.2065f, -15.9719f, 18.9365f,
													36.7772f, -7.517f, 18.7568f,
													25.0912f, 12.9919f, 19.6575f,
													5.96455f, 19.9955f, 22.4268f,
													-11.9324f, 19.4711f, 25.3631f,
													-27.0941f, 14.7661f, 28.0583f,
													7.54969f, 9.40126f, -54.4615f,
													16.0109f, 4.613f, -54.4615f,
													29.9135f, 9.40126f, -54.4615f,
													29.9135f, 7.20573f, -54.4615f,
													29.9135f, -25.0935f, 45.5385f,
													16.8454f, -27.5306f, 45.5385f,
													29.9135f, 9.40126f, 45.5385f,
													-20.0865f, 9.40126f, 45.5385f,
													-20.0865f, -0.546741f, 45.5385f,
													2.03889f, -30.2919f, 45.5385f,
													-27.4861f, 9.40126f, 45.5385f,
													7.54969f, 9.40126f, -54.4615f,
													29.9135f, 9.40126f, -54.4615f,
													-6.91031f, 9.40126f, -41.2853f,
													29.9135f, 9.40126f, -4.4615f,
													-20.0865f, 9.40126f, -4.4615f,
													-20.0865f, 9.40126f, -29.279f,
													29.9135f, 9.40126f, 45.5385f,
													-30.7467f, 9.40126f, -15.1217f,
													-28.7678f, 9.40126f, -21.3684f,
													-20.0865f, 9.40126f, 45.5385f,
													-34.1239f, 9.40126f, -4.4615f,
													-35.2935f, 9.40126f, -0.769467f,
													-28.9867f, 9.40126f, 36.6383f,
													-27.4861f, 9.40126f, 45.5385f,
													29.9135f, 9.40126f, -54.4615f,
													29.9135f, 7.20573f, -54.4615f,
													29.9135f, 6.15827f, -51.2185f,
													29.9135f, -8.94388f, -4.4615f,
													29.9135f, 9.40126f, -4.4615f,
													29.9135f, -17.6961f, 22.6359f,
													29.9135f, 9.40126f, 45.5385f,
													29.9135f, -25.0935f, 45.5385f,
													-28.7678f, 9.40126f, -21.3684f,
													-30.7467f, 9.40126f, -15.1217f,
													-29.5054f, 0.391495f, -6.18885f,
													-34.1239f, 9.40126f, -4.4615f,
													-35.2935f, 9.40126f, -0.769467f,
													29.9135f, -25.0935f, 45.5385f,
													29.9135f, -17.6961f, 22.6359f,
													16.8454f, -27.5306f, 45.5385f,
													2.03889f, -30.2919f, 45.5385f,
													-10.1522f, -21.5984f, 11.5841f,
													16.0109f, 4.613f, -54.4615f,
													29.9135f, 7.20573f, -54.4615f,
													29.9135f, 6.15827f, -51.2185f,
													29.9135f, -8.94388f, -4.4615f,
													16.0109f, 4.613f, -54.4615f,
													7.54969f, 9.40126f, -54.4615f,
													-10.1522f, -21.5984f, 11.5841f,
													-29.5054f, 0.391495f, -6.18885f,
													-28.7678f, 9.40126f, -21.3684f,
													-35.2935f, 9.40126f, -0.769467f,
													-28.9867f, 9.40126f, 36.6383f,
													-29.5054f, 0.391495f, -6.18885f,
													-10.1522f, -21.5984f, 11.5841f,
													2.03889f, -30.2919f, 45.5385f,
													-20.0865f, -0.546741f, 45.5385f,
													-27.4861f, 9.40126f, 45.5385f,
													12.2185f, 7.69071f, -40.3769f,
													21.3769f, 34.8307f, -40.3769f,
													-14.9215f, 34.8307f, -40.3769f,
													4.50432f, -15.1693f, -40.3769f,
													-14.9215f, -15.1693f, -40.3769f,
													-0.397007f, -29.6938f, -40.3769f,
													-14.9215f, -39.8458f, -40.3769f,
													-2.46044f, -35.8089f, -40.3769f,
													24.4182f, 34.8307f, -1.03714f,
													-14.9215f, 34.8307f, -40.3769f,
													26.3972f, 34.8307f, -7.28381f,
													21.3769f, 34.8307f, -40.3769f,
													-14.9215f, 34.8307f, 9.62309f,
													21.0411f, 34.8307f, 9.62309f,
													19.8714f, 34.8307f, 13.3151f,
													-2.47086f, 34.8307f, 22.0738f,
													7.31542f, 34.8307f, 24.2092f,
													-14.9215f, 34.8307f, 19.3569f,
													-14.9215f, -15.1693f, -40.3769f,
													-14.9215f, 34.8307f, -40.3769f,
													-14.9215f, 34.8307f, 9.62309f,
													-14.9215f, -15.1693f, 9.62309f,
													-14.9215f, 34.8307f, 19.3569f,
													-14.9215f, 1.72654f, 26.5189f,
													-14.9215f, -37.7415f, -12.9491f,
													-14.9215f, -39.8458f, -40.3769f,
													-14.9215f, -15.1693f, 30.1743f,
													-14.9215f, -36.0097f, 9.62309f,
													-14.9215f, -32.7318f, 33.9739f,
													-14.9215f, -34.4313f, 30.1961f,
													-14.9215f, 34.8307f, 19.3569f,
													-2.4709f, 34.8307f, 22.0737f,
													-14.9215f, 1.72652f, 26.5189f,
													-14.9215f, -15.1693f, 30.1742f,
													-14.9215f, -32.7319f, 33.9739f,
													-8.16909f, -29.7263f, 34.797f,
													7.31557f, 34.8307f, 24.2092f,
													-2.46057f, -35.8089f, -40.3769f,
													-14.9215f, -39.8457f, -40.3769f,
													9.49352f, -27.3366f, 19.5775f,
													0.240276f, -29.7263f, 27.5007f,
													-14.9215f, -34.4313f, 30.1959f,
													-14.9215f, -36.0097f, 9.62309f,
													-14.9215f, -37.7414f, -12.949f,
													19.8715f, 34.8307f, 13.3151f,
													21.0411f, 34.8307f, 9.62309f,
													25.6595f, 25.821f, 7.8958f,
													26.3972f, 34.8307f, -7.28368f,
													24.4183f, 34.8307f, -1.03711f,
													25.6595f, 25.821f, 7.8958f,
													26.3972f, 34.8307f, -7.28368f,
													9.49352f, -27.3366f, 19.5775f,
													-2.46057f, -35.8089f, -40.3769f,
													-0.397007f, -29.6938f, -40.3769f,
													4.50432f, -15.1693f, -40.3769f,
													12.2185f, 7.69071f, -40.3769f,
													21.3769f, 34.8307f, -40.3769f,
													-8.16909f, -29.7263f, 34.797f,
													7.31557f, 34.8307f, 24.2092f,
													0.240276f, -29.7263f, 27.5007f,
													9.49352f, -27.3366f, 19.5775f,
													25.6595f, 25.821f, 7.8958f,
													19.8715f, 34.8307f, 13.3151f,
													0.240276f, -29.7263f, 27.5007f,
													-14.9215f, -34.4313f, 30.1959f,
													-8.16909f, -29.7263f, 34.797f,
													-14.9215f, -32.7319f, 33.9739f,
													34.8575f, -12.9869f, -29.136f,
													34.8575f, 24.5256f, -29.136f,
													-8.70369f, 30.5743f, -29.136f,
													-3.4102f, 33.3911f, -29.136f,
													-15.1425f, 27.148f, -29.136f,
													-15.1425f, -12.9869f, -29.136f,
													-41.338f, 13.2086f, -29.136f,
													-38.0628f, -12.9869f, -29.136f,
													-42.0829f, 12.8122f, -29.136f,
													34.8575f, -12.9869f, 20.864f,
													34.8575f, 4.92442f, 2.9527f,
													34.8575f, -12.9869f, -29.136f,
													34.8575f, 24.5256f, -29.136f,
													34.8575f, -6.01665f, 20.864f,
													34.8575f, -12.9869f, 32.2749f,
													-38.0628f, -12.9869f, -29.136f,
													-29.2736f, -12.9869f, -15.0048f,
													-15.1425f, -12.9869f, -29.136f,
													-15.1425f, -12.9869f, 7.71515f,
													-10.1003f, -12.9869f, 15.8219f,
													34.8575f, -12.9869f, -29.136f,
													34.8575f, -12.9869f, 20.864f,
													-6.96422f, -12.9869f, 20.864f,
													0.158117f, -12.9869f, 32.3153f,
													16.4748f, -12.9869f, 39.2467f,
													11.6566f, -12.9869f, 41.0741f,
													34.8575f, -12.9869f, 32.2749f,
													0.158214f, -12.9869f, 32.3154f,
													-6.96419f, -12.9869f, 20.864f,
													-1.27103f, -4.53197f, 32.1356f,
													-10.1003f, -12.9869f, 15.8219f,
													-15.1425f, -12.9869f, 7.71512f,
													-17.7206f, 12.7842f, 10.0264f,
													-42.0829f, 12.8123f, -29.136f,
													-38.0628f, -12.9869f, -29.136f,
													-29.2736f, -12.9869f, -15.0048f,
													-3.4102f, 33.3911f, -29.136f,
													-17.7206f, 12.7842f, 10.0264f,
													-8.70369f, 30.5743f, -29.136f,
													-15.1425f, 27.148f, -29.136f,
													-41.338f, 13.2086f, -29.136f,
													-42.0829f, 12.8123f, -29.136f,
													34.8575f, -12.9869f, 32.2749f,
													16.4748f, -12.9869f, 39.2467f,
													34.8575f, -6.01665f, 20.864f,
													34.8575f, 4.92442f, 2.9527f,
													34.8575f, 24.5256f, -29.136f,
													-3.4102f, 33.3911f, -29.136f,
													-17.7206f, 12.7842f, 10.0264f,
													-1.27103f, -4.53197f, 32.1356f,
													11.6566f, -12.9869f, 41.0741f,
													-1.27103f, -4.53197f, 32.1356f,
													11.6566f, -12.9869f, 41.0741f,
													0.158214f, -12.9869f, 32.3154f,
													0.0289307f, 39.9012f, -61.1455f,
													-2.54625f, 16.3791f, -61.1455f,
													13.9315f, 42.494f, -61.1455f,
													13.9315f, 14.5344f, -61.1455f,
													13.9315f, -5.31049f, 38.8545f,
													0.86338f, 7.75765f, 38.8545f,
													13.9315f, 10.1948f, 38.8545f,
													-17.2768f, -5.31049f, 38.8545f,
													-13.9431f, 4.99635f, 38.8545f,
													-20.9687f, -16.7255f, 38.8545f,
													12.5846f, -48.5735f, 38.8545f,
													13.9315f, -48.7243f, 38.8545f,
													13.9315f, -17.0949f, -11.1455f,
													13.9315f, -5.31049f, -11.1455f,
													13.9315f, -5.31049f, -29.7745f,
													13.9315f, -37.3846f, 20.9286f,
													13.9315f, 14.5344f, -61.1455f,
													13.9315f, 42.494f, -61.1455f,
													13.9315f, 41.4465f, -57.9025f,
													13.9315f, -5.31049f, 38.8545f,
													13.9315f, -48.7243f, 38.8545f,
													13.9315f, 26.3444f, -11.1455f,
													13.9315f, 17.5921f, 15.9518f,
													13.9315f, 10.1948f, 38.8545f,
													13.9315f, 42.4941f, -61.1455f,
													13.9315f, 41.4467f, -57.9027f,
													0.0285797f, 39.9013f, -61.1455f,
													13.9315f, 26.3445f, -11.1455f,
													13.9315f, 17.5923f, 15.9517f,
													13.9315f, 10.1949f, 38.8545f,
													-26.1345f, 13.69f, 4.89963f,
													-13.9432f, 4.99643f, 38.8545f,
													0.863289f, 7.75774f, 38.8545f,
													-30.807f, -23.3456f, 6.6532f,
													-2.54625f, 16.3791f, -61.1455f,
													-32.655f, -18.2041f, 13.4884f,
													-26.1345f, 13.69f, 4.89963f,
													0.0285797f, 39.9013f, -61.1455f,
													-26.1345f, 13.69f, 4.89963f,
													-13.9432f, 4.99643f, 38.8545f,
													-32.655f, -18.2041f, 13.4884f,
													-20.9687f, -16.7255f, 38.8545f,
													-17.2768f, -5.31049f, 38.8545f,
													13.9315f, -48.7243f, 38.8545f,
													13.9315f, -37.3846f, 20.9286f,
													12.5846f, -48.5735f, 38.8545f,
													13.9315f, -17.0949f, -11.1455f,
													13.9315f, -5.31049f, -29.7745f,
													-30.807f, -23.3456f, 6.6532f,
													-2.54625f, 16.3791f, -61.1455f,
													13.9315f, 14.5344f, -61.1455f,
													-32.655f, -18.2041f, 13.4884f,
													-20.9687f, -16.7255f, 38.8545f,
													-30.807f, -23.3456f, 6.6532f,
													12.5846f, -48.5735f, 38.8545f,
													32.9967f, 9.70081f, -22.8865f,
													35.4077f, 12.5017f, -22.8865f,
													1.88543f, 40.8121f, -22.8865f,
													29.5216f, 40.8121f, -22.8865f,
													37.9828f, 36.0239f, -22.8865f,
													1.88543f, -9.18788f, -22.8865f,
													16.7359f, -9.18788f, -22.8865f,
													-11.8161f, 40.8121f, -22.8865f,
													-20.9746f, 13.6721f, -22.8865f,
													8.32422f, -15.6267f, -22.8865f,
													13.6177f, -12.8099f, -22.8865f,
													-28.6887f, -9.18788f, -22.8865f,
													1.88543f, -19.0529f, -22.8865f,
													-24.3101f, -32.9923f, -22.8865f,
													-33.59f, -23.7124f, -22.8865f,
													-25.055f, -33.3887f, -22.8865f,
													-35.6535f, -29.8275f, -22.8865f,
													1.88543f, 40.8121f, -22.8865f,
													29.5216f, 40.8121f, -22.8865f,
													15.0616f, 40.8121f, -9.71027f,
													1.88543f, 40.8121f, 2.29609f,
													-6.79587f, 40.8121f, 10.2066f,
													-11.8161f, 40.8121f, -22.8865f,
													-0.692513f, -33.4167f, 16.2762f,
													-25.055f, -33.3886f, -22.8865f,
													4.07097f, -30.2241f, 39.286f,
													-15.0555f, -23.2205f, 42.0553f,
													-23.6995f, -21.355f, 37.0679f,
													-35.6536f, -29.8274f, -22.8865f,
													-6.79593f, 40.8121f, 10.2064f,
													1.88543f, 40.8121f, 2.29581f,
													-7.53357f, 31.8023f, 25.3861f,
													15.0615f, 40.8121f, -9.71041f,
													11.8194f, 9.81274f, 43.1586f,
													37.9824f, 36.0239f, -22.8865f,
													29.5214f, 40.8121f, -22.8865f,
													-35.6536f, -29.8274f, -22.8865f,
													-33.5901f, -23.7124f, -22.8865f,
													-23.6995f, -21.355f, 37.0679f,
													-28.6888f, -9.18788f, -22.8865f,
													-20.9746f, 13.6721f, -22.8865f,
													-7.53357f, 31.8023f, 25.3861f,
													-6.79593f, 40.8121f, 10.2064f,
													-11.8161f, 40.8121f, -22.8865f,
													-25.055f, -33.3886f, -22.8865f,
													-0.692513f, -33.4167f, 16.2762f,
													-24.3102f, -32.9923f, -22.8865f,
													1.88543f, -19.0529f, -22.8865f,
													8.32418f, -15.6266f, -22.8865f,
													13.618f, -12.8097f, -22.8865f,
													11.8194f, 9.81274f, 43.1586f,
													37.9824f, 36.0239f, -22.8865f,
													5.2989f, -22.0815f, 51.7475f,
													7.14699f, -27.2233f, 44.9118f,
													35.4076f, 12.5012f, -22.8865f,
													11.8194f, 9.81274f, 43.1586f,
													5.2989f, -22.0815f, 51.7475f,
													-7.53357f, 31.8023f, 25.3861f,
													-23.6995f, -21.355f, 37.0679f,
													-15.0555f, -23.2205f, 42.0553f,
													35.4076f, 12.5012f, -22.8865f,
													32.9967f, 9.70081f, -22.8865f,
													7.14699f, -27.2233f, 44.9118f,
													4.07097f, -30.2241f, 39.286f,
													-0.692513f, -33.4167f, 16.2762f,
													13.618f, -12.8097f, -22.8865f,
													16.7359f, -9.18788f, -22.8865f,
													7.14699f, -27.2233f, 44.9118f,
													4.07097f, -30.2241f, 39.286f,
													5.2989f, -22.0815f, 51.7475f,
													-15.0555f, -23.2205f, 42.0553f,
													33.1597f, -3.4619f, 17.6225f,
													29.826f, -13.7687f, 17.6225f,
													11.0343f, 26.2833f, 17.6225f,
													11.0343f, -13.7687f, 17.6225f,
													26.1341f, -25.1838f, 17.6225f,
													22.5778f, -25.3122f, 17.6225f,
													-12.3807f, 9.64623f, 17.6225f,
													3.63467f, 36.2313f, 17.6225f,
													-4.4982f, 36.2313f, 17.6225f,
													11.0343f, -25.7293f, 17.6225f,
													-19.3232f, -13.7687f, 17.6225f,
													-23.2366f, -26.9675f, 17.6225f,
													2.13412f, 36.2313f, 8.72228f,
													-12.6925f, 36.2313f, -6.10432f,
													-4.17271f, 36.2313f, -28.6855f,
													-16.7287f, 36.2313f, -17.7914f,
													3.63467f, 36.2313f, 17.6225f,
													-4.4982f, 36.2313f, 17.6225f,
													-16.7287f, 36.2313f, -17.7913f,
													-12.6925f, 36.2313f, -6.10434f,
													-32.2133f, -28.3257f, -7.20351f,
													-4.4982f, 36.2313f, 17.6225f,
													-23.2367f, -26.9675f, 17.6225f,
													-19.3232f, -13.7687f, 17.6225f,
													-12.3807f, 9.64623f, 17.6225f,
													-14.5506f, -25.9361f, -22.4231f,
													-23.8033f, -28.3257f, -14.5004f,
													-5.90771f, -27.8014f, -17.4364f,
													20.9682f, 5.23197f, -16.3325f,
													33.1595f, -3.46165f, 17.6225f,
													1.6154f, 27.2214f, -34.1049f,
													-4.17274f, 36.2313f, -28.6855f,
													2.13407f, 36.2313f, 8.72223f,
													3.63462f, 36.2313f, 17.6225f,
													11.0343f, 26.2832f, 17.6225f,
													1.6154f, 27.2214f, -34.1049f,
													-4.17274f, 36.2313f, -28.6855f,
													-14.5506f, -25.9361f, -22.4231f,
													-23.8033f, -28.3257f, -14.5004f,
													-32.2133f, -28.3257f, -7.20351f,
													-16.7287f, 36.2313f, -17.7913f,
													14.4478f, -26.6623f, -7.74374f,
													26.1341f, -25.1837f, 17.6225f,
													20.9682f, 5.23197f, -16.3325f,
													33.1595f, -3.46165f, 17.6225f,
													29.8259f, -13.7687f, 17.6225f,
													-14.5506f, -25.9361f, -22.4231f,
													-5.90771f, -27.8014f, -17.4364f,
													1.6154f, 27.2214f, -34.1049f,
													20.9682f, 5.23197f, -16.3325f,
													14.4478f, -26.6623f, -7.74374f,
													-23.2367f, -26.9675f, 17.6225f,
													11.0343f, -25.7293f, 17.6225f,
													-32.2133f, -28.3257f, -7.20351f,
													22.5778f, -25.3122f, 17.6225f,
													-23.8033f, -28.3257f, -14.5004f,
													-5.90771f, -27.8014f, -17.4364f,
													14.4478f, -26.6623f, -7.74374f,
													26.1341f, -25.1837f, 17.6225f,
													-3.49003f, 43.2227f, -49.9396f,
													15.3987f, 24.334f, -49.9396f,
													-1.0791f, 46.0236f, -49.9396f,
													15.3987f, 44.1789f, -49.9396f,
													-19.7509f, 24.334f, -49.9396f,
													15.3987f, 11.8466f, -49.9396f,
													-22.8691f, 20.712f, -49.9396f,
													15.3987f, -19.0798f, 50.0604f,
													15.3987f, -21.1432f, 50.0604f,
													14.0517f, -18.929f, 50.0604f,
													15.3987f, 11.8466f, -49.9396f,
													15.3987f, -7.75465f, -17.8509f,
													15.3987f, 24.334f, -49.9396f,
													15.3987f, 24.334f, -18.5686f,
													15.3987f, 12.5495f, 0.060406f,
													15.3987f, -18.6957f, 0.060406f,
													15.3987f, -7.74012f, 32.1345f,
													15.3987f, -23.8068f, 48.2012f,
													15.3987f, -25.666f, 46.9035f,
													15.3987f, -25.666f, 11.4713f,
													15.3987f, 44.1789f, -49.9396f,
													15.3987f, -19.0798f, 50.0604f,
													15.3987f, -21.1432f, 50.0604f,
													-6.40977f, -25.666f, 21.8688f,
													-2.98403f, -25.666f, 18.4431f,
													-7.80222f, -25.666f, 20.2704f,
													15.3987f, -25.666f, 46.9035f,
													15.3987f, -25.666f, 11.4713f,
													-32.416f, 3.29756f, 12.2323f,
													-20.7301f, -17.211f, 11.3318f,
													-37.1796f, 0.105114f, -10.7773f,
													-7.8023f, -25.666f, 20.2703f,
													-2.98378f, -25.666f, 18.4428f,
													-20.7301f, -17.211f, 11.3318f,
													15.3987f, -25.666f, 11.4711f,
													-37.1796f, 0.105114f, -10.7773f,
													-22.8692f, 20.712f, -49.9396f,
													15.3987f, 11.8465f, -49.9396f,
													15.3987f, -7.75481f, -17.8508f,
													15.3987f, -18.6958f, 0.060406f,
													15.3987f, 44.1788f, -49.9396f,
													15.3987f, 24.334f, -18.5687f,
													-1.07873f, 46.0235f, -49.9396f,
													15.3987f, 12.5495f, 0.060406f,
													15.3987f, -7.74031f, 32.1347f,
													15.3987f, -19.0799f, 50.0604f,
													-29.3394f, 6.29887f, 17.8589f,
													14.0524f, -18.9292f, 50.0604f,
													-22.8692f, 20.712f, -49.9396f,
													-19.751f, 24.334f, -49.9396f,
													-37.1796f, 0.105114f, -10.7773f,
													-32.416f, 3.29756f, 12.2323f,
													-29.3394f, 6.29887f, 17.8589f,
													-1.07873f, 46.0235f, -49.9396f,
													-3.48997f, 43.2226f, -49.9396f,
													14.0524f, -18.9292f, 50.0604f,
													15.3987f, -21.1432f, 50.0604f,
													-29.3394f, 6.29887f, 17.8589f,
													-32.416f, 3.29756f, 12.2323f,
													-20.7301f, -17.211f, 11.3318f,
													-7.8023f, -25.666f, 20.2703f,
													-6.40977f, -25.666f, 21.8688f,
													15.3987f, -25.666f, 46.9035f,
													55.2747f, -18.9041f, 17.9434f,
													16.8182f, 19.5525f, 17.9434f,
													55.2747f, -14.3813f, 17.9434f,
													53.9278f, -12.1671f, 17.9434f,
													20.3745f, 19.6809f, 17.9434f,
													5.27474f, 19.1354f, 17.9434f,
													5.27474f, -18.9041f, 17.9434f,
													-30.7309f, 17.1016f, 17.9434f,
													-28.9962f, 17.8972f, 17.9434f,
													-44.7253f, -18.9041f, 17.9434f,
													-44.7253f, 10.6829f, 17.9434f,
													55.2747f, -18.9041f, 17.9434f,
													55.2747f, -17.0449f, 16.0842f,
													55.2747f, -18.9041f, 14.7865f,
													55.2747f, -14.3813f, 17.9434f,
													-44.7253f, -18.9041f, -9.98509f,
													-44.7253f, -18.9041f, 17.9434f,
													-11.3723f, -18.9041f, -15.4095f,
													5.27474f, -18.9041f, 17.9434f,
													5.27474f, -18.9041f, -18.117f,
													33.4663f, -18.9041f, -10.2481f,
													32.0739f, -18.9041f, -11.8465f,
													20.5753f, -18.9041f, -20.6054f,
													55.2747f, -18.9041f, 17.9434f,
													55.2747f, -18.9041f, 14.7865f,
													-44.7253f, 11.8339f, -11.4835f,
													-44.7253f, 13.5335f, -7.70581f,
													-44.7253f, 2.14147f, -11.011f,
													-44.7253f, 12.7246f, -0.427895f,
													-44.7253f, -18.9041f, -9.98509f,
													-44.7253f, -18.9041f, 17.9434f,
													-44.7253f, 10.6829f, 17.9434f,
													-37.973f, 16.539f, -6.8829f,
													-28.9962f, 17.8972f, 17.9434f,
													-44.7253f, 13.5335f, -7.70605f,
													-44.7253f, 12.7246f, -0.427923f,
													-44.7253f, 10.6829f, 17.9434f,
													-30.7309f, 17.1016f, 17.9434f,
													-29.5643f, 16.539f, -14.1787f,
													-44.7253f, 11.8343f, -11.4836f,
													-11.6668f, 17.0634f, -17.115f,
													7.46001f, 10.0597f, -19.8843f,
													19.146f, -10.4492f, -20.7851f,
													20.5752f, -18.9041f, -20.6054f,
													5.27474f, -18.9041f, -18.117f,
													-11.3723f, -18.9041f, -15.4095f,
													-44.7253f, -18.9041f, -9.98513f,
													-44.7253f, 2.14142f, -11.0111f,
													-37.973f, 16.539f, -6.8829f,
													-44.7253f, 13.5335f, -7.70605f,
													-29.5643f, 16.539f, -14.1787f,
													-44.7253f, 11.8343f, -11.4836f,
													19.146f, -10.4492f, -20.7851f,
													20.5752f, -18.9041f, -20.6054f,
													32.0735f, -18.9041f, -11.8468f,
													10.5365f, 13.0605f, -14.2583f,
													53.9286f, -12.1676f, 17.9434f,
													8.68832f, 18.2024f, -7.42243f,
													20.3744f, 19.681f, 17.9434f,
													-11.6668f, 17.0634f, -17.115f,
													7.46001f, 10.0597f, -19.8843f,
													8.68832f, 18.2024f, -7.42243f,
													10.5365f, 13.0605f, -14.2583f,
													8.68832f, 18.2024f, -7.42243f,
													20.3744f, 19.681f, 17.9434f,
													-11.6668f, 17.0634f, -17.115f,
													-29.5643f, 16.539f, -14.1787f,
													-37.973f, 16.539f, -6.8829f,
													-28.9962f, 17.8972f, 17.9434f,
													5.27474f, 19.1354f, 17.9434f,
													16.8182f, 19.5525f, 17.9434f,
													19.146f, -10.4492f, -20.7851f,
													32.0735f, -18.9041f, -11.8468f,
													7.46001f, 10.0597f, -19.8843f,
													10.5365f, 13.0605f, -14.2583f,
													53.9286f, -12.1676f, 17.9434f,
													55.2747f, -14.3814f, 17.9434f,
													55.2747f, -17.0449f, 16.0843f,
													55.2747f, -18.9041f, 14.7866f,
													33.4662f, -18.9041f, -10.248f,
	};

	const TArray<int32> FracturedGeometry::RawIndicesArray = {
													0, 1, 2,
													0, 2, 3,
													3, 2, 4,
													5, 0, 3,
													3, 4, 6,
													5, 3, 7,
													7, 3, 6,
													7, 6, 8,
													9, 10, 11,
													11, 10, 12,
													10, 13, 12,
													11, 12, 14,
													12, 13, 15,
													14, 12, 16,
													12, 15, 16,
													16, 15, 17,
													18, 19, 20,
													20, 19, 21,
													19, 22, 21,
													20, 21, 23,
													21, 22, 24,
													23, 21, 25,
													21, 24, 25,
													25, 24, 26,
													27, 28, 29,
													29, 28, 30,
													28, 31, 30,
													29, 30, 32,
													30, 31, 33,
													32, 30, 34,
													30, 33, 34,
													34, 33, 35,
													36, 37, 38,
													38, 37, 39,
													37, 40, 39,
													38, 39, 41,
													39, 40, 42,
													41, 39, 43,
													39, 42, 43,
													43, 42, 44,
													45, 46, 47,
													47, 46, 48,
													46, 49, 48,
													47, 48, 50,
													48, 49, 51,
													50, 48, 52,
													48, 51, 52,
													52, 51, 53,
													54, 55, 56,
													57, 58, 54,
													54, 58, 55,
													58, 57, 59,
													59, 57, 60,
													61, 58, 59,
													62, 63, 64,
													65, 66, 67,
													67, 66, 62,
													62, 66, 63,
													68, 69, 70,
													71, 72, 68,
													68, 72, 69,
													73, 74, 71,
													71, 74, 72,
													71, 75, 73,
													76, 77, 78,
													78, 77, 79,
													79, 77, 80,
													81, 80, 82,
													80, 77, 82,
													83, 84, 85,
													85, 84, 86,
													86, 84, 87,
													88, 87, 89,
													87, 84, 89,
													90, 91, 92,
													92, 91, 93,
													93, 91, 94,
													94, 91, 95,
													96, 97, 98,
													98, 97, 99,
													99, 97, 100,
													101, 102, 103,
													103, 102, 104,
													104, 102, 105,
													105, 102, 106,
													107, 106, 108,
													108, 106, 102,
													106, 109, 105,
													106, 107, 110,
													106, 111, 109,
													110, 112, 106,
													106, 112, 111,
													113, 114, 115,
													115, 114, 116,
													116, 114, 117,
													118, 116, 119,
													119, 116, 117,
													116, 118, 120,
													121, 122, 123,
													122, 124, 123,
													123, 124, 125,
													125, 124, 126,
													126, 124, 127,
													128, 129, 130,
													129, 131, 130,
													131, 132, 130,
													132, 133, 130,
													130, 133, 134,
													134, 133, 135,
													135, 133, 136,
													137, 138, 139,
													139, 138, 140,
													140, 138, 141,
													141, 138, 142,
													143, 144, 145,
													146, 147, 148,
													149, 150, 151,
													151, 150, 152,
													152, 150, 153,
													153, 150, 154,
													154, 150, 155,
													155, 150, 156,
													156, 150, 157,
													157, 150, 158,
													159, 160, 161,
													161, 160, 162,
													163, 164, 165,
													165, 164, 166,
													167, 166, 168,
													168, 166, 164,
													169, 166, 167,
													170, 171, 172,
													172, 171, 173,
													174, 175, 173,
													173, 175, 172,
													174, 173, 176,
													174, 177, 175,
													175, 177, 178,
													179, 174, 176,
													174, 180, 177,
													180, 174, 181,
													181, 174, 182,
													182, 174, 179,
													182, 179, 183,
													184, 185, 186,
													186, 187, 184,
													184, 187, 188,
													187, 189, 188,
													188, 189, 190,
													190, 189, 191,
													192, 193, 194,
													193, 195, 194,
													194, 195, 196,
													197, 198, 199,
													199, 198, 200,
													200, 198, 201,
													201, 198, 202,
													203, 202, 204,
													204, 202, 205,
													202, 198, 205,
													206, 207, 208,
													208, 207, 209,
													209, 207, 210,
													211, 212, 213,
													213, 212, 214,
													214, 212, 215,
													215, 212, 216,
													216, 212, 217,
													218, 219, 220,
													218, 220, 221,
													221, 220, 222,
													222, 223, 221,
													224, 225, 222,
													222, 225, 223,
													226, 227, 228,
													228, 227, 229,
													227, 226, 230,
													230, 226, 231,
													231, 232, 230,
													230, 232, 233,
													233, 232, 234,
													230, 233, 235,
													236, 237, 238,
													236, 238, 239,
													240, 241, 238,
													238, 241, 239,
													239, 242, 236,
													236, 242, 243,
													239, 241, 244,
													239, 245, 242,
													244, 246, 239,
													239, 246, 245,
													245, 246, 247,
													248, 249, 250,
													250, 249, 251,
													251, 249, 252,
													252, 249, 253,
													253, 249, 254,
													255, 256, 257,
													257, 256, 258,
													259, 258, 260,
													260, 258, 261,
													258, 256, 261,
													262, 263, 264,
													265, 264, 266,
													264, 263, 266,
													267, 268, 269,
													269, 268, 270,
													270, 268, 271,
													271, 268, 272,
													272, 268, 273,
													273, 268, 274,
													275, 276, 277,
													277, 276, 278,
													278, 276, 279,
													279, 276, 280,
													281, 282, 283,
													283, 282, 284,
													285, 286, 287,
													287, 286, 288,
													287, 289, 285,
													285, 289, 290,
													289, 291, 290,
													290, 291, 292,
													292, 291, 293,
													294, 295, 296,
													296, 295, 297,
													295, 294, 298,
													298, 294, 299,
													300, 301, 302,
													302, 301, 303,
													303, 304, 302,
													302, 304, 305,
													305, 304, 306,
													306, 304, 307,
													307, 308, 306,
													306, 308, 309,
													309, 308, 310,
													306, 309, 311,
													312, 313, 314,
													313, 315, 314,
													315, 316, 314,
													314, 316, 317,
													317, 316, 318,
													319, 318, 320,
													318, 316, 320,
													321, 322, 323,
													323, 322, 324,
													324, 322, 325,
													322, 326, 325,
													327, 328, 329,
													329, 328, 330,
													330, 328, 331,
													331, 328, 332,
													332, 328, 333,
													333, 328, 334,
													334, 328, 335,
													336, 337, 338,
													339, 340, 341,
													341, 340, 342,
													343, 344, 345,
													346, 347, 343,
													343, 347, 344,
													346, 343, 348,
													348, 343, 349,
													349, 343, 350,
													351, 352, 353,
													352, 351, 354,
													353, 352, 355,
													355, 352, 356,
													356, 352, 357,
													352, 354, 358,
													358, 354, 359,
													352, 360, 357,
													360, 352, 361,
													361, 352, 358,
													358, 362, 361,
													363, 364, 365,
													364, 366, 365,
													366, 367, 365,
													367, 368, 365,
													365, 368, 369,
													370, 369, 371,
													369, 368, 371,
													372, 373, 374,
													374, 373, 375,
													375, 373, 376,
													377, 378, 379,
													380, 379, 381,
													379, 378, 381,
													382, 383, 384,
													383, 385, 384,
													385, 386, 384,
													384, 386, 387,
													387, 386, 388,
													388, 386, 389,
													390, 391, 392,
													392, 391, 393,
													394, 395, 396,
													396, 395, 397,
													397, 395, 398,
													399, 400, 396,
													396, 400, 394,
													396, 401, 399,
													399, 401, 402,
													403, 404, 399,
													399, 404, 400,
													405, 399, 402,
													399, 406, 403,
													399, 405, 406,
													406, 405, 407,
													407, 405, 408,
													407, 408, 409,
													409, 408, 410,
													411, 412, 413,
													411, 413, 414,
													415, 416, 414,
													414, 416, 411,
													417, 418, 419,
													419, 418, 420,
													420, 418, 421,
													421, 418, 422,
													423, 424, 425,
													424, 426, 425,
													425, 426, 427,
													427, 426, 428,
													428, 426, 429,
													430, 431, 432,
													431, 433, 432,
													433, 434, 432,
													432, 434, 435,
													435, 434, 436,
													436, 434, 437,
													438, 439, 440,
													440, 439, 441,
													441, 439, 442,
													439, 443, 442,
													444, 445, 446,
													446, 445, 447,
													447, 445, 448,
													449, 450, 451,
													451, 450, 452,
													452, 450, 453,
													454, 455, 456,
													456, 455, 457,
													457, 455, 458,
													459, 458, 460,
													458, 455, 460,
													461, 462, 463,
													463, 462, 464,
													465, 466, 467,
													467, 466, 468,
													469, 470, 466,
													466, 470, 468,
													468, 471, 467,
													467, 471, 472,
													472, 471, 473,
													470, 474, 468,
													468, 475, 471,
													468, 474, 475,
													475, 474, 476,
													477, 478, 479,
													479, 478, 480,
													477, 481, 478,
													478, 481, 482,
													483, 484, 485,
													484, 486, 485,
													487, 485, 488,
													488, 485, 489,
													485, 486, 489,
													490, 491, 492,
													493, 494, 495,
													495, 494, 496,
													496, 494, 497,
													498, 497, 499,
													497, 494, 499,
													500, 501, 502,
													502, 501, 503,
													503, 501, 504,
													504, 501, 505,
													506, 507, 508,
													509, 508, 510,
													508, 507, 510,
													511, 512, 513,
													513, 512, 514,
													514, 512, 515,
													516, 517, 518,
													517, 519, 518,
													518, 519, 520,
													520, 519, 521,
													521, 519, 522,
													522, 519, 523,
													524, 525, 526,
													526, 525, 527,
													528, 525, 524,
													525, 528, 529,
													529, 528, 530,
													531, 532, 533,
													534, 535, 536,
													536, 535, 537,
													537, 535, 538,
													538, 535, 539,
													538, 539, 540,
													540, 539, 541,
													541, 539, 542,
													542, 539, 543,
													537, 544, 536,
													545, 540, 546,
													546, 540, 541,
													547, 548, 549,
													550, 551, 547,
													547, 551, 548,
													552, 553, 554,
													555, 556, 557,
													556, 558, 557,
													557, 558, 559,
													559, 558, 560,
													561, 560, 562,
													562, 560, 563,
													560, 558, 563,
													564, 565, 566,
													565, 567, 566,
													567, 568, 566,
													568, 569, 566,
													566, 569, 570,
													570, 569, 571,
													572, 573, 574,
													574, 573, 575,
													575, 573, 576,
													577, 576, 578,
													576, 573, 578,
													579, 580, 581,
													581, 580, 582,
													582, 580, 583,
													583, 580, 584,
													584, 580, 585,
													585, 580, 586,
													587, 588, 589,
													589, 588, 590,
													590, 588, 591,
													588, 587, 592,
													592, 587, 593,
													593, 594, 592,
													592, 594, 595,
													596, 597, 593,
													593, 597, 594,
													598, 599, 600,
													601, 599, 598,
													602, 603, 604,
													605, 606, 603,
													603, 606, 604,
													607, 608, 605,
													605, 608, 606,
													606, 608, 609,
													610, 611, 605,
													605, 611, 607,
													612, 613, 614,
													614, 613, 615,
													614, 615, 616,
													616, 615, 617,
													617, 615, 618,
													619, 620, 621,
													621, 620, 622,
													623, 622, 624,
													622, 620, 624,
													625, 626, 627,
													627, 626, 628,
													628, 626, 629,
													629, 626, 630,
													630, 626, 631,
													631, 626, 632,
													633, 632, 634,
													632, 626, 634,
													635, 636, 637,
													637, 636, 638,
													639, 640, 641,
													642, 643, 644,
													644, 643, 645,
													646, 647, 648,
													648, 647, 649,
													650, 651, 652,
													652, 651, 653,
													653, 651, 654,
													655, 654, 656,
													656, 654, 657,
													654, 651, 657,
													658, 659, 660,
													660, 659, 661,
													661, 659, 662,
													662, 659, 663,
													663, 659, 664,
													665, 664, 666,
													664, 659, 666,
};

const TArray<int32> FracturedGeometry::RawBoneMapArray = {
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													0,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													1,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													2,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													3,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													4,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													5,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													6,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													7,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													8,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													9,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
													10,
};

const TArray<FTransform> FracturedGeometry::RawTransformArray = {
   FTransform(FQuat(0, 0, 0, 1), FVector(0, 0, 0), FVector(1, 1, 1)),
   FTransform(FQuat(0, 0, 0, 1), FVector(-39.6322f, 14.7309f, 34.2175f), FVector(1, 1, 1)),
   FTransform(FQuat(0, 0, 0, 1), FVector(-22.9059f, -34.0281f, -7.4853f), FVector(1, 1, 1)),
   FTransform(FQuat(0, 0, 0, 1), FVector(20.0865f, 40.5987f, 4.4615f), FVector(1, 1, 1)),
   FTransform(FQuat(0, 0, 0, 1), FVector(-35.0785f, 15.1693f, -9.62309f), FVector(1, 1, 1)),
   FTransform(FQuat(0, 0, 0, 1), FVector(15.1425f, -37.0131f, -20.864f), FVector(1, 1, 1)),
   FTransform(FQuat(0, 0, 0, 1), FVector(36.0685f, 5.31049f, 11.1455f), FVector(1, 1, 1)),
   FTransform(FQuat(0, 0, 0, 1), FVector(-1.88543f, 9.18788f, -27.1135f), FVector(1, 1, 1)),
   FTransform(FQuat(0, 0, 0, 1), FVector(-11.0343f, 13.7687f, 32.3775f), FVector(1, 1, 1)),
   FTransform(FQuat(0, 0, 0, 1), FVector(34.6013f, -24.334f, -0.060406f), FVector(1, 1, 1)),
   FTransform(FQuat(0, 0, 0, 1), FVector(-5.27474f, -31.0959f, 32.0566f), FVector(1, 1, 1)),
};

const TArray<int32> FracturedGeometry::RawLevelArray = {
   0,
   1,
   1,
   1,
   1,
   1,
   1,
   1,
   1,
   1,
   1,
};

const TArray<int32> FracturedGeometry::RawParentArray = {
  -1,
   0, 
   0, 
   0, 
   0, 
   0, 
   0, 
   0, 
   0, 
   0, 
   0, 
};

const TArray<TSet<int32>> FracturedGeometry::RawChildrenArray = {

};

const TArray<int32> FracturedGeometry::RawSimulationTypeArray = {
   0,
   0,
   0,
   0,
   0,
   0,
   0,
   0,
   0,
   0,
   0
};

const TArray<int32> FracturedGeometry::RawStatusFlagsArray = {
   0,
   0,
   0,
   0,
   0,
   0,
   0,
   0,
   0,
   0,
   0
};

FracturedGeometry::FracturedGeometry()
{}

FracturedGeometry::~FracturedGeometry()
{}

GlobalFracturedGeometry::GlobalFracturedGeometry()
	: RawIndicesArray0(FracturedGeometry::RawIndicesArray)
	, RawIndicesArray1(FracturedGeometry::RawIndicesArray)
	, RawIndicesArray2(FracturedGeometry::RawIndicesArray)
	, RawIndicesArray(RawIndicesArray1)
	, RawBoneMapArray(FracturedGeometry::RawBoneMapArray)
	, RawTransformArray(FracturedGeometry::RawTransformArray)
	, RawLevelArray(FracturedGeometry::RawLevelArray)
	, RawParentArray(FracturedGeometry::RawParentArray)
	, RawChildrenArray(FracturedGeometry::RawChildrenArray)
	, RawSimulationTypeArray(FracturedGeometry::RawSimulationTypeArray)
	, RawStatusFlagsArray(FracturedGeometry::RawStatusFlagsArray)
{
	// The FracturedGeometry data set has lots of free and coincident vertices.
	// We're going to concatenate 2 copies of it.
	const int NumOrigFractured = FracturedGeometry::RawVertexArray.Num();
	RawVertexArray.Reserve(NumOrigFractured * 3);
	RawVertexArray.Append(FracturedGeometry::RawVertexArray);
	RawVertexArray.Append(FracturedGeometry::RawVertexArray);
	RawVertexArray.Append(FracturedGeometry::RawVertexArray);
	check(RawVertexArray.Num() == NumOrigFractured * 3);

	// Randomly perturb some of the vertices in the first point set.
	FRandomStream RandIdx(NumOrigFractured);
	FRandomStream RandPos(NumOrigFractured);
	for(int32 i=0; i < NumOrigFractured; i += RandIdx.RandRange(0,10))
	{
		RawVertexArray[i] += RandPos.FRandRange(-1.0f, 1.0f);
	}

	// Measure the extent of the second point set.
	Chaos::FAABB3 BBox(
		Chaos::FVec3(
			RawVertexArray[NumOrigFractured + 0],
			RawVertexArray[NumOrigFractured + 1],
			RawVertexArray[NumOrigFractured + 2]),
		Chaos::FVec3(
			RawVertexArray[NumOrigFractured + 0],
			RawVertexArray[NumOrigFractured + 1],
			RawVertexArray[NumOrigFractured + 2]));
	for(int32 i=NumOrigFractured+3; i < NumOrigFractured*2; i+=3)
	{
		BBox.GrowToInclude(
			Chaos::FVec3(
				RawVertexArray[i + 0],
				RawVertexArray[i + 1],
				RawVertexArray[i + 2]));
	}

	// Translate the third point set out of the way.
	const Chaos::FVec3 Extents = BBox.Extents();
	for(int32 i=NumOrigFractured*2 + 1; i < NumOrigFractured*3; i+=3)
	{
		RawVertexArray[i] += Extents[1] + 1.0e-5;
	}

	// Offset the index arrays (orig array is flat)
	for(int32 &Idx : RawIndicesArray1)
	{
		Idx += NumOrigFractured / 3;
	}
	for (int32 &Idx : RawIndicesArray2)
	{
		Idx += NumOrigFractured / 3 * 2;
	}

	// Merge the index arrays
	RawIndicesArrayMerged.Reserve(RawIndicesArray0.Num() + RawIndicesArray1.Num() + RawIndicesArray2.Num());
	RawIndicesArrayMerged.Append(RawIndicesArray0);
	RawIndicesArrayMerged.Append(RawIndicesArray1);
	RawIndicesArrayMerged.Append(RawIndicesArray2);
}

GlobalFracturedGeometry::~GlobalFracturedGeometry()
{}


}