// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/Real.h"
#include "Containers/Array.h"
#include "Math/Transform.h"

namespace GeometryCollectionTest
{

	class FracturedGeometry
	{
	public:
		FracturedGeometry();
		~FracturedGeometry();

		static const TArray<float>			RawVertexArray;
		static const TArray<int32>			RawIndicesArray;
		static const TArray<int32>			RawBoneMapArray;
		static const TArray<FTransform>		RawTransformArray;
		static const TArray<int32>			RawLevelArray;
		static const TArray<int32>			RawParentArray;
		static const TArray<TSet<int32>>	RawChildrenArray;
		static const TArray<int32>			RawSimulationTypeArray;
		static const TArray<int32>			RawStatusFlagsArray;
	};

	/**
	 * A version of \c FracturedGeometry in a global point pool.
	 */
	class GlobalFracturedGeometry
	{
	public:
		GlobalFracturedGeometry();
		~GlobalFracturedGeometry();

		TArray<float>			RawVertexArray;
		TArray<int32>			RawIndicesArray0; // Randomly coincident to RawIndicesArray1
		TArray<int32>			RawIndicesArray1; // Unchanged from FracturedGeometry
		TArray<int32>			RawIndicesArray2; // Offset in Y
		const TArray<int32> &	RawIndicesArray; // Points to RawIndicesArray1
		TArray<int32>			RawIndicesArrayMerged; // RawIndicesArray 0, 1, and 2 concatenated
		const TArray<int32>		RawBoneMapArray;
		const TArray<FTransform> RawTransformArray;
		const TArray<int32>		RawLevelArray;
		const TArray<int32>		RawParentArray;
		const TArray<TSet<int32>> RawChildrenArray;
		const TArray<int32>		RawSimulationTypeArray;
		const TArray<int32>		RawStatusFlagsArray;
	};
}