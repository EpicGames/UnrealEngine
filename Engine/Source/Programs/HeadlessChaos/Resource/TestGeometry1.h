// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/Real.h"
#include "Chaos/Array.h"
class TestGeometry1
{

public:
	TestGeometry1();
	~TestGeometry1() {};

	static const TArray<float>	RawVertexArray;
	static const TArray<int32>	RawIndicesArray;
};


const TArray<float> TestGeometry1::RawVertexArray = {
														-0.118278, 2.164261, 0.478031,
														0.718749, 2.083855, 0.478031,
														1.996569, -3.569340, 108.680893,
														-1.337000, -1.467590, 0.478031,
														-0.769773, -3.303612, 108.680908,
														-3.126952, 3.547722, 108.321014,
														1.344566, -1.725174, 0.478031,
														5.406225, 2.745453, 108.614349,
														-0.016398, 1.967595, -0.166328,
														1.364451, 0.926640, -0.166328,
														0.248418, -2.341320, -0.166325,
														-1.132432, -1.300364, -0.166325,
														-0.505581, -3.006680, 109.182297,
														4.463128, -1.863823, 109.187286,
														2.423427, 4.341752, 108.944153,
														-2.562841, 3.195637, 108.834076,
														-0.798691, -2.429302, 19.550392,
														-1.710983, -2.839973, 34.633438,
														-2.137956, -2.842075, 48.920158,
														-1.520910, 3.391621, 89.422653,
														-1.023157, 2.832703, 75.749176,
														0.466790, 2.115548, 19.550064,
														-0.284844, 1.749657, 34.631626,
														-0.030453, 1.842498, 48.914162,
														0.571760, -3.149826, 89.479492,
														0.860690, -2.889763, 75.783325,
														-1.463210, -2.974493, 62.579678,
														-2.737792, -1.023791, 75.785049,
														-4.645150, -1.379299, 89.484047,
														-1.098912, 2.182862, 62.565571,
														-3.242825, 1.197943, 48.917366,
														-2.573089, 1.047616, 34.632591,
														-1.490659, 1.393471, 19.550217,
														0.762244, 1.981854, 62.565571,
														1.200588, 2.592537, 75.749161,
														1.076512, 3.226872, 89.420547,
														0.412727, -3.177101, 62.579678,
														-0.407297, -3.008318, 48.920177,
														-0.653349, -2.941570, 34.633438,
														1.162380, -1.707573, 19.550411,
														-1.112370, 1.351433, 0.478031,
														1.569190, 1.093853, 0.478027,
														5.009187, -2.249374, 108.688980,
														-0.486561, -2.457596, 0.478031,
														-3.527044, -1.431587, 108.617859,
														-0.132941, 4.865506, 108.317337,
														0.350470, -2.537998, 0.478031,
														2.655671, 4.608155, 108.436157,
														-0.937359, 1.147748, -0.166328,
														0.587145, 1.909619, -0.166328,
														1.169379, -1.521469, -0.166325,
														-0.355126, -2.283344, -0.166325,
														-2.911711, -1.158359, 109.109955,
														1.800330, -3.228180, 109.182327,
														4.811073, 2.502804, 109.121490,
														0.071135, 4.557862, 108.828049,
														-1.717021, -1.430984, 19.550411,
														-2.804149, -1.813122, 34.633701,
														-3.486029, -1.759451, 48.921036,
														-4.122972, 2.281559, 89.451645,
														-2.822353, 1.916489, 75.764954,
														1.384725, 1.117272, 19.550217,
														0.807887, 0.722851, 34.632587,
														1.316387, 0.760000, 48.917374,
														3.108346, -2.122696, 89.484055,
														2.746032, -2.166569, 75.785332,
														-2.934515, -1.802063, 62.581707,
														-0.897159, -2.518925, 75.783951,
														-2.390798, -2.865246, 89.479469,
														-2.704913, 1.342780, 62.573124,
														-1.749091, 2.007581, 48.914188,
														-1.337567, 1.850785, 34.631626,
														-0.429829, 2.201680, 19.550064,
														2.234860, 0.809285, 62.573101,
														2.967290, 1.289101, 75.764999,
														3.402455, 1.823254, 89.446701,
														2.029080, -2.338138, 62.581711,
														1.092560, -2.199259, 48.921040,
														0.586423, -2.138805, 34.633698,
														0.099985, -2.515621, 19.550396,
														-4.402355, -2.674805, 98.238632,
														-4.379259, -3.473682, 107.576279,
														-4.781888, 5.331349, 107.380157,
														-4.471851, 3.351716, 98.178802,
														5.157888, 3.223763, 98.121567,
														7.003778, 4.456452, 107.384628,
														6.316382, -3.970959, 107.556442,
														5.361264, -3.196583, 98.226959,
														-2.100168, 4.546628, 97.920746,
														2.566453, 4.537404, 97.891342,
														2.900227, -4.115490, 98.035255,
														-1.575076, -3.927822, 98.012695,
														-4.784684, -2.316856, 100.913162,
														-4.941677, 3.305516, 100.822098,
														6.590655, 3.061493, 100.790977,
														5.915840, -2.832695, 100.892723,
														-1.958512, 4.879285, 100.978027,
														2.627915, 4.822053, 100.958572,
														2.958829, -4.154419, 101.472519,
														-1.227224, -3.984520, 101.471344,
														6.141220, -6.251899, 102.221481,
														6.414061, -7.333297, 107.194427,
														1.074660, -7.851779, 107.333084,
														-4.610565, -6.847911, 107.378983,
														-5.039687, -5.744300, 102.405838,
														1.895637, -31.120670, 95.092422,
														1.925205, -31.228951, 95.458954,
														0.791492, -31.723511, 95.026016,
														-0.010666, -31.704067, 94.990784,
														-1.085203, -31.124859, 95.350754,
														-1.254812, -31.011150, 94.978165,
														-0.211526, -31.586075, 94.616516,
														0.769409, -31.613083, 94.659927,
														5.503389, -11.806747, 102.833008,
														3.951998, -22.387203, 100.882767,
														5.686645, -12.542461, 106.348152,
														4.931853, -17.738579, 105.034973,
														4.080035, -22.856766, 102.754944,
														3.089825, -27.389305, 99.532295,
														1.073435, -18.812393, 104.915695,
														0.789641, -28.054207, 99.117805,
														1.086493, -13.072914, 106.559692,
														0.993720, -23.723972, 102.448410,
														-4.118896, -12.138500, 106.433441,
														-3.000721, -17.408981, 105.040924,
														-2.222830, -22.600422, 102.702744,
														-1.545119, -27.208572, 99.442642,
														-4.520717, -11.383587, 102.916397,
														-3.356556, -16.904469, 102.375839,
														-2.531435, -22.120926, 100.825623,
														-1.795674, -26.866867, 98.332397,
														4.776331, -17.248192, 102.373352,
														3.002346, -27.054867, 98.427963,
														-12.386702, 28.952370, 98.789261,
														-12.171432, 29.093346, 99.365143,
														-5.798985, 29.967010, 99.214951,
														8.086379, 28.876984, 99.176834,
														13.914124, 27.020508, 99.118164,
														13.847492, 26.981255, 98.443832,
														8.156349, 28.712029, 98.562912,
														-5.921177, 29.858025, 98.651154,
														-7.003762, 11.967457, 102.465637,
														-9.145967, 17.516953, 102.300095,
														-11.024023, 23.248642, 101.078888,
														-6.723549, 12.418083, 106.309235,
														-8.775763, 18.530907, 104.629715,
														-10.511821, 24.085823, 102.256454,
														0.946986, 13.830704, 106.432770,
														-2.192296, 19.768799, 104.512039,
														-3.023478, 25.008789, 102.241119,
														4.376859, 19.395287, 104.444107,
														5.482043, 24.479782, 102.127899,
														8.659883, 11.453320, 106.283890,
														10.788408, 17.083794, 104.563614,
														12.605414, 22.199188, 102.198486,
														8.329966, 11.292984, 102.392532,
														12.397736, 21.655228, 100.858368,
														10.480765, 16.374031, 102.128601,
														1.231530, 29.948357, 99.215302,
														1.205420, 29.811386, 98.626434,
														7.211511, 6.821710, 107.072479,
														6.823788, 5.857701, 101.329071,
														-5.012759, 2.121865, 98.191940,
														-4.986602, -1.356838, 98.225540,
														5.728124, 2.098198, 98.140045,
														6.049889, -2.059830, 98.208298,
														-5.070560, 0.928833, 107.478226,
														-5.353168, 0.494328, 100.867630,
														6.335255, 0.242745, 107.470535,
														6.281883, 0.114403, 100.841843,
														-5.489666, 0.382515, 98.208740,
														6.388949, 0.019185, 98.174179,
													};

const TArray<int> TestGeometry1::RawIndicesArray = {
														69, 66, 58,
														76, 73, 63,
														33, 29, 70,
														25, 36, 26,
														40, 3, 11,
														1, 0, 8,
														6, 41, 9,
														43, 46, 10,
														44, 5, 15,
														2, 4, 12,
														7, 42, 13,
														45, 47, 14,
														13, 53, 14,
														9, 49, 8,
														59, 5, 44,
														59, 28, 27,
														69, 60, 27,
														56, 3, 40,
														56, 32, 31,
														58, 57, 31,
														64, 42, 7,
														64, 75, 74,
														76, 65, 74,
														61, 41, 6,
														61, 39, 78,
														63, 62, 78,
														35, 47, 45,
														35, 19, 20,
														33, 34, 20,
														72, 0, 1,
														72, 21, 22,
														70, 71, 22,
														79, 46, 43,
														79, 16, 17,
														37, 38, 17,
														37, 18, 26,
														68, 4, 2,
														25, 67, 68,
														43, 3, 56,
														45, 5, 59,
														1, 41, 61,
														2, 42, 64,
														11, 3, 43,
														8, 0, 40,
														9, 41, 1,
														10, 46, 6,
														15, 5, 45,
														12, 4, 44,
														13, 42, 2,
														14, 47, 7,
														26, 66, 27,
														68, 67, 27,
														68, 28, 44,
														70, 29, 69,
														70, 30, 31,
														72, 71, 31,
														72, 32, 40,
														33, 73, 74,
														35, 34, 74,
														35, 75, 7,
														37, 36, 76,
														37, 77, 78,
														79, 38, 78,
														79, 39, 6,
														17, 16, 56,
														17, 57, 58,
														26, 18, 58,
														20, 19, 59,
														20, 60, 69,
														22, 21, 61,
														22, 62, 63,
														33, 23, 63,
														25, 24, 64,
														25, 65, 76,
														166, 81, 92,
														85, 94, 169,
														135, 140, 159,
														108, 107, 112,
														135, 134, 133,
														107, 106, 105,
														110, 109, 108,
														138, 137, 136,
														168, 86, 81,
														84, 83, 162,
														163, 92, 80,
														164, 94, 84,
														96, 88, 89,
														90, 91, 99,
														83, 88, 96,
														87, 90, 98,
														99, 91, 80,
														97, 89, 84,
														101, 86, 95,
														86, 101, 102,
														104, 92, 81,
														92, 104, 99,
														98, 99, 104,
														131, 114, 117,
														117, 122, 119,
														119, 122, 125,
														125, 129, 128,
														129, 114, 131,
														132, 105, 106,
														114, 132, 118,
														115, 101, 100,
														116, 115, 113,
														106, 107, 120,
														118, 120, 122,
														102, 101, 115,
														121, 115, 116,
														108, 120, 107,
														120, 108, 109,
														122, 120, 126,
														123, 103, 102,
														124, 123, 121,
														109, 110, 130,
														126, 130, 129,
														104, 103, 123,
														127, 123, 124,
														105, 132, 130,
														132, 114, 129,
														104, 127, 113,
														113, 127, 128,
														142, 145, 144,
														145, 148, 147,
														150, 147, 148,
														147, 150, 153,
														152, 153, 157,
														155, 157, 142,
														133, 134, 146,
														143, 146, 145,
														144, 82, 93,
														134, 135, 149,
														146, 149, 148,
														82, 144, 147,
														151, 149, 158,
														149, 151, 150,
														151, 136, 137,
														150, 151, 154,
														154, 137, 138,
														153, 154, 156,
														85, 160, 161,
														94, 161, 97,
														140, 143, 159,
														97, 155, 96,
														133, 143, 140,
														157, 156, 143,
														96, 155, 141,
														136, 158, 159,
														151, 158, 136,
														158, 149, 135,
														159, 156, 139,
														14, 53, 12,
														15, 55, 12,
														82, 147, 160,
														86, 102, 81,
														81, 102, 103,
														161, 160, 152,
														160, 147, 152,
														9, 48, 11,
														11, 51, 10,
														82, 160, 85,
														97, 161, 155,
														112, 105, 110,
														139, 156, 138,
														96, 141, 93,
														98, 100, 95,
														93, 162, 83,
														167, 92, 163,
														169, 94, 164,
														95, 165, 87,
														159, 143, 156,
														87, 165, 163,
														87, 80, 91,
														164, 162, 170,
														83, 84, 89,
														166, 167, 93,
														168, 169, 95,
														85, 168, 166,
														167, 170, 162,
														169, 171, 165,
														171, 170, 163,
														163, 165, 171,
														165, 95, 169,
														162, 93, 167,
														166, 82, 85,
														95, 86, 168,
														93, 82, 166,
														89, 88, 83,
														170, 171, 164,
														91, 90, 87,
														163, 80, 87,
														164, 171, 169,
														163, 170, 167,
														110, 111, 112,
														10, 50, 11,
														11, 50, 9,
														152, 155, 161,
														12, 52, 15,
														12, 55, 14,
														159, 139, 136,
														143, 142, 157,
														161, 94, 85,
														156, 157, 153,
														138, 156, 154,
														154, 153, 150,
														137, 154, 151,
														150, 148, 149,
														148, 145, 146,
														149, 146, 134,
														93, 141, 144,
														145, 142, 143,
														146, 143, 133,
														142, 141, 155,
														157, 155, 152,
														153, 152, 147,
														147, 144, 145,
														144, 141, 142,
														128, 131, 113,
														113, 100, 104,
														129, 130, 132,
														130, 110, 105,
														124, 128, 127,
														123, 127, 104,
														129, 125, 126,
														130, 126, 109,
														121, 119, 124,
														102, 121, 123,
														126, 125, 122,
														109, 126, 120,
														116, 119, 121,
														115, 121, 102,
														122, 117, 118,
														120, 118, 106,
														113, 131, 116,
														100, 113, 115,
														118, 117, 114,
														106, 118, 132,
														131, 128, 129,
														128, 124, 125,
														125, 124, 119,
														119, 116, 117,
														117, 116, 131,
														104, 100, 98,
														81, 103, 104,
														95, 100, 101,
														84, 94, 97,
														80, 92, 99,
														98, 95, 87,
														96, 93, 83,
														99, 98, 90,
														89, 97, 96,
														162, 164, 84,
														81, 166, 168,
														136, 139, 138,
														108, 111, 110,
														105, 112, 107,
														133, 140, 135,
														112, 111, 108,
														159, 158, 135,
														169, 168, 85,
														92, 167, 166,
														76, 36, 25,
														64, 65, 25,
														63, 73, 33,
														63, 23, 22,
														61, 62, 22,
														69, 29, 20,
														59, 60, 20,
														58, 66, 26,
														58, 18, 17,
														56, 57, 17,
														6, 46, 79,
														78, 39, 79,
														78, 38, 37,
														76, 77, 37,
														7, 47, 35,
														74, 75, 35,
														74, 34, 33,
														40, 0, 72,
														31, 32, 72,
														31, 71, 70,
														69, 30, 70,
														44, 4, 68,
														27, 28, 68,
														27, 67, 26,
														7, 54, 14,
														2, 53, 13,
														44, 52, 12,
														45, 55, 15,
														6, 50, 10,
														1, 49, 9,
														40, 48, 8,
														43, 51, 11,
														64, 24, 2,
														61, 21, 1,
														59, 19, 45,
														56, 16, 43,
														68, 24, 25,
														2, 24, 68,
														26, 36, 37,
														17, 18, 37,
														17, 38, 79,
														43, 16, 79,
														22, 23, 70,
														22, 71, 72,
														1, 21, 72,
														20, 29, 33,
														20, 34, 35,
														45, 19, 35,
														78, 77, 63,
														78, 62, 61,
														6, 39, 61,
														74, 73, 76,
														74, 65, 64,
														7, 75, 64,
														31, 30, 58,
														31, 57, 56,
														40, 32, 56,
														27, 66, 69,
														27, 60, 59,
														44, 28, 59,
														8, 48, 9,
														14, 54, 13,
														14, 55, 45,
														13, 54, 7,
														12, 53, 2,
														15, 52, 44,
														10, 51, 43,
														9, 50, 6,
														8, 49, 1,
														11, 48, 40,
														26, 67, 25,
														70, 23, 33,
														63, 77, 76,
														58, 30, 69
													};

