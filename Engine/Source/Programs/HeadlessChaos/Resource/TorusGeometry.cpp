// Copyright Epic Games, Inc. All Rights Reserved.

#include "../Resource/TorusGeometry.h"
const TArray<float> TorusGeometry::RawVertexArray = {
													120.000000f, -0.000193f, -0.000002f,
													114.126831f, 37.081871f, -0.000002f,
													97.082130f, 70.534103f, -0.000002f,
													70.534348f, 97.081955f, -0.000002f,
													37.082165f, 114.126740f, -0.000002f,
													0.000116f, 120.000000f, -0.000002f,
													-37.081944f, 114.126816f, -0.000002f,
													-70.534164f, 97.082092f, -0.000002f,
													-97.082001f, 70.534286f, -0.000002f,
													-114.126762f, 37.082092f, -0.000002f,
													-120.000000f, 0.000039f, -0.000002f,
													-114.126793f, -37.082016f, -0.000002f,
													-97.082047f, -70.534225f, -0.000002f,
													-70.534225f, -97.082039f, -0.000002f,
													-37.082043f, -114.126778f, -0.000002f,
													-0.000005f, -120.000000f, -0.000002f,
													37.082035f, -114.126785f, -0.000002f,
													70.534225f, -97.082039f, -0.000002f,
													97.082039f, -70.534225f, -0.000002f,
													114.126785f, -37.082039f, -0.000002f,
													115.320885f, -0.000185f, -12.855755f,
													109.676727f, 35.635952f, -12.855755f,
													93.296646f, 67.783791f, -12.855755f,
													67.784035f, 93.296471f, -12.855755f,
													35.636234f, 109.676643f, -12.855755f,
													0.000111f, 115.320885f, -12.855755f,
													-35.636021f, 109.676712f, -12.855755f,
													-67.783852f, 93.296608f, -12.855755f,
													-93.296516f, 67.783966f, -12.855755f,
													-109.676659f, 35.636162f, -12.855755f,
													-115.320885f, 0.000038f, -12.855755f,
													-109.676689f, -35.636089f, -12.855755f,
													-93.296562f, -67.783905f, -12.855755f,
													-67.783905f, -93.296555f, -12.855755f,
													-35.636116f, -109.676674f, -12.855755f,
													-0.000005f, -115.320885f, -12.855755f,
													35.636112f, -109.676682f, -12.855755f,
													67.783913f, -93.296555f, -12.855755f,
													93.296555f, -67.783913f, -12.855755f,
													109.676682f, -35.636116f, -12.855755f,
													103.472961f, -0.000166f, -19.696156f,
													98.408676f, 31.974760f, -19.696156f,
													83.711464f, 60.819767f, -19.696156f,
													60.819984f, 83.711311f, -19.696156f,
													31.975012f, 98.408600f, -19.696156f,
													0.000100f, 103.472961f, -19.696156f,
													-31.974821f, 98.408661f, -19.696156f,
													-60.819824f, 83.711426f, -19.696156f,
													-83.711349f, 60.819931f, -19.696156f,
													-98.408615f, 31.974947f, -19.696156f,
													-103.472961f, 0.000034f, -19.696156f,
													-98.408646f, -31.974882f, -19.696156f,
													-83.711388f, -60.819874f, -19.696156f,
													-60.819874f, -83.711388f, -19.696156f,
													-31.974907f, -98.408630f, -19.696156f,
													-0.000005f, -103.472961f, -19.696156f,
													31.974901f, -98.408638f, -19.696156f,
													60.819881f, -83.711388f, -19.696156f,
													83.711388f, -60.819881f, -19.696156f,
													98.408638f, -31.974905f, -19.696156f,
													90.000000f, -0.000144f, -17.320509f,
													85.595123f, 27.811403f, -17.320509f,
													72.811600f, 52.900574f, -17.320509f,
													52.900764f, 72.811462f, -17.320509f,
													27.811625f, 85.595055f, -17.320509f,
													0.000087f, 90.000000f, -17.320509f,
													-27.811459f, 85.595108f, -17.320509f,
													-52.900623f, 72.811569f, -17.320509f,
													-72.811501f, 52.900715f, -17.320509f,
													-85.595070f, 27.811567f, -17.320509f,
													-90.000000f, 0.000029f, -17.320509f,
													-85.595093f, -27.811512f, -17.320509f,
													-72.811539f, -52.900665f, -17.320509f,
													-52.900665f, -72.811531f, -17.320509f,
													-27.811533f, -85.595085f, -17.320509f,
													-0.000004f, -90.000000f, -17.320509f,
													27.811527f, -85.595085f, -17.320509f,
													52.900673f, -72.811531f, -17.320509f,
													72.811531f, -52.900673f, -17.320509f,
													85.595085f, -27.811531f, -17.320509f,
													81.206146f, -0.000130f, -6.840407f,
													77.231667f, 25.093966f, -6.840407f,
													65.697212f, 47.731686f, -6.840407f,
													47.731857f, 65.697098f, -6.840407f,
													25.094164f, 77.231606f, -6.840407f,
													0.000078f, 81.206146f, -6.840407f,
													-25.094015f, 77.231659f, -6.840407f,
													-47.731731f, 65.697189f, -6.840407f,
													-65.697121f, 47.731812f, -6.840407f,
													-77.231621f, 25.094114f, -6.840407f,
													-81.206146f, 0.000026f, -6.840407f,
													-77.231644f, -25.094063f, -6.840407f,
													-65.697159f, -47.731770f, -6.840407f,
													-47.731770f, -65.697151f, -6.840407f,
													-25.094082f, -77.231628f, -6.840407f,
													-0.000004f, -81.206146f, -6.840407f,
													25.094078f, -77.231636f, -6.840407f,
													47.731773f, -65.697151f, -6.840407f,
													65.697151f, -47.731773f, -6.840407f,
													77.231636f, -25.094080f, -6.840407f,
													81.206146f, -0.000130f, 6.840396f,
													77.231667f, 25.093966f, 6.840396f,
													65.697212f, 47.731686f, 6.840396f,
													47.731857f, 65.697098f, 6.840396f,
													25.094164f, 77.231606f, 6.840396f,
													0.000078f, 81.206146f, 6.840396f,
													-25.094015f, 77.231659f, 6.840396f,
													-47.731731f, 65.697189f, 6.840396f,
													-65.697121f, 47.731812f, 6.840396f,
													-77.231621f, 25.094114f, 6.840396f,
													-81.206146f, 0.000026f, 6.840396f,
													-77.231644f, -25.094063f, 6.840396f,
													-65.697159f, -47.731770f, 6.840396f,
													-47.731770f, -65.697151f, 6.840396f,
													-25.094082f, -77.231628f, 6.840396f,
													-0.000004f, -81.206146f, 6.840396f,
													25.094078f, -77.231636f, 6.840396f,
													47.731773f, -65.697151f, 6.840396f,
													65.697151f, -47.731773f, 6.840396f,
													77.231636f, -25.094080f, 6.840396f,
													89.999992f, -0.000144f, 17.320503f,
													85.595116f, 27.811401f, 17.320503f,
													72.811592f, 52.900570f, 17.320503f,
													52.900757f, 72.811462f, 17.320503f,
													27.811623f, 85.595047f, 17.320503f,
													0.000087f, 89.999992f, 17.320503f,
													-27.811455f, 85.595100f, 17.320503f,
													-52.900620f, 72.811562f, 17.320503f,
													-72.811493f, 52.900711f, 17.320503f,
													-85.595062f, 27.811565f, 17.320503f,
													-89.999992f, 0.000029f, 17.320503f,
													-85.595085f, -27.811508f, 17.320503f,
													-72.811531f, -52.900661f, 17.320503f,
													-52.900661f, -72.811523f, 17.320503f,
													-27.811531f, -85.595078f, 17.320503f,
													-0.000004f, -89.999992f, 17.320503f,
													27.811525f, -85.595085f, 17.320503f,
													52.900669f, -72.811523f, 17.320503f,
													72.811523f, -52.900669f, 17.320503f,
													85.595085f, -27.811527f, 17.320503f,
													103.472954f, -0.000166f, 19.696157f,
													98.408676f, 31.974756f, 19.696157f,
													83.711456f, 60.819763f, 19.696157f,
													60.819981f, 83.711304f, 19.696157f,
													31.975010f, 98.408592f, 19.696157f,
													0.000100f, 103.472954f, 19.696157f,
													-31.974819f, 98.408653f, 19.696157f,
													-60.819820f, 83.711426f, 19.696157f,
													-83.711342f, 60.819923f, 19.696157f,
													-98.408607f, 31.974945f, 19.696157f,
													-103.472954f, 0.000034f, 19.696157f,
													-98.408638f, -31.974880f, 19.696157f,
													-83.711388f, -60.819870f, 19.696157f,
													-60.819870f, -83.711380f, 19.696157f,
													-31.974905f, -98.408623f, 19.696157f,
													-0.000005f, -103.472954f, 19.696157f,
													31.974899f, -98.408630f, 19.696157f,
													60.819874f, -83.711380f, 19.696157f,
													83.711380f, -60.819874f, 19.696157f,
													98.408630f, -31.974901f, 19.696157f,
													115.320877f, -0.000185f, 12.855764f,
													109.676720f, 35.635952f, 12.855764f,
													93.296638f, 67.783783f, 12.855764f,
													67.784027f, 93.296471f, 12.855764f,
													35.636230f, 109.676636f, 12.855764f,
													0.000111f, 115.320877f, 12.855764f,
													-35.636021f, 109.676704f, 12.855764f,
													-67.783852f, 93.296600f, 12.855764f,
													-93.296509f, 67.783966f, 12.855764f,
													-109.676651f, 35.636158f, 12.855764f,
													-115.320877f, 0.000038f, 12.855764f,
													-109.676682f, -35.636089f, 12.855764f,
													-93.296555f, -67.783905f, 12.855764f,
													-67.783905f, -93.296547f, 12.855764f,
													-35.636116f, -109.676666f, 12.855764f,
													-0.000005f, -115.320877f, 12.855764f,
													35.636108f, -109.676674f, 12.855764f,
													67.783913f, -93.296547f, 12.855764f,
													93.296547f, -67.783913f, 12.855764f,
													109.676674f, -35.636112f, 12.855764f
};

const TArray<int32> TorusGeometry::RawIndicesArray = {
														0, 1, 21,
														1, 2, 22,
														2, 3, 23,
														3, 4, 24,
														4, 5, 25,
														5, 6, 26,
														6, 7, 27,
														7, 8, 28,
														8, 9, 29,
														9, 10, 30,
														10, 11, 31,
														11, 12, 32,
														12, 13, 33,
														13, 14, 34,
														14, 15, 35,
														15, 16, 36,
														16, 17, 37,
														17, 18, 38,
														18, 19, 39,
														19, 0, 20,
														20, 21, 41,
														21, 22, 42,
														22, 23, 43,
														23, 24, 44,
														24, 25, 45,
														25, 26, 46,
														26, 27, 47,
														27, 28, 48,
														28, 29, 49,
														29, 30, 50,
														30, 31, 51,
														31, 32, 52,
														32, 33, 53,
														33, 34, 54,
														34, 35, 55,
														35, 36, 56,
														36, 37, 57,
														37, 38, 58,
														38, 39, 59,
														39, 20, 40,
														40, 41, 61,
														41, 42, 62,
														42, 43, 63,
														43, 44, 64,
														44, 45, 65,
														45, 46, 66,
														46, 47, 67,
														47, 48, 68,
														48, 49, 69,
														49, 50, 70,
														50, 51, 71,
														51, 52, 72,
														52, 53, 73,
														53, 54, 74,
														54, 55, 75,
														55, 56, 76,
														56, 57, 77,
														57, 58, 78,
														58, 59, 79,
														59, 40, 60,
														60, 61, 81,
														61, 62, 82,
														62, 63, 83,
														63, 64, 84,
														64, 65, 85,
														65, 66, 86,
														66, 67, 87,
														67, 68, 88,
														68, 69, 89,
														69, 70, 90,
														70, 71, 91,
														71, 72, 92,
														72, 73, 93,
														73, 74, 94,
														74, 75, 95,
														75, 76, 96,
														76, 77, 97,
														77, 78, 98,
														78, 79, 99,
														79, 60, 80,
														80, 81, 101,
														81, 82, 102,
														82, 83, 103,
														83, 84, 104,
														84, 85, 105,
														85, 86, 106,
														86, 87, 107,
														87, 88, 108,
														88, 89, 109,
														89, 90, 110,
														90, 91, 111,
														91, 92, 112,
														92, 93, 113,
														93, 94, 114,
														94, 95, 115,
														95, 96, 116,
														96, 97, 117,
														97, 98, 118,
														98, 99, 119,
														99, 80, 100,
														100, 101, 121,
														101, 102, 122,
														102, 103, 123,
														103, 104, 124,
														104, 105, 125,
														105, 106, 126,
														106, 107, 127,
														107, 108, 128,
														108, 109, 129,
														109, 110, 130,
														110, 111, 131,
														111, 112, 132,
														112, 113, 133,
														113, 114, 134,
														114, 115, 135,
														115, 116, 136,
														116, 117, 137,
														117, 118, 138,
														118, 119, 139,
														119, 100, 120,
														120, 121, 141,
														121, 122, 142,
														122, 123, 143,
														123, 124, 144,
														124, 125, 145,
														125, 126, 146,
														126, 127, 147,
														127, 128, 148,
														128, 129, 149,
														129, 130, 150,
														130, 131, 151,
														131, 132, 152,
														132, 133, 153,
														133, 134, 154,
														134, 135, 155,
														135, 136, 156,
														136, 137, 157,
														137, 138, 158,
														138, 139, 159,
														139, 120, 140,
														140, 141, 161,
														141, 142, 162,
														142, 143, 163,
														143, 144, 164,
														144, 145, 165,
														145, 146, 166,
														146, 147, 167,
														147, 148, 168,
														148, 149, 169,
														149, 150, 170,
														150, 151, 171,
														151, 152, 172,
														152, 153, 173,
														153, 154, 174,
														154, 155, 175,
														155, 156, 176,
														156, 157, 177,
														157, 158, 178,
														158, 159, 179,
														159, 140, 160,
														160, 161, 1,
														161, 162, 2,
														162, 163, 3,
														163, 164, 4,
														164, 165, 5,
														165, 166, 6,
														166, 167, 7,
														167, 168, 8,
														168, 169, 9,
														169, 170, 10,
														170, 171, 11,
														171, 172, 12,
														172, 173, 13,
														173, 174, 14,
														174, 175, 15,
														175, 176, 16,
														176, 177, 17,
														177, 178, 18,
														178, 179, 19,
														179, 160, 0,
														0, 19, 179,
														19, 18, 178,
														18, 17, 177,
														17, 16, 176,
														16, 15, 175,
														15, 14, 174,
														14, 13, 173,
														13, 12, 172,
														12, 11, 171,
														11, 10, 170,
														10, 9, 169,
														9, 8, 168,
														8, 7, 167,
														7, 6, 166,
														6, 5, 165,
														5, 4, 164,
														4, 3, 163,
														3, 2, 162,
														2, 1, 161,
														1, 0, 160,
														160, 179, 159,
														179, 178, 158,
														178, 177, 157,
														177, 176, 156,
														176, 175, 155,
														175, 174, 154,
														174, 173, 153,
														173, 172, 152,
														172, 171, 151,
														171, 170, 150,
														170, 169, 149,
														169, 168, 148,
														168, 167, 147,
														167, 166, 146,
														166, 165, 145,
														165, 164, 144,
														164, 163, 143,
														163, 162, 142,
														162, 161, 141,
														161, 160, 140,
														140, 159, 139,
														159, 158, 138,
														158, 157, 137,
														157, 156, 136,
														156, 155, 135,
														155, 154, 134,
														154, 153, 133,
														153, 152, 132,
														152, 151, 131,
														151, 150, 130,
														150, 149, 129,
														149, 148, 128,
														148, 147, 127,
														147, 146, 126,
														146, 145, 125,
														145, 144, 124,
														144, 143, 123,
														143, 142, 122,
														142, 141, 121,
														141, 140, 120,
														120, 139, 119,
														139, 138, 118,
														138, 137, 117,
														137, 136, 116,
														136, 135, 115,
														135, 134, 114,
														134, 133, 113,
														133, 132, 112,
														132, 131, 111,
														131, 130, 110,
														130, 129, 109,
														129, 128, 108,
														128, 127, 107,
														127, 126, 106,
														126, 125, 105,
														125, 124, 104,
														124, 123, 103,
														123, 122, 102,
														122, 121, 101,
														121, 120, 100,
														100, 119, 99,
														119, 118, 98,
														118, 117, 97,
														117, 116, 96,
														116, 115, 95,
														115, 114, 94,
														114, 113, 93,
														113, 112, 92,
														112, 111, 91,
														111, 110, 90,
														110, 109, 89,
														109, 108, 88,
														108, 107, 87,
														107, 106, 86,
														106, 105, 85,
														105, 104, 84,
														104, 103, 83,
														103, 102, 82,
														102, 101, 81,
														101, 100, 80,
														80, 99, 79,
														99, 98, 78,
														98, 97, 77,
														97, 96, 76,
														96, 95, 75,
														95, 94, 74,
														94, 93, 73,
														93, 92, 72,
														92, 91, 71,
														91, 90, 70,
														90, 89, 69,
														89, 88, 68,
														88, 87, 67,
														87, 86, 66,
														86, 85, 65,
														85, 84, 64,
														84, 83, 63,
														83, 82, 62,
														82, 81, 61,
														81, 80, 60,
														60, 79, 59,
														79, 78, 58,
														78, 77, 57,
														77, 76, 56,
														76, 75, 55,
														75, 74, 54,
														74, 73, 53,
														73, 72, 52,
														72, 71, 51,
														71, 70, 50,
														70, 69, 49,
														69, 68, 48,
														68, 67, 47,
														67, 66, 46,
														66, 65, 45,
														65, 64, 44,
														64, 63, 43,
														63, 62, 42,
														62, 61, 41,
														61, 60, 40,
														40, 59, 39,
														59, 58, 38,
														58, 57, 37,
														57, 56, 36,
														56, 55, 35,
														55, 54, 34,
														54, 53, 33,
														53, 52, 32,
														52, 51, 31,
														51, 50, 30,
														50, 49, 29,
														49, 48, 28,
														48, 47, 27,
														47, 46, 26,
														46, 45, 25,
														45, 44, 24,
														44, 43, 23,
														43, 42, 22,
														42, 41, 21,
														41, 40, 20,
														20, 39, 19,
														39, 38, 18,
														38, 37, 17,
														37, 36, 16,
														36, 35, 15,
														35, 34, 14,
														34, 33, 13,
														33, 32, 12,
														32, 31, 11,
														31, 30, 10,
														30, 29, 9,
														29, 28, 8,
														28, 27, 7,
														27, 26, 6,
														26, 25, 5,
														25, 24, 4,
														24, 23, 3,
														23, 22, 2,
														22, 21, 1,
														21, 20, 0
};