// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/Real.h"
#include "Chaos/Array.h"

class TorusGeometry
{

public:
	TorusGeometry() {}
	~TorusGeometry() {}

	static const TArray<float>	RawVertexArray;
	static const TArray<int32>	RawIndicesArray;
};
